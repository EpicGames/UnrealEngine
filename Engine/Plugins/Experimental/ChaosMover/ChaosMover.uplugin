{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Chaos Mover",
	"Description": "Chaos Mover is an Unreal Engine plugin to support Chaos physics based character movement using the Mover framework.",
	"Category": "Physics",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "ChaosMover",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "Mover",
			"Enabled": true
		}
	]
}