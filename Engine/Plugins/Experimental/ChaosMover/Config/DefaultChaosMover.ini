[CoreRedirects]
+PropertyRedirects=(OldName="/Script/ChaosMover.ChaosCharacterMoverComponent.OnLanded",NewName="/Script/ChaosMover.ChaosCharacterMoverComponent.OnLandedDelegate")

[/Script/Engine.PhysicsSettings]
PhysicsPrediction=(bEnablePhysicsPrediction=True)