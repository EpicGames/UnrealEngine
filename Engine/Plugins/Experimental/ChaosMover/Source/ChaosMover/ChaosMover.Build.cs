// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class ChaosMover : ModuleRules
	{
		public ChaosMover(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"GameplayTags",
					"InputCore",
					"NetCore",
					"Mover",
					"Water"
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"CoreUObject",
					"DeveloperSettings",
					"Engine",
					"MotionWarping"
				}
			);

			SetupModulePhysicsSupport(Target);
		}
	}
}