// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosMoverAsyncCallback.h"

#include "Chaos/Framework/Parallel.h"
#include "Chaos/ResimConsoleVariables.h"
#include "ChaosMover/Backends/ChaosMoverBackend.h"
#include "ChaosMover/ChaosMoverConsoleVariables.h"
#include "ChaosMover/ChaosMoverSimulation.h"
#include "ChaosMover/ChaosMoverSimulationTypes.h"
#include "ChaosMoverSubsystem.h"
#include "PBDRigidsSolver.h"

namespace UE::ChaosMover
{
	void FAsyncCallback::OnInjectInputs_External(int32 PhysicsStep, int32 NumSteps)
	{
		QUICK_SCOPE_CYCLE_COUNTER(ChaosMover_AsyncCallback_OnInjectInputs_External);

		if (Subsystem)
		{
			TArray<TWeakObjectPtr<UChaosMoverBackendComponent>>& Backends = Subsystem->ValidateAndGetBackends();
			const FMoverTimeStep& TimeStep = Subsystem->GetMoverTimeStep();
			const int32 NetPhysicsTickOffset = Subsystem->GetNetworkPhysicsTickOffset();

			UE::ChaosMover::FAsyncCallbackInput* AsyncInput = GetProducerInputData_External();
			AsyncInput->Reset();
			AsyncInput->InputData.SetNum(Backends.Num());
			AsyncInput->PhysicsSolver = static_cast<Chaos::FPhysicsSolver*>(GetSolver());
			AsyncInput->NetworkPhysicsTickOffset = NetPhysicsTickOffset;
			for (int Idx = 0; Idx < Backends.Num(); ++Idx)
			{
				TWeakObjectPtr<UChaosMoverBackendComponent>& Backend = Backends[Idx];
				AsyncInput->Backends.Add(Backend);
				if (Backend.Get())
				{
					Backend->ProduceInputData(PhysicsStep, NumSteps, TimeStep, AsyncInput->InputData[Idx]);
				}
			}
		}
	}

	void FAsyncCallback::OnProcessInputs_Internal(int32 PhysicsStep)
	{
		QUICK_SCOPE_CYCLE_COUNTER(ChaosMover_AsyncCallback_OnProcessInputs_Internal);

		const FAsyncCallbackInput* AsyncInput = GetConsumerInput_Internal();

		if (!AsyncInput || AsyncInput->InputData.IsEmpty())
		{
			return;
		}

		FMoverTimeStep TimeStep;
		GetCurrentMoverTimeStep(AsyncInput, TimeStep);

		// This array of resimulated backend indices is only used in the resim case
		const TArray<int32, TInlineAllocator<64>>* ResimulatedBackendIndicesPtr = TimeStep.bIsResimulating ? DetermineResimulatedBackendIndices(AsyncInput->Backends) : nullptr;

		auto LambdaParallelUpdate = [&TimeStep, &AsyncInput, PhysicsStep, ResimulatedBackendIndicesPtr](int32 Idx) {
			int32 BackendIndex = ResimulatedBackendIndicesPtr ? (*ResimulatedBackendIndicesPtr)[Idx] : Idx;
			if (TStrongObjectPtr<UChaosMoverBackendComponent> Backend = AsyncInput->Backends[BackendIndex].Pin())
			{				
				Backend->GetSimulation()->ProcessInputs(PhysicsStep, TimeStep, AsyncInput->InputData[BackendIndex]);
			}
			};

		// Backends are added to AsyncInput in InjectInputs_EXTERNAL
		const int32 NumBackends = AsyncInput->Backends.Num();
		const int32 NumSimulatedBackends = ResimulatedBackendIndicesPtr ? ResimulatedBackendIndicesPtr->Num() : NumBackends;
		const bool ForceSingleThread = UE::ChaosMover::CVars::bForceSingleThreadedPT;
		Chaos::PhysicsParallelFor(NumSimulatedBackends, LambdaParallelUpdate, ForceSingleThread);
	}

	void FAsyncCallback::OnPreSimulate_Internal()
	{
		QUICK_SCOPE_CYCLE_COUNTER(ChaosMover_AsyncCallback_OnPreSimulate_Internal);

		const FAsyncCallbackInput* AsyncInput = GetConsumerInput_Internal();

		if (!AsyncInput || AsyncInput->InputData.IsEmpty())
		{
			return;
		}

		FMoverTimeStep TimeStep;
		GetCurrentMoverTimeStep(AsyncInput, TimeStep);

		FAsyncCallbackOutput& AsyncOutput = GetProducerOutputData_Internal();
		
		const int32 NumInputs = AsyncInput->InputData.Num();
		AsyncOutput.OutputData.SetNum(NumInputs);
		AsyncOutput.TimeStep.SetNum(NumInputs);
		AsyncOutput.RollbackNewSyncState.SetNum(NumInputs);

		const int32 NumBackends = AsyncInput->Backends.Num();
		ensure(NumInputs == NumBackends);
		AsyncOutput.Backends = AsyncInput->Backends;

		// This array of resimulated backend indices is only used in the resim case
		const TArray<int32, TInlineAllocator<64>>* ResimulatedBackendIndicesPtr = (TimeStep.bIsResimulating && Chaos::ResimConsoleVars::bIsBubbleResimEnabled) ? &ResimulatedBackendIndices : nullptr;
		
		auto LambdaParallelUpdate = [&TimeStep, &AsyncInput, &AsyncOutput, ResimulatedBackendIndicesPtr, this](int32 Idx) {
			int32 BackendIndex = ResimulatedBackendIndicesPtr ? (*ResimulatedBackendIndicesPtr)[Idx] : Idx;
			AsyncOutput.TimeStep[BackendIndex] = TimeStep;
			if (TStrongObjectPtr<UChaosMoverBackendComponent> Backend = AsyncInput->Backends[BackendIndex].Pin())
			{
				FMoverSimContext SimContext;
				SimContext.Simulation = Backend->GetSimulation();
				SimContext.Blackboard = SimContext.Simulation->GetRollbackBlackboardSimWrapper();
				SimContext.SimulationOwner = nullptr;	// Not required for ChaosMover modes

				if (TimeStep.bIsFirstResimFrame)
				{
					AsyncOutput.RollbackNewSyncState[BackendIndex] = Backend->GetSimulation()->GetCurrentSyncState();
				}

				Backend->GetSimulation()->SimulationTick(SimContext, TimeStep, AsyncInput->InputData[BackendIndex], AsyncOutput.OutputData[BackendIndex]);
			}
		};

		const bool ForceSingleThread = UE::ChaosMover::CVars::bForceSingleThreadedPT;
		const int32 NumSimulatedBackends = ResimulatedBackendIndicesPtr ? ResimulatedBackendIndicesPtr->Num() : NumBackends;
		Chaos::PhysicsParallelFor(NumSimulatedBackends, LambdaParallelUpdate, ForceSingleThread);
	}

	void FAsyncCallback::OnContactModification_Internal(Chaos::FCollisionContactModifier& Modifier)
	{
		QUICK_SCOPE_CYCLE_COUNTER(ChaosMover_OnContactModification_Internal);

		const FAsyncCallbackInput* AsyncInput = GetConsumerInput_Internal();

		if (!AsyncInput)
		{
			return;
		}

		FMoverTimeStep TimeStep;
		GetCurrentMoverTimeStep(AsyncInput, TimeStep);

		// This array of resimulated backend indices is only used in the resim case
		const TArray<int32, TInlineAllocator<64>>* ResimulatedBackendIndicesPtr =
			(TimeStep.bIsResimulating && Chaos::ResimConsoleVars::bIsBubbleResimEnabled && !Chaos::ResimConsoleVars::bCanFrozenParticlesModifyContacts) ?
			&ResimulatedBackendIndices : nullptr;

		const int32 NumSimulatedBackends = ResimulatedBackendIndicesPtr ? ResimulatedBackendIndicesPtr->Num() : AsyncInput->Backends.Num();

		const FAsyncCallbackOutput& AsyncOutput = GetProducerOutputData_Internal();
		for (int Idx = 0; Idx < NumSimulatedBackends; ++Idx)
		{
			int32 BackendIndex = ResimulatedBackendIndicesPtr ? (*ResimulatedBackendIndicesPtr)[Idx] : Idx;
			if (TStrongObjectPtr<UChaosMoverBackendComponent> Backend = AsyncInput->Backends[BackendIndex].Pin())
			{
				Backend->GetSimulation()->ModifyContacts(TimeStep, AsyncInput->InputData[BackendIndex], AsyncOutput.OutputData[BackendIndex], Modifier);
			}
		}
	}

	void FAsyncCallback::OnPostSolve_Internal()
	{
#if WITH_CHAOS_VISUAL_DEBUGGER
		if (FChaosVisualDebuggerTrace::IsTracing())
		{
			QUICK_SCOPE_CYCLE_COUNTER(ChaosMover_AsyncCallback_OnPostSolve_Internal);

			const FAsyncCallbackInput* AsyncInput = GetConsumerInput_Internal();

			if (!AsyncInput || AsyncInput->InputData.IsEmpty())
			{
				return;
			}

			FMoverTimeStep TimeStep;
			GetCurrentMoverTimeStep(AsyncInput, TimeStep);

			// This array of resimulated backend indices is only used in the resim case
			const TArray<int32, TInlineAllocator<64>>* ResimulatedBackendIndicesPtr =
				(TimeStep.bIsResimulating && Chaos::ResimConsoleVars::bIsBubbleResimEnabled && !Chaos::ResimConsoleVars::bCanFrozenParticlesModifyContacts) ?
				&ResimulatedBackendIndices : nullptr;

			const int32 NumSimulatedBackends = ResimulatedBackendIndicesPtr ? ResimulatedBackendIndicesPtr->Num() : AsyncInput->Backends.Num();

			const FAsyncCallbackOutput& AsyncOutput = GetProducerOutputData_Internal();
			for (int Idx = 0; Idx < NumSimulatedBackends; ++Idx)
			{
				int32 BackendIndex = ResimulatedBackendIndicesPtr ? (*ResimulatedBackendIndicesPtr)[Idx] : Idx;
				if (TStrongObjectPtr<UChaosMoverBackendComponent> Backend = AsyncInput->Backends[BackendIndex].Pin())
				{
					Backend->GetSimulation()->TraceMoverData(TimeStep, AsyncOutput.OutputData[BackendIndex]);
				}
			}
		}
#endif // WITH_CHAOS_VISUAL_DEBUGGER
	}

	void FAsyncCallback::GetCurrentMoverTimeStep(const FAsyncCallbackInput* AsyncInput, FMoverTimeStep& OutMoverTimeStep) const
	{
		OutMoverTimeStep.BaseSimTimeMs = GetSimTime_Internal() * 1000.0;
		OutMoverTimeStep.StepMs = GetDeltaTime_Internal() * 1000.0f;
		if (AsyncInput && AsyncInput->PhysicsSolver)
		{
			OutMoverTimeStep.ServerFrame = AsyncInput->PhysicsSolver->GetCurrentFrame() + AsyncInput->NetworkPhysicsTickOffset;
			OutMoverTimeStep.bIsResimulating = AsyncInput->PhysicsSolver->GetEvolution()->IsResimming();
			OutMoverTimeStep.bIsFirstResimFrame = OutMoverTimeStep.bIsResimulating && (AsyncInput->PhysicsSolver->GetCurrentFrame() == AsyncInput->PhysicsSolver->GetRewindData()->GetResimFrame());
		}
	}

	const TArray<int32, TInlineAllocator<64>>* FAsyncCallback::DetermineResimulatedBackendIndices(const TArray<TWeakObjectPtr<UChaosMoverBackendComponent>>& InBackends)
	{
		if (Chaos::ResimConsoleVars::bIsBubbleResimEnabled)
		{
			QUICK_SCOPE_CYCLE_COUNTER(ChaosMover_DetermineResimulatedBackendIndices);

			int32 NumBackends = InBackends.Num();
			ResimulatedBackendIndices.Reserve(NumBackends);
			ResimulatedBackendIndices.Empty();

			for (int32 BackendIndex = 0; BackendIndex < NumBackends; ++BackendIndex)
			{
				if (TStrongObjectPtr<UChaosMoverBackendComponent> Backend = InBackends[BackendIndex].Pin())
				{
					if (Backend->ShouldResim())
					{
						ResimulatedBackendIndices.Add(BackendIndex);
					}
				}
			}

			return &ResimulatedBackendIndices;
		}

		return nullptr;
	}
}