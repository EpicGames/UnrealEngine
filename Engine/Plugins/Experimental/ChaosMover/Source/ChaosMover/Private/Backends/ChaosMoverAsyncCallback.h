// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/SimCallbackObject.h"
#include "ChaosMover/ChaosMoverSimulationTypes.h"

class UChaosMoverBackendComponent;
class UChaosMoverSubsystem;

namespace UE::ChaosMover
{
	struct FAsyncCallbackInput : public Chaos::FSimCallbackInput
	{
		TArray<FSimulationInputData> InputData;
		TArray<TWeakObjectPtr<UChaosMoverBackendComponent>> Backends;
		Chaos::FPhysicsSolver* PhysicsSolver;
		int32 NetworkPhysicsTickOffset;

		void Reset()
		{
			InputData.Empty();
			Backends.Empty();
			PhysicsSolver = nullptr;
			NetworkPhysicsTickOffset = 0;
		}
	};

	struct FAsyncCallbackOutput : public Chaos::FSimCallbackOutput
	{
		TArray<FSimulationOutputData> OutputData;
		TArray<TWeakObjectPtr<UChaosMoverBackendComponent>> Backends;
		TArray<FMoverTimeStep> TimeStep;
		// Sync state at the rollback destination frame (NewBaseTimeStep), populated only when bIsFirstResimFrame is true.
		TArray<FMoverSyncState> RollbackNewSyncState;
		void Reset()
		{
			OutputData.Empty();
			Backends.Empty();
			TimeStep.Empty();
			RollbackNewSyncState.Empty();
		}
	};

	class FAsyncCallback : public Chaos::TSimCallbackObject<
		FAsyncCallbackInput,
		FAsyncCallbackOutput,
		Chaos::ESimCallbackOptions::InjectInputsExternal |
		Chaos::ESimCallbackOptions::ProcessInputsInternal |
		Chaos::ESimCallbackOptions::Presimulate |
		Chaos::ESimCallbackOptions::ContactModification |
		Chaos::ESimCallbackOptions::PostSolve |
		Chaos::ESimCallbackOptions::Rewind>
	{
	public:
		void SetSubsystem(UChaosMoverSubsystem* InSubsystem)
		{
			Subsystem = InSubsystem;
		}

	protected:
		virtual void OnInjectInputs_External(int32 PhysicsStep, int32 NumSteps) override;
		virtual void OnProcessInputs_Internal(int32 PhysicsStep) override;
		virtual void OnPreSimulate_Internal() override;
		virtual void OnContactModification_Internal(Chaos::FCollisionContactModifier& Modifier) override;
		virtual void OnPostSolve_Internal() override;

		void GetCurrentMoverTimeStep(const FAsyncCallbackInput* AsyncInput, FMoverTimeStep& OutMoverTimeStep) const;

		// With bubble resim enabled, determines which backends should resimulate and returns a pointer to ResimulatedBackendIndices
		// Only call this function during resim frames
		const TArray<int32, TInlineAllocator<64>>* DetermineResimulatedBackendIndices(const TArray<TWeakObjectPtr<UChaosMoverBackendComponent>>& InBackends);

		// Array of resimulated backend indices. This array is used only in the resim case.
		// During a resim frame, it is reset and filled in OnProcessInputs_Internal and reused in subsequent callbacks on that same frame, such as OnPreSimulate_Internal
		TArray<int32, TInlineAllocator<64>> ResimulatedBackendIndices;

		UChaosMoverSubsystem* Subsystem = nullptr;
	};
}