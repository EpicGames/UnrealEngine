// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosMoverSubsystem.h"

#include "ChaosMoverAsyncCallback.h"

#include "ChaosMover/Backends/ChaosMoverBackend.h"
#include "ChaosMover/ChaosMoverConsoleVariables.h"
#include "ChaosMover/ChaosMoverLog.h"
#include "GameFramework/PlayerController.h"
#include "PBDRigidsSolver.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "Physics/NetworkPhysicsComponent.h"
#include "PhysicsEngine/PhysicsSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosMoverSubsystem)

bool UChaosMoverSubsystem::DoesSupportWorldType(const EWorldType::Type WorldType) const
{
	return WorldType == EWorldType::Game || WorldType == EWorldType::PIE;
}

void UChaosMoverSubsystem::OnWorldBeginPlay(UWorld& InWorld)
{
	Super::OnWorldBeginPlay(InWorld);

	if (FPhysScene* PhysScene = InWorld.GetPhysicsScene())
	{
		PhysScenePostTickCallbackHandle = PhysScene->OnPhysScenePostTick.AddUObject(this, &UChaosMoverSubsystem::OnPostPhysicsTick);

		if (Chaos::FPhysicsSolver* Solver = PhysScene->GetSolver())
		{
			AsyncCallback = Solver->CreateAndRegisterSimCallbackObject_External<UE::ChaosMover::FAsyncCallback>();
			AsyncCallback->SetSubsystem(this);
		}
	}
}

void UChaosMoverSubsystem::Deinitialize()
{
	if (UWorld* World = GetWorld())
	{
		if (FPhysScene* PhysScene = World->GetPhysicsScene())
		{
			if (PhysScenePostTickCallbackHandle.IsValid())
			{
				PhysScene->OnPhysScenePostTick.Remove(PhysScenePostTickCallbackHandle);
			}

			if (Chaos::FPhysicsSolver* Solver = PhysScene->GetSolver())
			{
				if (AsyncCallback)
				{
					AsyncCallback->SetSubsystem(nullptr);
					Solver->UnregisterAndFreeSimCallbackObject_External(AsyncCallback);
					AsyncCallback = nullptr;
				}
			}
		}
	}

	Super::Deinitialize();
}

void UChaosMoverSubsystem::Register(TWeakObjectPtr<UChaosMoverBackendComponent> InBackend)
{
	if (Backends.Num() == 0 && InBackend.IsValid())
	{
		if (UWorld* World = GetWorld())
		{
			UPhysicsSettings* PhysicsSettings = UPhysicsSettings::Get();
			bool bProjectSettingAsyncEnabled = PhysicsSettings ? PhysicsSettings->bTickPhysicsAsync : false;
			bool bProjectSettingPhysicsPredictionEnabled = PhysicsSettings ? PhysicsSettings->PhysicsPrediction.bEnablePhysicsPrediction : false;

			if (PhysicsSettings)
			{
				if (World->GetNetMode() != ENetMode::NM_Standalone)
				{
					if (!bProjectSettingAsyncEnabled)
					{
						UE_LOGF(LogChaosMover, Warning, "ChaosMoverSubsystem: bTickPhysicsAsync project setting is disabled. bTickPhysicsAsync is required for networked physics to replicate correctly. Enable 'Tick Physics Async' in Project Settings > Engine > Physics.");
#if !UE_BUILD_SHIPPING
						UE_DEBUG_BREAK();
#endif
					}

					if (!bProjectSettingPhysicsPredictionEnabled)
					{
						UE_LOGF(LogChaosMover, Warning, "ChaosMoverSubsystem: bEnablePhysicsPrediction project setting is disabled. Enable 'Enable Physics Prediction' in Project Settings > Engine > Physics.");
#if !UE_BUILD_SHIPPING
						UE_DEBUG_BREAK();
#endif
					}
				}
			}
		}
	}

	Backends.AddUnique(InBackend);
}

void UChaosMoverSubsystem::Unregister(TWeakObjectPtr<UChaosMoverBackendComponent> InBackend)
{
	Backends.Remove(InBackend);
}

TArray<TWeakObjectPtr<UChaosMoverBackendComponent>>& UChaosMoverSubsystem::ValidateAndGetBackends()
{
	// Clear invalid data before we attempt to use it.
	Backends.RemoveAllSwap([](const TWeakObjectPtr<UChaosMoverBackendComponent> Backend) {
		return !Backend.IsValid();
		});

	return Backends;
}

void UChaosMoverSubsystem::OnPostPhysicsTick(FChaosScene* Scene)
{
	if (!AsyncCallback)
	{
		return;
	}

	QUICK_SCOPE_CYCLE_COUNTER(ChaosMover_OnPostPhysicsTick);

	const bool ForceSingleThread = UE::ChaosMover::CVars::bForceSingleThreadedGT;

	// Pop each async output which START TIME is <= ResultsTimeInMs
	// Since they are ordered, this ensures that we will pop the last output which end time is greater than ResultsTimeInMs, if there is such an output available,
	// but not the ones after that, which will remain in the queue
	while (Chaos::TSimCallbackOutputHandle<UE::ChaosMover::FAsyncCallbackOutput> AsyncOutput = AsyncCallback->PopOutputData_External())
	{
		auto LambdaParallelUpdate = [&AsyncOutput](int32 Idx) {
			if (UChaosMoverBackendComponent* Backend = AsyncOutput->Backends[Idx].Get())
			{
				// In the async path, UMoverComponent::OnSimulationRollback is never called on the GT.
				// The subsystem is the first to know about a rollback via bIsFirstResimFrame, so
				// notify the backend here before consuming output for this resim sequence.
				if (AsyncOutput->TimeStep[Idx].bIsFirstResimFrame)
				{
					Backend->OnSimulationRollback(AsyncOutput->RollbackNewSyncState[Idx], AsyncOutput->TimeStep[Idx], Backend->GetPreRollbackTimeStep());
				}

				Backend->ConsumeOutputData(AsyncOutput->TimeStep[Idx], AsyncOutput->OutputData[Idx]);
			}
		};

		// AsyncOutput->Backends is reset and filled each physics frame in FAsyncCallback::OnPreSimulate_Internal
		Chaos::PhysicsParallelFor(AsyncOutput->Backends.Num(), LambdaParallelUpdate, ForceSingleThread);
	}

	double ResultsTimeInMs = 0.0;
	if (Scene)
	{
		if (Chaos::FPhysicsSolver* Solver = Scene->GetSolver())
		{
			ResultsTimeInMs = Solver->GetPhysicsResultsTime_External() * 1000.0;
		}
	}

	// Call finalize frame
	auto LambdaParallelUpdate = [this, ResultsTimeInMs](int32 Idx) {
		if (UChaosMoverBackendComponent* Backend = Backends[Idx].Get())
		{
			Backend->FinalizeFrame(ResultsTimeInMs);
		}
	};

	Chaos::PhysicsParallelFor(Backends.Num(), LambdaParallelUpdate, ForceSingleThread);
}

int32 UChaosMoverSubsystem::GetNetworkPhysicsTickOffset() const
{
	if (UWorld* World = GetWorld())
	{
		if (APlayerController* PlayerController = World->GetFirstPlayerController())
		{
			return PlayerController->GetNetworkPhysicsTickOffset();
		}
	}

	return 0;
}

FMoverTimeStep UChaosMoverSubsystem::GetMoverTimeStep() const
{
	FMoverTimeStep TimeStep;
	if (UWorld* World = GetWorld())
	{
		if (FPhysScene* Scene = World->GetPhysicsScene())
		{
			if (Chaos::FPhysicsSolver* Solver = Scene->GetSolver())
			{
				TimeStep.BaseSimTimeMs = Solver->GetPhysicsResultsTime_External() * 1000.0;

				if (Solver->IsUsingAsyncResults())
				{
					const float AsyncDt = Solver->GetAsyncDeltaTime();
					if (AsyncDt > UE_SMALL_NUMBER)
					{
						TimeStep.ServerFrame = FMath::FloorToInt(Solver->GetPhysicsResultsTime_External() / AsyncDt);
					}
					else
					{
						TimeStep.ServerFrame = Solver->GetCurrentFrame();
					}

					TimeStep.StepMs = AsyncDt * 1000.0f;
				}
				else
				{
					TimeStep.ServerFrame = Solver->GetCurrentFrame();
					TimeStep.StepMs = FMath::Clamp(World->GetDeltaSeconds(), Solver->GetMinDeltaTime_External(), Solver->GetMaxDeltaTime_External()) * 1000.0f;
				}

				TimeStep.ServerFrame += GetNetworkPhysicsTickOffset();
			}
		}
	}

	return TimeStep;
}
