// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/Declares.h"
#include "Subsystems/WorldSubsystem.h"

#include "ChaosMoverSubsystem.generated.h"

namespace UE::ChaosMover
{
	class FAsyncCallback;
	class FSimulation;
}

class FChaosScene;
class UChaosMoverBackendComponent;
struct FMoverTimeStep;

UCLASS()
class UChaosMoverSubsystem : public UWorldSubsystem
{
	GENERATED_BODY()

public:
	void Register(TWeakObjectPtr<UChaosMoverBackendComponent> Backend);
	void Unregister(TWeakObjectPtr<UChaosMoverBackendComponent> Backend);

	// Remove invalid backends before returning
	TArray<TWeakObjectPtr<UChaosMoverBackendComponent>>& ValidateAndGetBackends();

	FMoverTimeStep GetMoverTimeStep() const;
	int32 GetNetworkPhysicsTickOffset() const;

protected:
	virtual bool DoesSupportWorldType(const EWorldType::Type WorldType) const override;
	virtual void OnWorldBeginPlay(UWorld& InWorld) override;
	virtual void Deinitialize() override;

	void OnPostPhysicsTick(FChaosScene* Scene);

	TArray<TWeakObjectPtr<UChaosMoverBackendComponent>> Backends;

	FDelegateHandle InjectInputsExternalCallbackHandle;
	FDelegateHandle PhysScenePostTickCallbackHandle;
	class UE::ChaosMover::FAsyncCallback* AsyncCallback = nullptr;
};