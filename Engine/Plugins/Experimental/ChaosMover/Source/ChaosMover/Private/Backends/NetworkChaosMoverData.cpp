// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkChaosMoverData.h"

#include "ChaosMover/Backends/ChaosMoverBackend.h"
#include "ChaosMover/ChaosMoverSimulation.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NetworkChaosMoverData)

void FNetworkChaosMoverInputData::ApplyData(UActorComponent* NetworkComponent) const
{
	if (UChaosMoverBackendComponent* BackendComp = Cast<UChaosMoverBackendComponent>(NetworkComponent))
	{
		BackendComp->GetSimulation()->ApplyNetInputData(InputCmdContext);
	}
}

void FNetworkChaosMoverInputData::BuildData(const UActorComponent* NetworkComponent)
{
	if (const UChaosMoverBackendComponent* BackendComp = Cast<const UChaosMoverBackendComponent>(NetworkComponent))
	{
		BackendComp->GetSimulation()->BuildNetInputData(InputCmdContext);
	}
}

void FNetworkChaosMoverInputData::DecayData(float DecayAmount)
{
	InputCmdContext.InputCollection.Decay(DecayAmount);
}

bool FNetworkChaosMoverInputData::NetSerialize(FArchive& Ar, UPackageMap* PackageMap, bool& bOutSuccess)
{
	SerializeFrames(Ar);
	if (PackageMap)
	{
		InputCmdContext.NetSerialize(FNetSerializeParams(Ar, PackageMap));
		bOutSuccess = !Ar.IsError();
	}
	else
	{
		bOutSuccess = false;
	}

	return bOutSuccess;
}

void FNetworkChaosMoverInputData::InterpolateData(const FNetworkPhysicsData& MinData, const FNetworkPhysicsData& MaxData)
{
	const FNetworkChaosMoverInputData& MinDataInput = static_cast<const FNetworkChaosMoverInputData&>(MinData);
	const FNetworkChaosMoverInputData& MaxDataInput = static_cast<const FNetworkChaosMoverInputData&>(MaxData);

	const float LerpFactor = (LocalFrame - MinDataInput.LocalFrame) / (MaxDataInput.LocalFrame - MinDataInput.LocalFrame);
	InputCmdContext.InputCollection.Interpolate(MinDataInput.InputCmdContext.InputCollection, MaxDataInput.InputCmdContext.InputCollection, LerpFactor);
}

void FNetworkChaosMoverInputData::MergeData(const FNetworkPhysicsData& FromData)
{
	const FNetworkChaosMoverInputData& TypedFrom = static_cast<const FNetworkChaosMoverInputData&>(FromData);
	InputCmdContext.InputCollection.Merge(TypedFrom.InputCmdContext.InputCollection);
}

void FNetworkChaosMoverInputData::ValidateData(const UActorComponent* NetworkComponent)
{
	// TODO
}

bool FNetworkChaosMoverInputData::CompareData(const FNetworkPhysicsData& PredictedData)
{
	const FMoverInputCmdContext& PredictedInputCmd = static_cast<const FNetworkChaosMoverInputData&>(PredictedData).InputCmdContext;
	return !PredictedInputCmd.InputCollection.ShouldReconcile(InputCmdContext.InputCollection);
}

const FString FNetworkChaosMoverInputData::DebugData()
{
	TAnsiStringBuilder<256> StringBuilder;
	InputCmdContext.ToString(StringBuilder);
	return FString::Printf(TEXT("FNetworkChaosMoverInputData:\n%hs"), *StringBuilder);
}

//////////////////////////////////////////////////////////////////////////
// FNetworkChaosMoverStateData

void FNetworkChaosMoverStateData::ApplyData(UActorComponent* NetworkComponent) const
{
	if (NetworkComponent)
	{
		if (UChaosMoverBackendComponent* BackendComp = Cast<UChaosMoverBackendComponent>(NetworkComponent))
		{
			BackendComp->GetSimulation()->ApplyNetStateData(SyncState);
		}
	}
}

void FNetworkChaosMoverStateData::BuildData(const UActorComponent* NetworkComponent)
{
	if (NetworkComponent)
	{
		if (const UChaosMoverBackendComponent* BackendComp = Cast<const UChaosMoverBackendComponent>(NetworkComponent))
		{
			BackendComp->GetSimulation()->BuildNetStateData(SyncState);
		}
	}
}

bool FNetworkChaosMoverStateData::NetSerialize(FArchive& Ar, UPackageMap* PackageMap, bool& bOutSuccess)
{
	SerializeFrames(Ar);
	if (PackageMap)
	{
		FNetSerializeParams Params(Ar, PackageMap);
		SyncState.NetSerialize(Params);
		bOutSuccess = !Ar.IsError();
	}
	else
	{
		bOutSuccess = false;
	}

	return bOutSuccess;
}

void FNetworkChaosMoverStateData::InterpolateData(const FNetworkPhysicsData& MinData, const FNetworkPhysicsData& MaxData)
{
	const FNetworkChaosMoverStateData& MinState = static_cast<const FNetworkChaosMoverStateData&>(MinData);
	const FNetworkChaosMoverStateData& MaxState = static_cast<const FNetworkChaosMoverStateData&>(MaxData);

	const float LerpFactor = (LocalFrame - MinState.LocalFrame) / (MaxState.LocalFrame - MinState.LocalFrame);
	SyncState.Interpolate(&MinState.SyncState, &MaxState.SyncState, LerpFactor);
}

bool FNetworkChaosMoverStateData::CompareData(const FNetworkPhysicsData& PredictedData)
{
	const FMoverSyncState& PredictedSyncState = static_cast<const FNetworkChaosMoverStateData&>(PredictedData).SyncState;
	return !PredictedSyncState.ShouldReconcile(SyncState);
}

const FString FNetworkChaosMoverStateData::DebugData()
{
	TAnsiStringBuilder<256> StringBuilder;
	SyncState.ToString(StringBuilder);
	return FString::Printf(TEXT("FNetworkChaosMoverStateData:\n%hs"), *StringBuilder);
}
