// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MoverSimulationTypes.h"
#include "Physics/NetworkPhysicsComponent.h"

#include "NetworkChaosMoverData.generated.h"

USTRUCT()
struct FNetworkChaosMoverInputData : public FNetworkPhysicsData
{
	GENERATED_BODY()

	UPROPERTY()
	FMoverInputCmdContext InputCmdContext;

	/**  Apply the data onto the network physics component */
	virtual void ApplyData(UActorComponent* NetworkComponent) const override;

	/**  Build the data from the network physics component */
	virtual void BuildData(const UActorComponent* NetworkComponent) override;

	/** Decay input during resimulation and forward prediction */
	virtual void DecayData(float DecayAmount) override;

	/**  Serialize data function that will be used to transfer the struct across the network */
	bool NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess);

	/** Interpolate the data in between two inputs data */
	virtual void InterpolateData(const FNetworkPhysicsData& MinData, const FNetworkPhysicsData& MaxData) override;

	/** Merge data into this input */
	virtual void MergeData(const FNetworkPhysicsData& FromData) override;

	/** Check input data is valid - Input is send from client to server, no need to make sure it's reasonable */
	virtual void ValidateData(const UActorComponent* NetworkComponent) override;

	virtual bool CompareData(const FNetworkPhysicsData& PredictedData) override;

	/** Return string with debug information */
	virtual const FString DebugData() override;
};

template<>
struct TStructOpsTypeTraits<FNetworkChaosMoverInputData> : public TStructOpsTypeTraitsBase2<FNetworkChaosMoverInputData>
{
	enum
	{
		WithNetSerializer = true,
	};
};

USTRUCT()
struct FNetworkChaosMoverStateData : public FNetworkPhysicsData
{
	GENERATED_BODY()

	UPROPERTY()
	FMoverSyncState SyncState;

	/**  Apply the data onto the network physics component */
	virtual void ApplyData(UActorComponent* NetworkComponent) const override;

	/**  Build the data from the network physics component */
	virtual void BuildData(const UActorComponent* NetworkComponent) override;

	/**  Serialize data function that will be used to transfer the struct across the network */
	bool NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess);

	/** Interpolate the data in between two inputs data */
	virtual void InterpolateData(const FNetworkPhysicsData& MinData, const FNetworkPhysicsData& MaxData) override;

	virtual bool CompareData(const FNetworkPhysicsData& PredictedData) override;

	/** Return string with debug information */
	virtual const FString DebugData() override;
};

template<>
struct TStructOpsTypeTraits<FNetworkChaosMoverStateData> : public TStructOpsTypeTraitsBase2<FNetworkChaosMoverStateData>
{
	enum
	{
		WithNetSerializer = true,
	};
};

namespace UE::ChaosMover
{
	struct FNetworkDataTraits
	{
		using InputsType = FNetworkChaosMoverInputData;
		using StatesType = FNetworkChaosMoverStateData;
	};
}