// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosMover/ChaosCompositeMovementMode.h"
#include "ChaosMover/ChaosMoverLog.h"
#include "ChaosMover/ChaosMoverSimulation.h"
#include "MoverSimulationTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosCompositeMovementMode)


UChaosCompositeMovementMode::UChaosCompositeMovementMode(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSupportsAsync = true;
}

void UChaosCompositeMovementMode::OnRegistered(const FName ModeName, const FMoverSimContext& SimContext)
{
	Super::OnRegistered(ModeName, SimContext);

	if (MoveExecutor && Simulation)
	{
		MoveExecutor->SetSimulation(Simulation);
	}

	if (MoveExecutor)
	{
		MoveExecutor->OnModeRegistered(ModeName);
	}
}

void UChaosCompositeMovementMode::OnUnregistered(const FMoverSimContext& SimContext)
{
	if (MoveExecutor)
	{
		MoveExecutor->OnModeUnregistered();
		MoveExecutor->SetSimulation(nullptr);
	}

	Super::OnUnregistered(SimContext);
}

void UChaosCompositeMovementMode::Activate(const FMoverEventContext& Context, FName PrevModeName, const FMoverSimContext& SimContext, const FMoverTickStartData& StartState, FMoverSyncState* OutSyncState, FMoverAuxStateContext* OutAuxState)
{
	Super::Activate(Context, PrevModeName, SimContext, StartState, OutSyncState, OutAuxState);

	bMoveSourceEnded = false;

	if (MoveSource && Simulation)
	{
		FMoverTime SimTime;
		SimTime.TimeMs     = Context.EventTimeMs;
		SimTime.FrameCount = Context.ServerFrame;
		MoveSource->OnStart_Async(Simulation->GetBlackboard_Mutable(), SimTime);
	}
}

void UChaosCompositeMovementMode::Deactivate(const FMoverEventContext& Context, FName InNextModeName, const FMoverSimContext& SimContext)
{
	if (MoveSource && Simulation && !bMoveSourceEnded)
	{
		MoveSource->OnEnd_Async(Simulation->GetBlackboard_Mutable(), Context.EventTimeMs);
	}

	Super::Deactivate(Context, InNextModeName, SimContext);
}

void UChaosCompositeMovementMode::GenerateMove_Implementation(const FMoverSimContext& SimContext, const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, FProposedMove& OutProposedMove) const
{
	if (!MoveSource)
	{
		return;
	}

	UMoverBlackboard* SimBlackboard = Simulation ? Simulation->GetBlackboard_Mutable() : nullptr;
	MoveSource->GenerateMove_Async(StartState, TimeStep, SimBlackboard, OutProposedMove);
}

void UChaosCompositeMovementMode::SimulationTick_Implementation(const FSimulationTickParams& Params, FMoverTickEndData& OutputState)
{
	if (MoveExecutor)
	{
		MoveExecutor->ExecuteMove_Async(Params, OutputState);
	}
	else
	{
		UE_LOGF(LogChaosMover, Warning, "UChaosCompositeMovementMode (%ls): No MoveExecutor set. Mode will do nothing.", *GetName());
	}

	if (MoveSource)
	{
		const double CurrentSimTimeMs = Params.TimeStep.BaseSimTimeMs;
		if (!bMoveSourceEnded && MoveSource->IsFinished(CurrentSimTimeMs))
		{
			bMoveSourceEnded = true;
			MoveSource->OnEnd_Async(Params.SimBlackboard, CurrentSimTimeMs);

			if (!NextModeName.IsNone())
			{
				OutputState.MovementEndState.NextModeName = NextModeName;
			}
		}
	}
}

void UChaosCompositeMovementMode::CollectSimulationInterfaces(FChaosMoverSimulationInterfaceCache& OutCache)
{
	Super::CollectSimulationInterfaces(OutCache);

	if (IChaosPreSimulationTickInterface* PreSim = Cast<IChaosPreSimulationTickInterface>(MoveExecutor))
	{
		OutCache.PreSimInterfaces.AddUnique(PreSim);
	}
	if (IChaosPostSimulationTickInterface* PostSim = Cast<IChaosPostSimulationTickInterface>(MoveExecutor))
	{
		OutCache.PostSimInterfaces.AddUnique(PostSim);
	}
}
