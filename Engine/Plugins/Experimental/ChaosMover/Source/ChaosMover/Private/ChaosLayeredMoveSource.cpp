// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosMover/ChaosLayeredMoveSource.h"
#include "DefaultMovementSet/LayeredMoves/MontageStateProvider.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosLayeredMoveSource)


void UChaosLayeredMoveSource::OnStart_Async(UMoverBlackboard* SimBlackboard, const FMoverTime& SimTime)
{
	if (FLayeredMoveBase* Move = LayeredMove.GetMutablePtr<FLayeredMoveBase>())
	{
		Move->StartMove_Async(SimBlackboard, SimTime);
	}
}

void UChaosLayeredMoveSource::GenerateMove_Async(const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, UMoverBlackboard* SimBlackboard, FProposedMove& OutProposedMove)
{
	if (FLayeredMoveBase* Move = LayeredMove.GetMutablePtr<FLayeredMoveBase>())
	{
		Move->GenerateMove_Async(StartState, TimeStep, SimBlackboard, OutProposedMove);
	}
}

bool UChaosLayeredMoveSource::IsFinished(double CurrentSimTimeMs) const
{
	if (const FLayeredMoveBase* Move = LayeredMove.GetPtr<FLayeredMoveBase>())
	{
		return Move->IsFinished(CurrentSimTimeMs);
	}

	return true;
}

void UChaosLayeredMoveSource::OnEnd_Async(UMoverBlackboard* SimBlackboard, double CurrentSimTimeMs)
{
	if (FLayeredMoveBase* Move = LayeredMove.GetMutablePtr<FLayeredMoveBase>())
	{
		Move->EndMove_Async(SimBlackboard, CurrentSimTimeMs);
	}
}

void UChaosLayeredMoveSource::AppendMontageOutputEntry(TArray<FMoverSimDrivenMontageEntry>& OutEntries, const FMoverTimeStep& TimeStep) const
{
	if (const FLayeredMove_MontageStateProvider* MontageMove = LayeredMove.GetPtr<FLayeredMove_MontageStateProvider>())
	{
		MontageMove->AppendMontageOutputEntry(OutEntries, TimeStep);
	}
}
