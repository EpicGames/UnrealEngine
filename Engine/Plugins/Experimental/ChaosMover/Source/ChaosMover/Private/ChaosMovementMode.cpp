// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosMover/ChaosMovementMode.h"

#include "ChaosMover/ChaosMovementModeTransition.h"
#include "ChaosMover/ChaosMoverLog.h"
#if WITH_EDITOR
#include "ChaosMover/Backends/ChaosMoverBackend.h"
#include "Internationalization/Text.h"
#include "Misc/DataValidation.h"
#include "MoverComponent.h"
#endif // WITH_EDITOR

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosMovementMode)

#define LOCTEXT_NAMESPACE "ChaosMovementMode"

UChaosMovementMode::UChaosMovementMode(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, Simulation(nullptr)
{
}

void UChaosMovementMode::SetSimulation(UChaosMoverSimulation* InSimulation)
{
	Simulation = InSimulation;
	for (TObjectPtr<UBaseMovementModeTransition>& Transition : Transitions)
	{
		if (UChaosMovementModeTransition* ChaosTransition = Cast<UChaosMovementModeTransition>(Transition.Get()))
		{
			ChaosTransition->SetSimulation(InSimulation);
		}
	}
}

void UChaosMovementMode::CollectSimulationInterfaces(FChaosMoverSimulationInterfaceCache& OutCache)
{
	if (IChaosPreSimulationTickInterface* PreSim = Cast<IChaosPreSimulationTickInterface>(this))
	{
		OutCache.PreSimInterfaces.AddUnique(PreSim);
	}
	if (IChaosPostSimulationTickInterface* PostSim = Cast<IChaosPostSimulationTickInterface>(this))
	{
		OutCache.PostSimInterfaces.AddUnique(PostSim);
	}
}

#if WITH_EDITOR
EDataValidationResult UChaosMovementMode::IsDataValid(class FDataValidationContext& Context) const
{
	EDataValidationResult Result = Super::IsDataValid(Context);

	if (const UMoverComponent* MoverComponent = GetMoverComponent())
	{
		const UClass* BackendClass = MoverComponent->BackendClass;
		if (BackendClass && !BackendClass->IsChildOf<UChaosMoverBackendComponent>())
		{
			Context.AddError(NSLOCTEXT("ChaosMover", "ChaosMovementModeHasValidBackend", "Chaos movement modes need to have a backend class that supports Chaos Physics (UChaosMoverBackendComponent)."));
			Result = EDataValidationResult::Invalid;
		}
	}

	return Result;
}
#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE