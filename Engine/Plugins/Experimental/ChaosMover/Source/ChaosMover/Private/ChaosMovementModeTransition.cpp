// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosMover/ChaosMovementModeTransition.h"

#include "ChaosMover/ChaosMoverLog.h"
#if WITH_EDITOR
#include "ChaosMover/Backends/ChaosMoverBackend.h"
#include "Internationalization/Text.h"
#include "Misc/DataValidation.h"
#include "MoverComponent.h"
#endif // WITH_EDITOR

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosMovementModeTransition)

#define LOCTEXT_NAMESPACE "ChaosMovementModeTransition"

UChaosMovementModeTransition::UChaosMovementModeTransition(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, Simulation(nullptr)
{
}

#if WITH_EDITOR
EDataValidationResult UChaosMovementModeTransition::IsDataValid(class FDataValidationContext& Context) const
{
	EDataValidationResult Result = Super::IsDataValid(Context);
	
	if (const UMoverComponent* MoverComponent = GetMoverComponent())
	{
		const UClass* BackendClass = MoverComponent->BackendClass;
		if (BackendClass && !BackendClass->IsChildOf<UChaosMoverBackendComponent>())
		{
			Context.AddError(NSLOCTEXT("ChaosMover", "ChaosMovementModeTransitionHasValidBackend", "Chaos movement mode transitions need to have a backend class that supports Chaos Physics (UChaosMoverBackendComponent)."));
			Result = EDataValidationResult::Invalid;
		}
	}

	return Result;
}
#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE