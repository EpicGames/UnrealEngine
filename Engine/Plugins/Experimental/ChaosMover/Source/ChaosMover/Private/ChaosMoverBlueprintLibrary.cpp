// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosMover/ChaosMoverBlueprintLibrary.h"

#include "ChaosMover/ChaosMoverActionTypes.h"
#include "ChaosMover/ChaosMoverConsoleVariables.h"
#include "GameFramework/Actor.h"
#include "InstantMovementEffect.h"
#include "LayeredMove.h"
#include "MoverComponent.h"
#include "Physics/NetworkPhysicsComponent.h"
#include "Physics/NetworkPhysicsSettingsComponent.h"
#include "PhysicsEngine/PhysicsSettings.h"

namespace
{
	UNetworkPhysicsComponent* GetNetworkPhysicsComponent(UMoverComponent* MoverComponent)
	{
		AActor* Owner = MoverComponent ? MoverComponent->GetOwner() : nullptr;
		return Owner ? Owner->FindComponentByClass<UNetworkPhysicsComponent>() : nullptr;
	}

	float GetSchedulingDelaySeconds(UMoverComponent* MoverComponent)
	{
		AActor* Owner = MoverComponent ? MoverComponent->GetOwner() : nullptr;
		if (!Owner) { return 0.3f; }

		if (UNetworkPhysicsSettingsComponent* SettingsComp = Owner->FindComponentByClass<UNetworkPhysicsSettingsComponent>())
		{
			return SettingsComp->GetSettings().GeneralSettings.EventSchedulingMinDelaySeconds;
		}
		return UPhysicsSettings::Get()->PhysicsPrediction.MaxSupportedLatencyPrediction;
	}

	void EnqueueLayeredMoveAction_Authority(UMoverComponent* MoverComponent, void* MovePtr,
		FStructProperty* StructProp, FProperty* MostRecentProp, float DelaySeconds, const TCHAR* CallerName)
	{
		const bool bHasValidStructProp = StructProp && StructProp->Struct && StructProp->Struct->IsChildOf(FLayeredMoveBase::StaticStruct());
		if (!ensureMsgf(bHasValidStructProp && MovePtr,
			TEXT("An invalid type (%s) was sent to %s. A struct derived from FLayeredMoveBase is required."),
			StructProp ? *GetNameSafe(StructProp->Struct) : *MostRecentProp->GetClass()->GetName(), CallerName))
		{
			return;
		}

		AActor* Owner = MoverComponent ? MoverComponent->GetOwner() : nullptr;
		if (!MoverComponent || !Owner || !Owner->HasAuthority()) { return; }

		UNetworkPhysicsComponent* NetPhysics = GetNetworkPhysicsComponent(MoverComponent);
		if (!ensure(NetPhysics)) { return; }

		using EActionAuthorStyle = FNetworkPhysicsActionPayload::EActionAuthorStyle;
		FChaosMoverLayeredMoveAction Action;
		Action.AuthorStyle = EActionAuthorStyle::Authority;
		TUniquePtr<FLayeredMoveBase> Cloned(reinterpret_cast<FLayeredMoveBase*>(MovePtr)->Clone());
		Action.Move.InitializeAsScriptStruct(StructProp->Struct, reinterpret_cast<const uint8*>(Cloned.Get()));

		NetPhysics->EnqueueScheduledAction_External(Action, /* SourceId = */ 0u, DelaySeconds, /* bReliable = */ true);
	}

	void EnqueueLayeredMoveInstanceAction_Authority(UMoverComponent* MoverComponent,
		TSubclassOf<ULayeredMoveLogic> MoveLogicClass, FLayeredMoveInstancedData& InstancedData, float DelaySeconds)
	{
		AActor* Owner = MoverComponent ? MoverComponent->GetOwner() : nullptr;
		if (!MoverComponent || !Owner || !Owner->HasAuthority()) { return; }

		UNetworkPhysicsComponent* NetPhysics = GetNetworkPhysicsComponent(MoverComponent);
		if (!ensure(NetPhysics)) { return; }

		using EActionAuthorStyle = FNetworkPhysicsActionPayload::EActionAuthorStyle;
		FChaosMoverLayeredMoveInstanceAction Action;
		Action.AuthorStyle = EActionAuthorStyle::Authority;
		Action.MoveLogicClass = MoveLogicClass;
		TUniquePtr<FLayeredMoveInstancedData> Cloned(InstancedData.Clone());
		Action.InstancedData.InitializeAsScriptStruct(Cloned->GetScriptStruct(), reinterpret_cast<const uint8*>(Cloned.Get()));

		NetPhysics->EnqueueScheduledAction_External(Action, /* SourceId = */ 0u, DelaySeconds, /* bReliable = */ true);
	}

	void EnqueueInstantMovementEffectAction_Authority(UMoverComponent* MoverComponent, void* EffectPtr,
		FStructProperty* StructProp, FProperty* MostRecentProp, float DelaySeconds, const TCHAR* CallerName)
	{
		const bool bHasValidStructProp = StructProp && StructProp->Struct && StructProp->Struct->IsChildOf(FInstantMovementEffect::StaticStruct());
		if (!ensureMsgf(bHasValidStructProp && EffectPtr,
			TEXT("An invalid type (%s) was sent to %s. A struct derived from FInstantMovementEffect is required."),
			StructProp ? *GetNameSafe(StructProp->Struct) : *MostRecentProp->GetClass()->GetName(), CallerName))
		{
			return;
		}

		AActor* Owner = MoverComponent ? MoverComponent->GetOwner() : nullptr;
		if (!MoverComponent || !Owner || !Owner->HasAuthority()) { return; }

		UNetworkPhysicsComponent* NetPhysics = GetNetworkPhysicsComponent(MoverComponent);
		if (!ensure(NetPhysics)) { return; }

		using EActionAuthorStyle = FNetworkPhysicsActionPayload::EActionAuthorStyle;
		FChaosMoverInstantMovementEffectAction Action;
		Action.AuthorStyle = EActionAuthorStyle::Authority;
		TUniquePtr<FInstantMovementEffect> Cloned(reinterpret_cast<FInstantMovementEffect*>(EffectPtr)->Clone());
		Action.Effect.InitializeAsScriptStruct(StructProp->Struct, reinterpret_cast<const uint8*>(Cloned.Get()));

		NetPhysics->EnqueueScheduledAction_External(Action, /* SourceId = */ 0u, DelaySeconds, /* bReliable = */ true);
	}
}

bool UChaosMoverBlueprintLibrary::IsNetworkingMovesWithSimActions()
{
	return UE::ChaosMover::CVars::bNetworkMovesWithSimActions;
}

bool UChaosMoverBlueprintLibrary::IsNetworkingInstantMovementEffectsWithSimActions()
{
	return UE::ChaosMover::CVars::bNetworkInstantMovementEffectsWithSimActions;
}

DEFINE_FUNCTION(UChaosMoverBlueprintLibrary::execK2_QueueLayeredMove_Authority)
{
	P_GET_OBJECT(UMoverComponent, MoverComponent);
	Stack.StepCompiledIn<FStructProperty>(nullptr);
	void* MovePtr = Stack.MostRecentPropertyAddress;
	FStructProperty* StructProp = CastField<FStructProperty>(Stack.MostRecentProperty);
	FProperty* MostRecentProp = Stack.MostRecentProperty;
	P_FINISH;
	P_NATIVE_BEGIN;
	EnqueueLayeredMoveAction_Authority(MoverComponent, MovePtr, StructProp, MostRecentProp, 0.0f, TEXT("Queue Layered Move (Authority)"));
	P_NATIVE_END;
}

void UChaosMoverBlueprintLibrary::QueueLayeredMoveInstance_Authority(UMoverComponent* MoverComponent,
	TSubclassOf<ULayeredMoveLogic> MoveLogicClass, FLayeredMoveInstancedData& InstancedData)
{
	EnqueueLayeredMoveInstanceAction_Authority(MoverComponent, MoveLogicClass, InstancedData, 0.0f);
}

DEFINE_FUNCTION(UChaosMoverBlueprintLibrary::execK2_QueueInstantMovementEffect_Authority)
{
	P_GET_OBJECT(UMoverComponent, MoverComponent);
	Stack.StepCompiledIn<FStructProperty>(nullptr);
	void* EffectPtr = Stack.MostRecentPropertyAddress;
	FStructProperty* StructProp = CastField<FStructProperty>(Stack.MostRecentProperty);
	FProperty* MostRecentProp = Stack.MostRecentProperty;
	P_FINISH;
	P_NATIVE_BEGIN;
	EnqueueInstantMovementEffectAction_Authority(MoverComponent, EffectPtr, StructProp, MostRecentProp, 0.0f, TEXT("Queue Instant Movement Effect (Authority)"));
	P_NATIVE_END;
}

DEFINE_FUNCTION(UChaosMoverBlueprintLibrary::execK2_ScheduleLayeredMove_Authority)
{
	P_GET_OBJECT(UMoverComponent, MoverComponent);
	Stack.StepCompiledIn<FStructProperty>(nullptr);
	void* MovePtr = Stack.MostRecentPropertyAddress;
	FStructProperty* StructProp = CastField<FStructProperty>(Stack.MostRecentProperty);
	FProperty* MostRecentProp = Stack.MostRecentProperty;
	P_FINISH;
	P_NATIVE_BEGIN;
	EnqueueLayeredMoveAction_Authority(MoverComponent, MovePtr, StructProp, MostRecentProp, GetSchedulingDelaySeconds(MoverComponent), TEXT("Schedule Layered Move (Authority)"));
	P_NATIVE_END;
}

void UChaosMoverBlueprintLibrary::ScheduleLayeredMoveInstance_Authority(UMoverComponent* MoverComponent,
	TSubclassOf<ULayeredMoveLogic> MoveLogicClass, FLayeredMoveInstancedData& InstancedData)
{
	EnqueueLayeredMoveInstanceAction_Authority(MoverComponent, MoveLogicClass, InstancedData, GetSchedulingDelaySeconds(MoverComponent));
}

DEFINE_FUNCTION(UChaosMoverBlueprintLibrary::execK2_ScheduleInstantMovementEffect_Authority)
{
	P_GET_OBJECT(UMoverComponent, MoverComponent);
	Stack.StepCompiledIn<FStructProperty>(nullptr);
	void* EffectPtr = Stack.MostRecentPropertyAddress;
	FStructProperty* StructProp = CastField<FStructProperty>(Stack.MostRecentProperty);
	FProperty* MostRecentProp = Stack.MostRecentProperty;
	P_FINISH;
	P_NATIVE_BEGIN;
	EnqueueInstantMovementEffectAction_Authority(MoverComponent, EffectPtr, StructProp, MostRecentProp, GetSchedulingDelaySeconds(MoverComponent), TEXT("Schedule Instant Movement Effect (Authority)"));
	P_NATIVE_END;
}
