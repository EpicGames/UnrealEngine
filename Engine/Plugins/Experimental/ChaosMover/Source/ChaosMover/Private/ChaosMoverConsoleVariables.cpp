// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosMover/ChaosMoverConsoleVariables.h"

#include "HAL/IConsoleManager.h"

#include "ChaosMover/ChaosMoverSimulationTypes.h"
#include "ChaosMover/ChaosMoverSimulation.h"
#include "MoverComponent.h"
#include "GameFramework/PlayerController.h"
#include "GameFramework/Pawn.h"
#include "Engine/World.h"

namespace UE::ChaosMover
{
	namespace CVars
	{
		bool bForceSingleThreadedGT = true;
		FAutoConsoleVariableRef CVarChaosMoverForceSingleThreadedGT(TEXT("ChaosMover.ForceSingleThreadedGT"),
			bForceSingleThreadedGT, TEXT("Force updates on the game thread to be single threaded."));

		bool bForceSingleThreadedPT = true;
		FAutoConsoleVariableRef CVarChaosMoverForceSingleThreadedPT(TEXT("ChaosMover.ForceSingleThreadedPT"),
			bForceSingleThreadedPT, TEXT("Force updates on the worker thread to be single threaded."));

		bool bDrawGroundQueries = false;
		FAutoConsoleVariableRef CVarChaosMoverDrawGroundQueries(TEXT("ChaosMover.DebugDraw.GroundQueries"),
			bDrawGroundQueries, TEXT("Draw ground queries."));

		bool bDrawOverlapQueries = false;
		FAutoConsoleVariableRef CVarChaosMoverDrawOverlapQueries(TEXT("ChaosMover.DebugDraw.OverlapQueries"),
			bDrawOverlapQueries, TEXT("Draw overlap queries."));

		bool bSkipGenerateMoveIfOverridden = true;
		FAutoConsoleVariableRef CVarChaosMoverSkipGenerateMoveIfOverridden(TEXT("ChaosMover.Perf.SkipGenerateMoveIfOverridden"),
			bSkipGenerateMoveIfOverridden, TEXT("If true and we have a layered move fully overriding movement, then we will skip calling OnGenerateMove on the active movement mode for better performance\n"));
		
		bool bDisableResimDuplicateEventChecking = true;
		FAutoConsoleVariableRef CVarChaosMoverDisableResimDuplicateEventChecking(TEXT("ChaosMover.Networking.DisableResimDuplicateEventChecking"),
			bDisableResimDuplicateEventChecking, TEXT("If true we disable the check for duplicate events in resim and can resend the same events\n"));

		int32 InstantMovementEffectIDHistorySize = 30;
		FAutoConsoleVariableRef CVarChaosMoverInstantMovementEffectIDHistorySize(TEXT("ChaosMover.Networking.InstantMovementEffectIDHistorySize"),
			InstantMovementEffectIDHistorySize, TEXT("Number of past frames for which we keep the last seen instant movement effect IDs, to avoid duplicate processing when input is repeated\n"));

		int32 LayeredMoveIDHistorySize = 30;
		FAutoConsoleVariableRef CVarChaosMoverLayeredMoveIDHistorySize(TEXT("ChaosMover.Networking.LayeredMoveIDHistorySize"),
			LayeredMoveIDHistorySize, TEXT("Number of past frames for which we keep the last seen layered move IDs, to avoid duplicate processing when input is repeated\n"));

		int32 FrameDifferenceLeniencyForEventTimeComparison = 1;
		FAutoConsoleVariableRef CVarChaosMoverFrameDifferenceLeniencyForEventTimeComparison(TEXT("ChaosMover.Networking.FrameDifferenceLeniencyForEventTimeComparison"),
			FrameDifferenceLeniencyForEventTimeComparison, TEXT("Within this number of frames events are considered to be at the same time and will not trigger multiple times.\n"));

		float FallingCheckRelativeSpeedLimit = 60.0f;
		FAutoConsoleVariableRef CVarChaosMoverFallingCheckRelativeSpeedLimit(TEXT("ChaosMover.FallingCheckRelativeSpeedLimit"),
			FallingCheckRelativeSpeedLimit, TEXT("Above this limit the falling check will consider the character to be lifting off the surface.\n"));

		bool bEnablePreSimGroundCheck = false;
		FAutoConsoleVariableRef CVarChaosMoverEnablePreSimGroundCheck(TEXT("ChaosMover.EnablePreSimGroundCheck"),
			bEnablePreSimGroundCheck, TEXT(""));

		bool bEnableServerLaunchOverride = false;
		FAutoConsoleVariableRef CVarChaosMoverEnableServerLaunchOverride(TEXT("ChaosMover.EnableServerLaunchOverride"),
			bEnableServerLaunchOverride, TEXT(""));

		bool bNetworkInstantMovementEffectsWithSimActions = true;
		FAutoConsoleVariableRef CVarChaosMoverNetworkInstantMovementEffectsWithSimActions(
			TEXT("ChaosMover.Networking.NetworkInstantMovementEffectsWithSimActions"),
			bNetworkInstantMovementEffectsWithSimActions,
			TEXT("When true, instant movement effects queued via QueueInstantMovementEffect are networked via the sim action system (Proposed style). ")
			TEXT("When false, they are embedded in the input cmd (PreSim detection). See ChaosMoverConsoleVariables.h for details."));

		bool bNetworkMovesWithSimActions = true;
		FAutoConsoleVariableRef CVarChaosMoverNetworkMovesWithSimActions(
			TEXT("ChaosMover.Networking.NetworkMovesWithSimActions"),
			bNetworkMovesWithSimActions,
			TEXT("When true, layered moves queued via QueueLayeredMove/QueueLayeredMoveInstance are networked via the sim action system (Proposed style). ")
			TEXT("When false, they are embedded in the input cmd (PreSim detection). See ChaosMoverConsoleVariables.h for details."));

		static FAutoConsoleCommandWithWorld TeleportOnlyClientCommand
		(
			TEXT("ChaosMover.Debug.TeleportClientOnly"),
			TEXT("Sends an instant movement effect to teleport the locally controlled character if it has a UMoverComponent"),
			FConsoleCommandWithWorldDelegate::CreateStatic([](UWorld* World)
				{
					APlayerController* LocalPlayerController = World ? World->GetFirstPlayerController() : nullptr;
					APawn* LocalPawn = LocalPlayerController ? LocalPlayerController->GetPawn().Get() : nullptr;
					UMoverComponent* MoverComponent = LocalPawn ? LocalPawn->FindComponentByClass<UMoverComponent>() : nullptr;
					if (!MoverComponent)
					{
						return;
					}

					auto TeleportLambda = [](FApplyMovementEffectParams_Async& ApplyEffectParams, FMoverSyncState& OutputState)->bool
					{
						if (UChaosMoverSimulation* ChaosMoverSimulation = Cast<UChaosMoverSimulation>(ApplyEffectParams.Simulation))
						{
							if (const FMoverDefaultSyncState* DefaultSyncState = OutputState.SyncStateCollection.FindDataByType<FMoverDefaultSyncState>())
							{
								static FVector TeleportDeltaLocation(0, 0, 500);
								FVector TargetLocation = DefaultSyncState->GetLocation_WorldSpace() + TeleportDeltaLocation;
								ChaosMoverSimulation->AttemptTeleport(*ApplyEffectParams.TimeStep, FTransform(DefaultSyncState->GetOrientation_WorldSpace(), TargetLocation), /* bUseActorRotation = */ true, OutputState);
								return true;
							}
						}
						return false;
					};

					TSharedPtr<FAsyncLocalOnlyInstantMovementEffect> EffectPtr = MakeShared<FAsyncLocalOnlyInstantMovementEffect>();
					static FName EffectName("TeleportLocalOnlyInstantMovementEffect");
					EffectPtr->OptionalName = EffectName;
					EffectPtr->AsyncFunction = TeleportLambda;
					MoverComponent->QueueInstantMovementEffect(EffectPtr);
				}));
	}

	static FAutoConsoleCommandWithWorldAndArgs TeleportToCommand
	(
		TEXT("ChaosMover.Debug.TeleportTo"),
		TEXT("Usage = ChaosMover.Debug.TeleportTo <X> <Y> <Z> <Yaw> <Pitch> <Roll>. Teleports the ChaosMover character to (X, Y, Z) (Yaw, Pitch, Roll). Use '=' to keep the component unchanged, for instance 'ChaosMover.Debug.TeleportTo 0 0 = 0' will teleport to X=0,Y=0 leaving Z unchanged and set Yaw to 0, leaving pitch and roll unchanged"),
		FConsoleCommandWithWorldAndArgsDelegate::CreateStatic([](const TArray<FString>& Args, UWorld* World)
			{
				if (APlayerController* LocalPlayerController = World ? World->GetFirstPlayerController() : nullptr)
				{
					TObjectPtr<APawn> LocalPawn = LocalPlayerController->GetPawn();
					UMoverComponent* MoverComponent = LocalPawn ? LocalPawn->FindComponentByClass<UMoverComponent>() : nullptr;
					if (MoverComponent)
					{
						TSharedPtr<FDebugTeleportToInstantMovementEffect> EffectPtr = MakeShared<FDebugTeleportToInstantMovementEffect>();
						// Leaving a component unchanged with '=' means keep that character's location or rotation component unchanged
						if (Args.Num() > 0 && Args[0].Len() > 0 && Args[0][0] != '=')
						{
							LexFromString(EffectPtr->TeleportLocation.X, *Args[0]);
						}
						if (Args.Num() > 1 && Args[1].Len() > 0 && Args[1][0] != '=')
						{
							LexFromString(EffectPtr->TeleportLocation.Y, *Args[1]);
						}
						if (Args.Num() > 2 && Args[2].Len() > 0 && Args[2][0] != '=')
						{
							LexFromString(EffectPtr->TeleportLocation.Z, *Args[2]);
						}
						if (Args.Num() > 3 && Args[3].Len() > 0 && Args[3][0] != '=')
						{
							LexFromString(EffectPtr->TeleportRotation.Yaw, *Args[3]);
						}
						if (Args.Num() > 4 && Args[4].Len() > 0 && Args[4][0] != '=')
						{
							LexFromString(EffectPtr->TeleportRotation.Pitch, *Args[4]);
						}
						if (Args.Num() > 5 && Args[5].Len() > 0 && Args[5][0] != '=')
						{
							LexFromString(EffectPtr->TeleportRotation.Roll, *Args[5]);
						}
						MoverComponent->ScheduleInstantMovementEffect(EffectPtr);
					}
				}
			}));
}