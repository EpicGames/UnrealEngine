// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosMover/ChaosMoverSimulation.h"

#include "Chaos/ChaosEngineInterface.h"
#include "Chaos/Character/CharacterGroundConstraint.h"
#include "Chaos/Character/CharacterGroundConstraintContainer.h"
#include "Chaos/ContactModification.h"
#include "Chaos/KinematicTargets.h"
#include "Chaos/PBDJointConstraintData.h"
#include "Chaos/PhysicsObjectInternalInterface.h"
#include "Chaos/ResimConsoleVariables.h"
#include "ChaosMover/ChaosMovementMode.h"
#include "ChaosMover/ChaosCompositeMovementMode.h"
#include "ChaosMover/ChaosMovementModeTransition.h"
#include "ChaosMover/ChaosMoverConsoleVariables.h"
#include "ChaosMover/ChaosMoverLog.h"
#include "ChaosMover/ChaosMoverSimulationTypes.h"
#include "ChaosMover/Character/ChaosCharacterInputs.h"
#include "ChaosMover/PathedMovement/ChaosPathedMovementTypes.h"
#include "ChaosMover/Utilities/ChaosGroundMovementUtils.h"
#include "ChaosMover/Utilities/ChaosMoverQueryUtils.h"
#include "DefaultMovementSet/CharacterMoverSimulationTypes.h"
#include "DefaultMovementSet/MoverMontageSimulationTypes.h"
#include "DefaultMovementSet/LayeredMoves/MontageStateProvider.h"
#include "Engine/World.h"
#include "Framework/Threading.h"
#include "MoveLibrary/FloorQueryUtils.h"
#include "MoveLibrary/MovementMixer.h"
#include "MoveLibrary/MoverBlackboard.h"
#include "MoveLibrary/RollbackBlackboard.h"
#include "MoveLibrary/WaterMovementUtils.h"
#include "MovementModeStateMachine.h"
#include "MoverSimulationTypes.h"
#include "PBDRigidsSolver.h"
#include "Physics/NetworkPhysicsComponent.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "Physics/NetworkPhysicsComponent.h"
#include "PhysicsEngine/PhysicsObjectExternalInterface.h"
#include "PhysicsProxy/CharacterGroundConstraintProxy.h"
#include "UObject/UObjectGlobals.h"
#include "Chaos/GeometryParticles.h"
#include "MoverGameplayTagLog.h"
#include "ChaosMover/ChaosMoverActionTypes.h"

#if WITH_CHAOS_VISUAL_DEBUGGER
#include "ChaosVisualDebugger/ChaosVisualDebuggerTrace.h"
#include "ChaosVisualDebugger/MoverCVDRuntimeTrace.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosMoverSimulation)

UChaosMoverSimulation::UChaosMoverSimulation()
{
}

const FMoverDataCollection& UChaosMoverSimulation::GetLocalSimInput() const
{
	return LocalSimInput;
}

FMoverDataCollection& UChaosMoverSimulation::GetLocalSimInput_Mutable()
{
	// Only the Gameplay Thread is allowed to write to the local simulation input data collection
	Chaos::EnsureIsInGameThreadContext();

	return LocalSimInput;
}

FMoverDataCollection& UChaosMoverSimulation::GetDebugSimData()
{
	return DebugSimData;
}

TWeakObjectPtr<const UBaseMovementMode> UChaosMoverSimulation::GetCurrentMovementMode() const
{
	return StateMachine.GetCurrentMode();
}

TWeakObjectPtr<const UBaseMovementMode> UChaosMoverSimulation::FindMovementModeByName(const FName& Name) const
{
	return StateMachine.FindMovementMode(Name);
}

TWeakObjectPtr<UBaseMovementMode> UChaosMoverSimulation::GetCurrentMovementMode_Mutable()
{
	return StateMachine.GetCurrentMode();
}

TWeakObjectPtr<UBaseMovementMode> UChaosMoverSimulation::FindMovementModeByName_Mutable(const FName& Name)
{
	return StateMachine.FindMovementMode_Mutable(Name);
}

void UChaosMoverSimulation::QueueNextMovementMode(FName ModeName)
{
	StateMachine.QueueNextMode(ModeName);
}

bool UChaosMoverSimulation::HasMovementMode(const FName& Name) const
{
	return StateMachine.FindMovementMode(Name) != nullptr;
}

void UChaosMoverSimulation::ApplyNetInputData(const FMoverInputCmdContext& InNetInputCmd)
{
	NetInputCmd = InNetInputCmd;
	bInputCmdOverridden = true;
}

void UChaosMoverSimulation::BuildNetInputData(FMoverInputCmdContext& OutNetInputCmd) const
{
	OutNetInputCmd = NetInputCmd;
}

void UChaosMoverSimulation::ApplyNetStateData(const FMoverSyncState& InNetSyncState)
{
	NetSyncState = InNetSyncState;
	bSyncStateOverridden = true;
}

void UChaosMoverSimulation::BuildNetStateData(FMoverSyncState& OutNetSyncState) const
{
	OutNetSyncState = CurrentSyncState;
}

void UChaosMoverSimulation::Init(const FInitParams& InitParams)
{
	// Only the Gameplay Thread is allowed to Init the chaos mover simulation
	Chaos::EnsureIsInGameThreadContext();

	MovementMixerWeakPtr = InitParams.MovementMixer.Get();
	CharacterGroundConstraintHandle = InitParams.CharacterGroundConstraintHandle;
	ActuationConstraintHandle = InitParams.ActuationConstraintHandle;
	ActuationConstraintEndPointParticleHandle = InitParams.ActuationConstraintEndPointParticleHandle;
	PhysicsObject = InitParams.PhysicsObject;
	Solver = InitParams.Solver;
	World = InitParams.World;
	bIsUsingAsyncPhysics = InitParams.bIsUsingAsyncPhysics;

	CurrentSyncState = InitParams.InitialSyncState;

	// Set the pathed movement basis to the initial location of the controlled particle
	// We set this up before state machine init so the pathed movement modes have a pathed movement basis to work with on registration
	FTransform NewMovementBasisTransform(InitParams.TransformOnInit.GetRotation(), InitParams.TransformOnInit.GetLocation());
	SetMovementBasisTransform(NewMovementBasisTransform);

	UE::ChaosMover::FMoverStateMachine::FInitParams StateMachineInitParams;
	StateMachineInitParams.ImmediateMovementModeTransition = InitParams.ImmediateModeTransition;
	StateMachineInitParams.NullMovementMode = InitParams.NullMovementMode;
	StateMachineInitParams.Simulation = this;
	StateMachine.Init(StateMachineInitParams);

	for (const TPair<FName, TWeakObjectPtr<UBaseMovementMode>>& Element : InitParams.ModesToRegister)
	{
		const FName& ModeName = Element.Key;
		TWeakObjectPtr<UBaseMovementMode> Mode = Element.Value;

		if (Mode.Get() == nullptr)
		{
			UE_LOGF(LogChaosMover, Warning, "Invalid Movement Mode type '%ls' detected. Mover actor will not function correctly.", *ModeName.ToString());
			continue;
		}

		if (UChaosMovementMode* ChaosMode = Cast<UChaosMovementMode>(Mode.Get()))
		{
			ChaosMode->SetSimulation(this);
		}

		bool bIsDefaultMode = (InitParams.StartingMovementMode == ModeName);
		StateMachine.RegisterMovementMode(ModeName, Mode, bIsDefaultMode);
	}

	for (const TWeakObjectPtr<UBaseMovementModeTransition>& Transition : InitParams.TransitionsToRegister)
	{
		if (UChaosMovementModeTransition* ChaosTransition = Cast<UChaosMovementModeTransition>(Transition.Get()))
		{
			ChaosTransition->SetSimulation(this);
		}

		StateMachine.RegisterGlobalTransition(Transition);
	}

	StateMachine.QueueNextMode(StateMachine.GetDefaultModeName());

	OnInit();

	bInitialized = true;
}

void UChaosMoverSimulation::SetOwner(const AActor* Owner)
{
	if (ensure(Owner))
	{
		OwnerLocalRole = Owner->GetLocalRole();
		StateMachine.SetOwnerActorName(Owner->GetName());
		StateMachine.SetOwnerActorLocalNetRole(OwnerLocalRole);
	}
}

void UChaosMoverSimulation::Deinit()
{
	bInitialized = false;

	OnDeinit();
}

void UChaosMoverSimulation::BackupState(FChaosMoverPredictionStateBackup& Out) const
{
	// Todo @Harsha Extend this to a generic state back up flow
	const UMoverBlackboard* BB = GetBlackboard();
	if (!BB)
	{
		return;
	}

	// Reset to prevent stale values from a previous backup polluting this one
	Out = FChaosMoverPredictionStateBackup();

	FFloorCheckResult FloorResult;
	if (BB->TryGet(CommonBlackboard::LastFloorResult, FloorResult))
	{
		Out.BlackboardLastFloorResult = FloorResult;
	}

	FWaterCheckResult WaterResult;
	if (BB->TryGet(CommonBlackboard::LastWaterResult, WaterResult))
	{
		Out.BlackboardLastWaterResult = WaterResult;
	}

	UE::ChaosMover::FGroundDynamicsInfo GroundInfo;
	if (BB->TryGet(UE::ChaosMover::Blackboard::GroundDynamicsInfo, GroundInfo))
	{
		Out.BlackboardGroundDynamicsInfo = GroundInfo;
	}
}

void UChaosMoverSimulation::RestoreState(const FChaosMoverPredictionStateBackup& In)
{
	UMoverBlackboard* BB = GetBlackboard_Mutable();
	if (!BB)
	{
		return;
	}

	if (In.BlackboardLastFloorResult.IsSet())
	{
		BB->Set(CommonBlackboard::LastFloorResult, In.BlackboardLastFloorResult.GetValue());
	}
	else
	{
		BB->Invalidate(CommonBlackboard::LastFloorResult);
	}

	if (In.BlackboardLastWaterResult.IsSet())
	{
		BB->Set(CommonBlackboard::LastWaterResult, In.BlackboardLastWaterResult.GetValue());
	}
	else
	{
		BB->Invalidate(CommonBlackboard::LastWaterResult);
	}

	if (In.BlackboardGroundDynamicsInfo.IsSet())
	{
		BB->Set(UE::ChaosMover::Blackboard::GroundDynamicsInfo, In.BlackboardGroundDynamicsInfo.GetValue());
	}
	else
	{
		BB->Invalidate(UE::ChaosMover::Blackboard::GroundDynamicsInfo);
	}
}

void UChaosMoverSimulation::PredictTrajectory(const FMoverTimeStep& TimeStep, const UE::ChaosMover::FSimulationInputData& InputData, const FChaosMoverTrajectoryPredictionInputs* PredInputs, const FMoverSyncState& StartSyncState, UE::ChaosMover::FSimulationOutputData& OutputData) const
{
	Chaos::EnsureIsInPhysicsThreadContext();

	constexpr int32 MinNumSteps = 1;
	constexpr int32 MaxNumSteps = 120;
	constexpr float MinDt = 0.001f;

	// Using physics fixed dt
	const float Dt = TimeStep.StepMs * 0.001f;

	int32 NumSteps = 0;
	if (PredInputs)
	{
		// Requested horizon is NumPredictionSteps * SecondsPerStep (seconds). Cover at least that amount of time with steps of size Dt.
		const float RequestedHorizonSec = static_cast<float>(PredInputs->NumPredictionSteps) * PredInputs->SecondsPerStep;
		NumSteps = FMath::Clamp(FMath::CeilToInt(RequestedHorizonSec / Dt), MinNumSteps, MaxNumSteps);
		
		const float NewHorizonSec = NumSteps * Dt;
		if (NewHorizonSec < RequestedHorizonSec - UE_KINDA_SMALL_NUMBER)
		{
			UE_LOGF(LogChaosMover, Warning, "Predicted trajectory shorter than requested %.2f s, limited to %.2f s.",
				RequestedHorizonSec, NewHorizonSec);
		}
	}
	
	if (FMath::IsNearlyZero(Dt) || NumSteps == 0)
	{
		UE_LOGF(LogChaosMover, Warning, "Invalid predicted trajectory parameters detected, exiting trajectory prediction.");
		return;
	}

	const TStrongObjectPtr<const UBaseMovementMode> ModePtr = FindMovementModeByName(StartSyncState.MovementMode).Pin();
	const UBaseMovementMode* Mode = ModePtr.Get();
	if (!Mode)
	{
		UE_LOGF(LogChaosMover, Warning, "No active movement mode found, exiting trajectory prediction.");
		return;
	}

	FMoverTickStartData StepState;
	StepState.InputCmd = InputData.InputCmd;
	StepState.SyncState = StartSyncState;
	StepState.AuxState = InputData.AuxInputState;

	FMoverSimContext SimContext;
	// TODO: Need to figure out how to put the simulation in predictive mode. 
	// Currently, any GenerateMove call could find the Simulation object and perform a non-const operation. 
	// So this const-cast is not exposing anything new.
	SimContext.Simulation = const_cast<UChaosMoverSimulation*>(this);	
	SimContext.Blackboard = SimContext.Simulation->GetRollbackBlackboardPredictionWrapper();
	SimContext.SimulationOwner = nullptr;

	FMoverDefaultSyncState* StepDefaultSyncState = StepState.SyncState.SyncStateCollection.FindMutableDataByType<FMoverDefaultSyncState>();
	if (!StepDefaultSyncState)
	{
		return;
	}

	FMoverTimeStep FutureTimeStep = TimeStep;
	FutureTimeStep.StepMs = Dt * 1000.0f;

	FChaosMoverPredictedTrajectoryData& TrajOut = OutputData.AdditionalOutputData.FindOrAddMutableDataByType<FChaosMoverPredictedTrajectoryData>();
	TrajOut.Deltas.SetNumUninitialized(NumSteps + 1);

	FTransform PrevWorldTransform(StepDefaultSyncState->GetOrientation_WorldSpace(), StepDefaultSyncState->GetLocation_WorldSpace());
	TrajOut.BaseTransform = PrevWorldTransform;
	FVector PriorVelocity = StepDefaultSyncState->GetVelocity_WorldSpace();
	FRotator PriorOrientation = StepDefaultSyncState->GetOrientation_WorldSpace();

	for (int32 i = 0; i < NumSteps + 1; ++i)
	{
		RollbackBlackboard->BeginPredictionFrame(FutureTimeStep);

		FTrajectorySampleInfo& Sample = TrajOut.Deltas[i];
		const FTransform CurrentWorldTransform(StepDefaultSyncState->GetOrientation_WorldSpace(), StepDefaultSyncState->GetLocation_WorldSpace());
		if (i == 0)
		{
			Sample.Transform = FTransform::Identity;
		}
		else
		{
			Sample.Transform = CurrentWorldTransform.GetRelativeTransform(PrevWorldTransform);
		}
		Sample.LinearVelocity = StepDefaultSyncState->GetVelocity_WorldSpace();
		Sample.SimTimeMs = FutureTimeStep.BaseSimTimeMs;

		if (i == 0)
		{
			Sample.InstantaneousAcceleration = FVector::ZeroVector;
			Sample.AngularVelocity = FRotator::ZeroRotator;
		}
		else
		{
			Sample.InstantaneousAcceleration = (Sample.LinearVelocity - PriorVelocity) / Dt;
			Sample.AngularVelocity = (StepDefaultSyncState->GetOrientation_WorldSpace() - PriorOrientation) * (1.f / Dt);
		}

		PrevWorldTransform = CurrentWorldTransform;
		PriorVelocity = Sample.LinearVelocity;
		PriorOrientation = StepDefaultSyncState->GetOrientation_WorldSpace();

		if (i == NumSteps)
		{
			break;
		}

		FProposedMove Proposed;
		Mode->GenerateMove(SimContext, StepState, FutureTimeStep, Proposed);

		// Advance step (simple integration)
		const FVector NextPos = StepDefaultSyncState->GetLocation_WorldSpace() + (Proposed.LinearVelocity * Dt);
		const FRotator NextRot = UMovementUtils::ApplyAngularVelocityToRotator(StepDefaultSyncState->GetOrientation_WorldSpace(), Proposed.AngularVelocityDegrees, Dt);

		StepDefaultSyncState->SetTransforms_WorldSpace(
			NextPos,
			NextRot,
			Proposed.LinearVelocity,
			Proposed.AngularVelocityDegrees,
			StepDefaultSyncState->GetMovementBase(),
			StepDefaultSyncState->GetMovementBaseBoneName());

		FutureTimeStep.BaseSimTimeMs += FutureTimeStep.StepMs;
		++FutureTimeStep.ServerFrame;
	}

	RollbackBlackboard->EndPrediction();

}

void UChaosMoverSimulation::OnInit()
{
	
}

void UChaosMoverSimulation::OnDeinit()
{

}

void UChaosMoverSimulation::ProcessInputs(int32 PhysicsStep, const FMoverTimeStep& TimeStep, const UE::ChaosMover::FSimulationInputData& InputData)
{
	QUICK_SCOPE_CYCLE_COUNTER(ChaosMoverSimulation_ProcessInputs);

	Chaos::EnsureIsInPhysicsThreadContext();

	// Ensure that we're using the correct inputs. On the server or during resim the inputs will
	// be overridden by the network physics component, so use them instead of the generated inputs.
	// On the client the input data is read out to be sent to the server so update the data that is read.
	if (bInputCmdOverridden)
	{
		// If we are on the server we can also have inputs from actions applied to the mover component
		// e.g. Launch. If so, use the server versions
		if (UE::ChaosMover::CVars::bEnableServerLaunchOverride)
		{
			if (const FChaosMoverSimulationDefaultInputs* SimInputs = LocalSimInput.FindDataByType<FChaosMoverSimulationDefaultInputs>())
			{
				if (SimInputs->AsyncNetworkPhysicsComponent && SimInputs->AsyncNetworkPhysicsComponent->IsServer())
				{
					if (const FChaosMoverLaunchInputs* NetLaunchInputs = NetInputCmd.InputCollection.FindDataByType<FChaosMoverLaunchInputs>())
					{
						UE_LOGF(LogTemp, Warning, "Removed Launch. Frame=%u, Value=%ls", TimeStep.ServerFrame, *NetLaunchInputs->LaunchVelocityOrImpulse.ToCompactString());
						NetInputCmd.InputCollection.RemoveDataByType(FChaosMoverLaunchInputs::StaticStruct());
					}

					if (const FChaosMoverLaunchInputs* LocalLaunchInputs = InputData.InputCmd.InputCollection.FindDataByType<FChaosMoverLaunchInputs>())
					{
						UE_LOGF(LogTemp, Warning, "Added Launch. Frame=%u, Value=%ls", TimeStep.ServerFrame, *LocalLaunchInputs->LaunchVelocityOrImpulse.ToCompactString());
						FChaosMoverLaunchInputs& LaunchInputs = NetInputCmd.InputCollection.FindOrAddMutableDataByType<FChaosMoverLaunchInputs>();
						LaunchInputs = *LocalLaunchInputs;
					}
				}
			}
		}

		InputData.InputCmd = NetInputCmd;
		bInputCmdOverridden = false;
	}
	else
	{
		NetInputCmd = InputData.InputCmd;
	}

	// Sync state is overwritten by network physics for sim proxies and for resimulation
	// In either case we rollback the simulation to invalidate the blackboard and reset
	// the state machine based on the new state
	if (bSyncStateOverridden)
	{
		const FMoverSyncState PrevSyncState = CurrentSyncState;
		CurrentSyncState = NetSyncState;
		OnSimulationRollback(TimeStep, PrevSyncState);
		bSyncStateOverridden = false;
	}
}

void UChaosMoverSimulation::SimulationTick(const FMoverSimContext& SimContext, const FMoverTimeStep& TimeStep, const UE::ChaosMover::FSimulationInputData& InputData, UE::ChaosMover::FSimulationOutputData& OutputData)
{
	QUICK_SCOPE_CYCLE_COUNTER(ChaosMoverSimulation_SimulationTick);

	Chaos::EnsureIsInPhysicsThreadContext();

	const bool bIsBubbleResim = Chaos::ResimConsoleVars::bIsBubbleResimEnabled && TimeStep.bIsResimulating;
	const bool bShouldTick = bInitialized && (!bIsBubbleResim || ShouldResim());

	if (bShouldTick)
	{
		RollbackBlackboard->BeginSimulationFrame(TimeStep);

		OnPreSimulationTick(TimeStep, InputData);
		OnSimulationTick(SimContext, TimeStep, InputData, OutputData);
		OnPostSimulationTick(TimeStep, OutputData);

		RollbackBlackboard->EndSimulationFrame();
	}
}

bool UChaosMoverSimulation::ShouldResim() const
{
	const Chaos::FPBDRigidParticleHandle* ControlledParticle = GetControlledParticle();
	return ControlledParticle ? (ControlledParticle->ResimType() != Chaos::EResimType::FrozenDuringResim) : true;
}

void UChaosMoverSimulation::ModifyContacts(const FMoverTimeStep& TimeStep, const UE::ChaosMover::FSimulationInputData& InputData, const UE::ChaosMover::FSimulationOutputData& OutputData, Chaos::FCollisionContactModifier& Modifier)
{
	QUICK_SCOPE_CYCLE_COUNTER(ChaosMoverSimulation_ModifyContacts);

	Chaos::EnsureIsInPhysicsThreadContext();

	if (!bInitialized)
	{
		return;
	}

	if (TStrongObjectPtr<const UBaseMovementMode> CurrentModePtr = StateMachine.GetCurrentMode().Pin())
	{
		if (const UChaosMovementMode* ChaosMode = Cast<UChaosMovementMode>(CurrentModePtr.Get()))
		{
			// Base contact modification
			// Disable collisions for actors and components on the ignore list in the query params
			if (ChaosMode->IgnoredCollisionMode == EChaosMoverIgnoredCollisionMode::DisableCollisionsWithIgnored)
			{
				Chaos::FGeometryParticleHandle* UpdatedParticle = nullptr;
				const FChaosMoverSimulationDefaultInputs* SimInputs = LocalSimInput.FindDataByType<FChaosMoverSimulationDefaultInputs>();
				if (SimInputs)
				{
					Chaos::FReadPhysicsObjectInterface_Internal ReadInterface = Chaos::FPhysicsObjectInternalInterface::GetRead();
					UpdatedParticle = ReadInterface.GetParticle(SimInputs->PhysicsObject);
				}

				if (!UpdatedParticle)
				{
					return;
				}

				for (Chaos::FContactPairModifier& PairModifier : Modifier.GetContacts(UpdatedParticle))
				{
					const int32 OtherIdx = UpdatedParticle == PairModifier.GetParticlePair()[0] ? 1 : 0;

					if (const Chaos::FShapeInstance* Shape = PairModifier.GetShape(OtherIdx))
					{
						const Chaos::Filter::FCombinedShapeFilterData& CombinedShapeFilter = ChaosInterface::GetCombinedShapeFilterData(*Shape);
						const Chaos::Filter::FInstanceData& InstanceData = CombinedShapeFilter.GetInstanceData();
						const Chaos::Filter::FShapeFilterData& FilterData = CombinedShapeFilter.GetShapeFilterData();
						uint32 ComponentID = InstanceData.GetComponentId();
						if (SimInputs->CollisionQueryParams.GetIgnoredComponents().Contains(ComponentID))
						{
							PairModifier.Disable();
							continue;
						}

						uint32 ActorID = InstanceData.GetOwnerId();
						if (SimInputs->CollisionQueryParams.GetIgnoredSourceObjects().Contains(ActorID))
						{
							PairModifier.Disable();
							continue;
						}

						FMaskFilter ShapeMaskFilter = FilterData.GetMaskFilter();
						if (SimInputs->CollisionQueryParams.IgnoreMask & ShapeMaskFilter)
						{
							PairModifier.Disable();
							continue;
						}
					}
				}
			}

			// Mode specific contact modification
			ChaosMode->ModifyContacts(TimeStep, InputData, OutputData, Modifier);
		}
	}

	OnModifyContacts(TimeStep, InputData, OutputData, Modifier);
}


void UChaosMoverSimulation::OnSimulationRollback(const FMoverTimeStep& NewTimeStep, const FMoverSyncState& PrevSyncState)
{
	QUICK_SCOPE_CYCLE_COUNTER(ChaosMoverSimulation_OnSimulationRollback);

	// Mover Gameplay Tag Logging
	MOVER_TAG_LOG(LogChaosMover, "[WT:RollbackTrigger] Actor=%s Role=%s Frame=%d IsResim=%d IsRollback=%d | rolling back to this frame",
		*DebugOwnerName, *DebugOwnerRole, NewTimeStep.ServerFrame, (int32)NewTimeStep.bIsResimulating, (int32)true);

	// Rollback blackboard on the first frame of resimulation
	Blackboard->Invalidate(EInvalidationReason::Rollback);
	RollbackBlackboard->BeginRollback(NewTimeStep);

	// Remove any events that have been stored for this simulation tick
	Events.Empty();

	// Mover Gameplay Tag Logging
	MOVER_TAG_LOG(LogChaosMover, "[WT:EventBufClear] Actor=%s Role=%s Frame=%d IsResim=%d IsRollback=%d | in-flight Events cleared",
		*DebugOwnerName, *DebugOwnerRole, NewTimeStep.ServerFrame, (int32)NewTimeStep.bIsResimulating, (int32)true);


	// Make sure the movement basis matches the one in the sync state
	if (FChaosMovementBasis* MovementBasis = CurrentSyncState.SyncStateCollection.FindMutableDataByType<FChaosMovementBasis>())
	{
		SetMovementBasisTransform(FTransform(MovementBasis->BasisRotation, MovementBasis->BasisLocation));
	}

	StateMachine.OnSimulationRollback(NewTimeStep, PrevSyncState, CurrentSyncState);
	
	RollbackBlackboard->EndRollback();
}

void UChaosMoverSimulation::OnPreSimulationTick(const FMoverTimeStep& TimeStep, const UE::ChaosMover::FSimulationInputData& InputData)
{
	QUICK_SCOPE_CYCLE_COUNTER(ChaosMoverSimulation_OnPreSimulationTick);

	InternalServerFrame = TimeStep.ServerFrame;

	// Update the sync state from the current physics state
	FMoverDefaultSyncState& PreSimDefaultSyncState = CurrentSyncState.SyncStateCollection.FindOrAddMutableDataByType<FMoverDefaultSyncState>();
	PreSimDefaultSyncState.SetMovementBase(nullptr);
	if (const Chaos::FPBDRigidParticleHandle* ParticleHandle = GetControlledParticle())
	{
		PreSimDefaultSyncState.SetTransforms_WorldSpace(ParticleHandle->GetX(), FRotator(ParticleHandle->GetR()), ParticleHandle->GetV(), FMath::RadiansToDegrees(ParticleHandle->GetW()));
	}

	if (TStrongObjectPtr<UBaseMovementMode> CurrentModePtr = StateMachine.GetCurrentMode().Pin())
	{
		FChaosMoverSimulationInterfaceCache Cache;
		if (UChaosMovementMode* ChaosMode = Cast<UChaosMovementMode>(CurrentModePtr.Get()))
		{
			ChaosMode->CollectSimulationInterfaces(Cache);
		}

		FChaosMoverPreSimContext Context{ TimeStep, InputData, this };
		for (IChaosPreSimulationTickInterface* Interface : Cache.PreSimInterfaces)
		{
			if (Interface)
			{
				Interface->PreSimulationTick_Async(Context);
			}
		}
	}
}

void UChaosMoverSimulation::OnSimulationTick(const FMoverSimContext& SimContext, const FMoverTimeStep& TimeStep, const UE::ChaosMover::FSimulationInputData& InputData, UE::ChaosMover::FSimulationOutputData& OutputData)
{
	QUICK_SCOPE_CYCLE_COUNTER(ChaosMoverSimulation_OnSimulationTick);

	check(Blackboard.Get());

	FMoverTickStartData TickStartData(InputData.InputCmd, CurrentSyncState, InputData.AuxInputState);
	FMoverTickEndData TickEndData(&CurrentSyncState, &InputData.AuxInputState);

	StateMachine.OnSimulationTick(SimContext, TimeStep, TickStartData, Blackboard.Get(), MovementMixerWeakPtr.Get(), TickEndData);

	if (const FChaosMoverTrajectoryPredictionInputs* PredInputs = LocalSimInput.FindDataByType<FChaosMoverTrajectoryPredictionInputs>())
	{
		// PredictTrajectory() can mutate state; so we backup/restore state around PredictTrajectory()
		FChaosMoverPredictionStateBackup PredictionStateBackup;
		BackupState(PredictionStateBackup);
		PredictTrajectory(TimeStep, InputData, PredInputs, TickEndData.SyncState, OutputData);
		RestoreState(PredictionStateBackup);
	}

	// Copy the sync state locally and to the output data
	OutputData.SyncState = TickEndData.SyncState;
	OutputData.LastUsedInputCmd = InputData.InputCmd;
	OutputData.InternalGameplayTags = StateMachine.GetLastKnownGameplayTags();
}

FChaosMoverPostSimContext UChaosMoverSimulation::BuildPostSimContext(
	const FMoverTimeStep& TimeStep, UE::ChaosMover::FSimulationOutputData& OutputData)
{
	return FChaosMoverPostSimContext{
		.TimeStep   = TimeStep,
		.OutputData = OutputData,
		.Simulation = this,
	};
}

void UChaosMoverSimulation::SetActuationTargetTransform(const FTransform& TargetTransform)
{
	Chaos::FPBDRigidsEvolution* Evolution = Solver ? Solver->GetEvolution() : nullptr;
	if (!Evolution)
	{
		return;
	}

	Chaos::FKinematicTarget KinematicPathTarget = Chaos::FKinematicTarget::MakePositionTarget(TargetTransform);
	if (ActuationConstraintEndPointParticleHandle)
	{	
		Evolution->SetParticleKinematicTarget(ActuationConstraintEndPointParticleHandle, KinematicPathTarget);
	}
	if (IsControlledParticleKinematic())
	{
		if (Chaos::FPBDRigidParticleHandle* ParticleHandle = GetControlledParticle())
		{
			Evolution->SetParticleKinematicTarget(ParticleHandle, KinematicPathTarget);
		}
	}
}

void UChaosMoverSimulation::TeleportActuationTarget(const FTransform& TargetTransform, bool AlsoTeleportControlledParticle /*= false*/)
{
	Chaos::FPBDRigidsEvolution* Evolution = Solver ? Solver->GetEvolution() : nullptr;
	if (!Evolution)
	{
		return;
	}

	if (ActuationConstraintEndPointParticleHandle)
	{	
		Evolution->SetParticleTransform(ActuationConstraintEndPointParticleHandle, TargetTransform.GetTranslation(), TargetTransform.GetRotation(), /*bIsTeleport=*/true);
	}
	if (AlsoTeleportControlledParticle)
	{
		if (Chaos::FPBDRigidParticleHandle* ParticleHandle = GetControlledParticle())
		{
			Evolution->SetParticleTransform(ParticleHandle, TargetTransform.GetTranslation(), TargetTransform.GetRotation(), /*bIsTeleport=*/true);
		}
	}
}

const FTransform& UChaosMoverSimulation::GetMovementBasisTransform() const
{
	return MovementBasisTransform;
}

void UChaosMoverSimulation::SetMovementBasisTransform(const FTransform& InMovementBasisTransform)
{
	MovementBasisTransform = InMovementBasisTransform;
}

bool UChaosMoverSimulation::CanTeleport(const FTransform& TargetTransform, bool bUseActorRotation, const FMoverSyncState& SyncState)
{
	Chaos::FPBDRigidsEvolution* Evolution = Solver ? Solver->GetEvolution() : nullptr;
	Chaos::FPBDRigidParticleHandle* ParticleHandle = GetControlledParticle();
	if (!Evolution || !ParticleHandle)
	{
		return false;
	}

	// TODO: Add flags to modes for a teleport policy:
	// - Does the object fit or encroach? What shape type to use, or AABB, OBB?
	// - Is the floor underneath walkable?
	// - ... or should the mode be directly in charge of deciding whether one can teleport or not?
	const FMoverDefaultSyncState* DefaultSyncState = SyncState.SyncStateCollection.FindDataByType<FMoverDefaultSyncState>();
	if (!DefaultSyncState)
	{
		return false;
	}
	FTransform FinalTargetTransform(bUseActorRotation ? FQuat(DefaultSyncState->GetOrientation_WorldSpace()) : TargetTransform.GetRotation(), TargetTransform.GetLocation());
	TStrongObjectPtr<UBaseMovementMode> CurrentModePtr = StateMachine.GetCurrentMode().Pin();
	UChaosMovementMode* ChaosMovementMode = Cast<UChaosMovementMode>(CurrentModePtr.Get());
	return !ChaosMovementMode || ChaosMovementMode->CanTeleport(FinalTargetTransform, CurrentSyncState);
}

void UChaosMoverSimulation::Teleport(const FTransform& TargetTransform, const FMoverTimeStep& TimeStep, FMoverSyncState& OutputState)
{
	Chaos::FPBDRigidsEvolution* Evolution = Solver ? Solver->GetEvolution() : nullptr;
	Chaos::FPBDRigidParticleHandle* ParticleHandle = GetControlledParticle();
	if (!Evolution || !ParticleHandle)
	{
		return;
	}

	FMoverDefaultSyncState& DefaultSyncState = OutputState.SyncStateCollection.FindOrAddMutableDataByType<FMoverDefaultSyncState>();
	FTransform CurrentTransform(FQuat(DefaultSyncState.GetOrientation_WorldSpace()), DefaultSyncState.GetLocation_WorldSpace());
	FTransform TeleportDeltaTransform = CurrentTransform.Inverse() * TargetTransform;
	
	TStrongObjectPtr<UBaseMovementMode> CurrentModePtr = StateMachine.GetCurrentMode().Pin();
	UChaosMovementMode* ChaosMovementMode = Cast<UChaosMovementMode>(CurrentModePtr.Get());

	// Teleport the movement basis transform if the current mode uses one. Should we do this if ANY mode on the state machine uses a movement basis transform?
	if (ChaosMovementMode && ChaosMovementMode->UsesMovementBasisTransform())
	{
		// Transform that changes CurrentParticleTransform into TargetTransform
		FTransform NewMovementBasisTransform = MovementBasisTransform * TeleportDeltaTransform;
		SetMovementBasisTransform(NewMovementBasisTransform);
		// Add and/or update the movement basis transform as part of our state
		// For this to lead to a change of movement basis transform for sim proxies, we either need movement replication to be on
		// or, for an actor moved kinematically, "Apply Sim Proxy State at Runtime" to be checked in the network settings component
		FChaosMovementBasis& MovementBasis = OutputState.SyncStateCollection.FindOrAddMutableDataByType<FChaosMovementBasis>();
		MovementBasis.BasisLocation = NewMovementBasisTransform.GetLocation();
		MovementBasis.BasisRotation = NewMovementBasisTransform.GetRotation();
	}

	// Teleport the actuation target
	if (IsActuationConstraintEnabled())
	{
		TeleportActuationTarget(TargetTransform, /* AlsoTeleportControlledParticle = */ false); // We teleport the particle separately below
	}

	// Completely disable the character constraint and go into default mode
	// TODO: reestablish the constraint if ground is found at the target transform
	// This might not be necessary, the character ground constraint should be teleport friendly
	if (IsCharacterGroundConstraintEnabled())
	{
		DisableCharacterGroundConstraint();
		StateMachine.SetModeImmediately(TimeStep, StateMachine.GetDefaultModeName());
	}

	// Invalidate floor results. Should there be a way to indicate which blackboard results to invalidate for which type of operation, like rewind, teleport, etc?
	Blackboard->Invalidate(CommonBlackboard::LastFloorResult);
	Blackboard->Invalidate(CommonBlackboard::LastFoundDynamicMovementBase);

	// Rotate the velocities by the teleport rotation
	FVector NewLinearVelocity = TeleportDeltaTransform.TransformVector(ParticleHandle->GetV());
	FVector NewAngularVelocity = TeleportDeltaTransform.TransformVector(ParticleHandle->GetW());

	// Teleport the particle
	//     - Disable it
	Evolution->DisableParticle(ParticleHandle);
	//	   - Place the particle at the target transform
	Evolution->SetParticleTransform(ParticleHandle, TargetTransform.GetTranslation(), TargetTransform.GetRotation(), /*bIsTeleport=*/true);
	Evolution->SetParticleVelocities(ParticleHandle, NewLinearVelocity, NewAngularVelocity);
	//	   - Re-enable the particle
	Evolution->EnableParticle(ParticleHandle);

	// Invalidate the movement base for now, until we reestablish the character ground constraint
	DefaultSyncState.SetMovementBase(nullptr);
	// Make sure the sync state matches the particle transforms
	DefaultSyncState.SetTransforms_WorldSpace(ParticleHandle->GetX(), FRotator(ParticleHandle->GetR()), ParticleHandle->GetV(), FMath::RadiansToDegrees(ParticleHandle->GetW()));
	// Rotate the move direction intent
	DefaultSyncState.MoveDirectionIntent = TeleportDeltaTransform.TransformVector(DefaultSyncState.MoveDirectionIntent);
}

void UChaosMoverSimulation::OnPostSimulationTick(const FMoverTimeStep& TimeStep, UE::ChaosMover::FSimulationOutputData& OutputData)
{
	QUICK_SCOPE_CYCLE_COUNTER(ChaosMoverSimulation_OnPostSimulationTick);

	if (TStrongObjectPtr<UBaseMovementMode> CurrentModePtr = StateMachine.GetCurrentMode().Pin())
	{
		// Collect IChaosPostSimulationTickInterface implementations from the current mode.
		FChaosMoverSimulationInterfaceCache Cache;
		if (UChaosMovementMode* ChaosMode = Cast<UChaosMovementMode>(CurrentModePtr.Get()))
		{
			ChaosMode->CollectSimulationInterfaces(Cache);
		}

		// Disable both constraints upfront; implementations re-enable if they need them.
		DisableCharacterGroundConstraint();
		DisableActuationConstraint();

		// Dispatch to all collected implementations.
		FChaosMoverPostSimContext Context = BuildPostSimContext(TimeStep, OutputData);
		bool bAnyDispatched = false;
		for (IChaosPostSimulationTickInterface* Interface : Cache.PostSimInterfaces)
		{
			if (Interface)
			{
				Interface->PostSimulationTick_Async(Context);
				bAnyDispatched = true;
			}
		}

		// Fallback: if no implementation was dispatched (or one explicitly opted back in via
		// bApplyFallbackVelocity), apply the sync state velocity directly to the particle so
		// it does not drift under raw physics.
		// Angular velocity is intentionally left to physics -- a generic mode that does
		// not implement the interface should not have its rotation zeroed.
		// Non-resim sim proxies are driven by replication, not simulation; skip the write so
		// their replication-driven velocity is not overwritten by sync state (mirrors the
		// non-resim-proxy branch in ChaosCharacterSimUtils::ReconcileParticleVelocity).
		if (!bAnyDispatched || Context.bApplyFallbackVelocity)
		{
			if (const FMoverDefaultSyncState* PostSimSyncState =
					OutputData.SyncState.SyncStateCollection.FindDataByType<FMoverDefaultSyncState>())
			{
				if ((TimeStep.bIsResimulating || !IsNonResimSimProxy()))
				{
					if (Chaos::FPBDRigidParticleHandle* ParticleHandle = GetControlledParticle())
					{
						ParticleHandle->SetV(PostSimSyncState->GetVelocity_WorldSpace());
					}
				}
			}
		}
	}

	// Collect simulation-authoritative montage state from all active layered moves that produce
	// montage output (FLayeredMove_MontageStateProvider). Runs unconditionally so that modes that
	// do not implement IChaosCharacterMovementModeInterface (e.g. UChaosCompositeMovementMode) still
	// populate FMoverSimDrivenMontageData and trigger Montage_Stop on the game thread.
	{
		FMoverSimDrivenMontageData& MontageOutputData = OutputData.AdditionalOutputData.FindOrAddMutableDataByType<FMoverSimDrivenMontageData>();
		MontageOutputData.MontageStates.Reset();

		OutputData.SyncState.LayeredMoves.ForEachActiveMoveOfType<FLayeredMove_MontageStateProvider>(
			[&MontageOutputData, &TimeStep](const FLayeredMove_MontageStateProvider& Move)
			{
				Move.AppendMontageOutputEntry(MontageOutputData.MontageStates, TimeStep);
			});

		// Also collect from the active mode's source. UChaosLayeredMoveSource wraps an
		// FLayeredMoveBase directly rather than going through FLayeredMoveGroup, so it is
		// invisible to the ForEachActiveMoveOfType pass above.
		if (TStrongObjectPtr<UBaseMovementMode> CurrentModePtr = StateMachine.GetCurrentMode().Pin())
		{
			if (const UChaosCompositeMovementMode* CompositeMode = Cast<UChaosCompositeMovementMode>(CurrentModePtr.Get()))
			{
				if (CompositeMode->MoveSource)
				{
					CompositeMode->MoveSource->AppendMontageOutputEntry(MontageOutputData.MontageStates, TimeStep);
				}
			}
		}
	}

	CurrentSyncState = OutputData.SyncState;

	// Extract the events into the output data
	UpdateEvents(TimeStep, OutputData);

	// Set the output data to be valid if we've reached here
	OutputData.bIsValid = true;
}

void UChaosMoverSimulation::UpdateEvents(const FMoverTimeStep& TimeStep, UE::ChaosMover::FSimulationOutputData& OutputData)
{
	if (!UE::ChaosMover::CVars::bDisableResimDuplicateEventChecking)
	{
		if (const FChaosMoverSimulationDefaultInputs* SimInputs = LocalSimInput.FindDataByType<FChaosMoverSimulationDefaultInputs>())
		{
			// If networking is enabled then we can resimulate and potentially fire the same event
			// multiple times. To avoid this we maintain a list of processed events and if the same
			// event is generated at the same time then remove it
			if (SimInputs->AsyncNetworkPhysicsComponent && !SimInputs->AsyncNetworkPhysicsComponent->IsServer())
			{
				// Clear events that are outside of the resimulation window
				const int32 LatestReceivedStateServerFrame = SimInputs->AsyncNetworkPhysicsComponent->GetLatestReceivedStateFrame() + SimInputs->AsyncNetworkPhysicsComponent->GetNetworkPhysicsTickOffset();
				if (LatestReceivedStateServerFrame > 0)
				{
					ProcessedEvents.RemoveAllSwap([&LatestReceivedStateServerFrame](const TSharedPtr<FMoverSimulationEventData> Event) {
						return Event->Context.ServerFrame <= LatestReceivedStateServerFrame;
						});
				}

				if (TimeStep.bIsResimulating)
				{
					// Suppress re-emission of events the GT already received during the forward sim.
					// Events with bReEmitOnResim=true are never in ProcessedEvents and always pass through.

					Events.RemoveAll([this](const TSharedPtr<FMoverSimulationEventData> Event) {
						if (Event->bReEmitOnResim)
						{
							return false;
						}
						const int32 AllowedDiff = UE::ChaosMover::CVars::FrameDifferenceLeniencyForEventTimeComparison;
						for (TSharedPtr<FMoverSimulationEventData>& ProcessedEvent : ProcessedEvents)
						{
							if (const FMoverSimulationEventData* ProcessedEventPtr = ProcessedEvent.Get())
							{
								if (Event->IsEqual(*ProcessedEventPtr, AllowedDiff))
								{
									return true;
								}
							}

						}
						return false;
						});
				}
				else
				{
					// Record only events that should not re-emit on resim. Events with bReEmitOnResim=true
					// are intentionally excluded so they always pass the dedup check and re-emit on resim.
					for (TSharedPtr<FMoverSimulationEventData>& Event : Events)
					{
						if (!Event->bReEmitOnResim)
						{
							ProcessedEvents.Add(Event);
						}
					}
				}
			}
		}
	}

	// Extract events into output data
	OutputData.Events = Events;
	Events.Empty();
}

void UChaosMoverSimulation::TraceMoverData(const FMoverTimeStep& TimeStep, const UE::ChaosMover::FSimulationOutputData& OutputData)
{
	QUICK_SCOPE_CYCLE_COUNTER(ChaosMoverSimulation_TraceMoverData);

	// Send the latest worker thread data to CVD
#if WITH_CHAOS_VISUAL_DEBUGGER
	if (FChaosVisualDebuggerTrace::IsTracing())
	{
		// Trace time step info
		FChaosMoverTimeStepDebugData& TimeStepDebugData = DebugSimData.FindOrAddMutableDataByType<FChaosMoverTimeStepDebugData>();
		TimeStepDebugData.SetTimeStep(TimeStep);

		// Trace network physics info
		if (const FChaosMoverSimulationDefaultInputs* SimInputs = LocalSimInput.FindDataByType<FChaosMoverSimulationDefaultInputs>())
		{
			FNetworkPhysicsDebugData& NetworkPhysicsDebugData = DebugSimData.FindOrAddMutableDataByType<FNetworkPhysicsDebugData>();
			NetworkPhysicsDebugData.Set(SimInputs->AsyncNetworkPhysicsComponent);
		}

		// Trace simulation events
		{
			FChaosMoverSimulationEventsDebugData& EventsDebugData = DebugSimData.FindOrAddMutableDataByType<FChaosMoverSimulationEventsDebugData>();
			EventsDebugData.Events.Reset(OutputData.Events.Num());
			for (const TSharedPtr<FMoverSimulationEventData>& Event : OutputData.Events)
			{
				if (Event.IsValid())
				{
					FInstancedStruct& Slot = EventsDebugData.Events.AddDefaulted_GetRef();
					Slot.InitializeAs(Event->GetScriptStruct(), reinterpret_cast<const uint8*>(Event.Get()));
				}
			}
		}

		// Prepare tracing of LocalSimInput, InternalSimData and DebugSimData
		static FName NAME_LocalSimInput("LocalSimInput");
		static FName NAME_InternalSimData("InternalSimData");
		static FName NAME_DebugSimData("DebugSimData");
		static FName NAME_AdditionalOutputData("AdditionalOutputData");
		UE::MoverUtils::NamedDataCollections LocalSimDataCollections (
			{
				{ NAME_LocalSimInput, &LocalSimInput},
				{ NAME_InternalSimData, &InternalSimData},
				{ NAME_DebugSimData, &DebugSimData},
				{ NAME_AdditionalOutputData, &OutputData.AdditionalOutputData}
			});

		Chaos::FReadPhysicsObjectInterface_Internal Interface = Chaos::FPhysicsObjectInternalInterface::GetRead();
		const Chaos::FGeometryParticleHandle* ParticleHandle = PhysicsObject? Interface.GetParticle(PhysicsObject) : nullptr;
		int32 ParticleID = ParticleHandle ? ParticleHandle->UniqueIdx().Idx : INDEX_NONE;

		int32 SolverID = CVD_TRACE_GET_SOLVER_ID_FROM_WORLD(World);

		// Populate reflected mirrors of TSharedPtr move arrays so CVD can display active layered moves.
		OutputData.SyncState.LayeredMoves.PrepareCVDMirror();

		UE::MoverUtils::FMoverCVDRuntimeTrace::TraceMoverData(SolverID, ParticleID, &OutputData.LastUsedInputCmd, &OutputData.SyncState, &LocalSimDataCollections);

		// Clear debug sim data to avoid recording past debug data as current on the next frame
		DebugSimData.Empty();

		// Let the state machine know we have traced data. From this point on, the state machine no longer records instant movement effects directly into DebugSimData
		// but instead will log all its unprocessed instant movement effects into DebugSimData at the beginning of the next OnSimulationTick. This allows
		// us to capture both the instant movement effects that were added outside of SimulationTick AND any scheduled movement effects which are waiting for their
		// execution server frame to arrive
		StateMachine.OnEndTraceMoverData();
	}
#endif
}

void UChaosMoverSimulation::OnModifyContacts(const FMoverTimeStep& TimeStep, const UE::ChaosMover::FSimulationInputData& InputData, const UE::ChaosMover::FSimulationOutputData& OutputData, Chaos::FCollisionContactModifier& Modifier)
{
}

void UChaosMoverSimulation::AddEvent(TSharedPtr<FMoverSimulationEventData> Event)
{
	// Events are added to the event list for later extraction to game thread
	// We also allow the simulation to react to the event immediately
	Events.Add(Event);

	if (const FMoverSimulationEventData* EventData = Event.Get())
	{
		ProcessSimulationEvent(*EventData);
	}
}

const TArray<TSharedPtr<FMoverSimulationEventData>>& UChaosMoverSimulation::GetEvents() const
{
	return Events;
}

void UChaosMoverSimulation::ProcessSimulationEvent(const FMoverSimulationEventData& EventData)
{
	if (const FMovementModeChangedEventData* ModeChangedEventData = EventData.CastTo<FMovementModeChangedEventData>())
	{
		OnMovementModeChanged(*ModeChangedEventData);
	}
}

void UChaosMoverSimulation::OnMovementModeChanged(const FMovementModeChangedEventData& ModeChangedData)
{
	TStrongObjectPtr<UBaseMovementMode> PreviousModePtr = StateMachine.FindMovementMode(ModeChangedData.PreviousModeName).Pin();
	TStrongObjectPtr<UBaseMovementMode> NextModePtr = StateMachine.FindMovementMode(ModeChangedData.NewModeName).Pin();

	if (PreviousModePtr && NextModePtr)
	{
		if (IChaosCharacterConstraintMovementModeInterface* NextCharacterConstraintMode = Cast<IChaosCharacterConstraintMovementModeInterface>(NextModePtr.Get()))
		{
			if (CharacterGroundConstraintHandle)
			{
				// Character ground constraint modes currently assume moving a dynamic particle and using a character ground constraint
				// Revise if we start supporting moving a character kinematically
				if (!IsControlledParticleDynamic())
				{
					SetControlledParticleDynamic();
				}
			}
		}

		if (IChaosMovementActuationInterface* NextMovementActuationInterface = Cast<IChaosMovementActuationInterface>(NextModePtr.Get()))
		{
			// Reset the pathed movement state
			FChaosPathedMovementState* PresimPathedMovementState = CurrentSyncState.SyncStateCollection.FindMutableDataByType<FChaosPathedMovementState>();
			if (PresimPathedMovementState)
			{
				*PresimPathedMovementState = FChaosPathedMovementState();
			}

			if (ActuationConstraintHandle)
			{
				if (NextMovementActuationInterface->ShouldUseConstraint())
				{
					// Enable the constraint
					if (!ActuationConstraintHandle->IsEnabled())
					{
						EnableActuationConstraint();
					}
					// Set the controlled particle dynamic
					if (!IsControlledParticleDynamic())
					{
						SetControlledParticleDynamic();
					}
					// Apply movement actuation constraint settings
					// Bug: if we didn't also call SetSettings every frame in UChaosMoverSimulation::PostSimulationTickMovementActuation,
					// the settings wouldn't apply correctly from this call only if this mode is the first active mode (OnMovementModeChanged called on the very first OnSimulationTick)
					ActuationConstraintHandle->SetSettings(NextMovementActuationInterface->GetConstraintSettings());

					// If the mode was also a pathed mode, let's teleport to the position on the path
					// we're supposed to be at, using CurrentPathProgress
					IChaosPathedMovementModeInterface* NextPathedMovementMode = Cast<IChaosPathedMovementModeInterface>(NextMovementActuationInterface);
					if (NextPathedMovementMode)
					{	
						const float PathProgress = PresimPathedMovementState ? PresimPathedMovementState->LastChangePathProgress : 0.0f;
						// We resume the path at the reset progress
						// (if not 0, CurrentPathProgress was either not reset properly when the movement mode changed, or deliberately set to a different value)
						FTransform TargetLastFrame = NextPathedMovementMode->CalcTargetTransform(PathProgress, MovementBasisTransform);
						TeleportActuationTarget(TargetLastFrame);
					}
				}
				else
				{
					if (ActuationConstraintHandle->IsEnabled())
					{
						DisableActuationConstraint();
					}
					if (!IsControlledParticleKinematic())
					{
						SetControlledParticleKinematic();
					}
				}
			}
		}
	}
}

Chaos::FPBDRigidParticleHandle* UChaosMoverSimulation::GetControlledParticle() const
{
	if (PhysicsObject)
	{
		Chaos::FReadPhysicsObjectInterface_Internal Interface = Chaos::FPhysicsObjectInternalInterface::GetRead();
		return Interface.GetRigidParticle(PhysicsObject);
	}
	return nullptr;
}

Chaos::FCharacterGroundConstraintHandle* UChaosMoverSimulation::GetCharacterGroundConstraintHandle() const
{
	return CharacterGroundConstraintHandle;
}

Chaos::FPBDJointConstraintHandle* UChaosMoverSimulation::GetActuationConstraintHandle() const
{
	return ActuationConstraintHandle;
}

bool UChaosMoverSimulation::IsNonResimSimProxy() const
{
	const FChaosMoverSimulationDefaultInputs* SimInputs =
		LocalSimInput.FindDataByType<FChaosMoverSimulationDefaultInputs>();
	return SimInputs && SimInputs->AsyncNetworkPhysicsComponent &&
		SimInputs->AsyncNetworkPhysicsComponent->IsNonResimSimProxy();
}

void UChaosMoverSimulation::SetControlledParticleDynamic()
{
	if (Chaos::FPBDRigidParticleHandle* ControlledParticle = GetControlledParticle())
	{
		check(Solver);
		Chaos::FPBDRigidsEvolution& Evolution = *Solver->GetEvolution();
		Evolution.SetParticleObjectState(ControlledParticle, Chaos::EObjectStateType::Dynamic);
	}
}

void UChaosMoverSimulation::SetControlledParticleKinematic()
{
	if (Chaos::FPBDRigidParticleHandle* ControlledParticle = GetControlledParticle())
	{
		check(Solver);
		Chaos::FPBDRigidsEvolution& Evolution = *Solver->GetEvolution();
		Evolution.SetParticleObjectState(ControlledParticle, Chaos::EObjectStateType::Kinematic);

		if (const Chaos::TPBDRigidParticleHandleImp<Chaos::FReal, 3, true>* ControlledRigidParticle = ControlledParticle->CastToRigidParticle())
		{
			if (ControlledRigidParticle->UpdateKinematicFromSimulation() == true)
			{
				// Should we instead call SetUpdateKinematicFromSimulation on the GT when some of the modes may animate kinematically?
				UE_LOGF(LogChaosMover, Warning, "The updated component for %ls is not set to Update Kinematic from Simulation but is being moved kinematically", *GetClass()->GetName());
			}
		}
	}
}

bool UChaosMoverSimulation::IsControlledParticleDynamic() const
{
	const Chaos::FPBDRigidParticleHandle* ControlledParticle = GetControlledParticle();
	return ControlledParticle ? ControlledParticle->IsDynamic() : false;
}

bool UChaosMoverSimulation::IsControlledParticleKinematic() const
{
	const Chaos::FPBDRigidParticleHandle* ControlledParticle = GetControlledParticle();
	return ControlledParticle ? ControlledParticle->IsKinematic() : false;
}

void UChaosMoverSimulation::EnableCharacterGroundConstraint()
{
	if (CharacterGroundConstraintHandle)
	{
		if (CharacterGroundConstraintHandle->GetCharacterParticle())
		{
			CharacterGroundConstraintHandle->SetEnabled(true);
		}
	}
}

void UChaosMoverSimulation::DisableCharacterGroundConstraint()
{
	if (CharacterGroundConstraintHandle)
	{
		CharacterGroundConstraintHandle->SetEnabled(false);
	}
}

bool UChaosMoverSimulation::IsCharacterGroundConstraintEnabled() const
{
	return (CharacterGroundConstraintHandle && CharacterGroundConstraintHandle->IsEnabled());
}

void UChaosMoverSimulation::EnableActuationConstraint()
{
	if (ActuationConstraintHandle)
	{
		ActuationConstraintHandle->SetConstraintEnabled(true);
	}
}

void UChaosMoverSimulation::DisableActuationConstraint()
{

	if (ActuationConstraintHandle)
	{
		ActuationConstraintHandle->SetConstraintEnabled(false);
	}
}

bool UChaosMoverSimulation::IsActuationConstraintEnabled() const
{
	return (ActuationConstraintHandle && ActuationConstraintHandle->IsEnabled());
}

void UChaosMoverSimulation::K2_QueueInstantMovementEffect(const int32& EffectAsRawData)
{
	// This will never be called, the exec version below will be hit instead
	checkNoEntry();
}

DEFINE_FUNCTION(UChaosMoverSimulation::execK2_QueueInstantMovementEffect)
{
	Stack.StepCompiledIn<FStructProperty>(nullptr);
	void* EffectPtr = Stack.MostRecentPropertyAddress;
	FStructProperty* StructProp = CastField<FStructProperty>(Stack.MostRecentProperty);

	P_FINISH;

	P_NATIVE_BEGIN;

	const bool bHasValidStructProp = StructProp && StructProp->Struct && StructProp->Struct->IsChildOf(FInstantMovementEffect::StaticStruct());

	if (ensureMsgf((bHasValidStructProp && EffectPtr), TEXT("An invalid type (%s) was sent to a QueueInstantMovementEffect node. A struct derived from FInstantMovementEffect is required. No Movement Effect will be queued."),
		StructProp ? *GetNameSafe(StructProp->Struct) : *Stack.MostRecentProperty->GetClass()->GetName()))
	{
		// Could we steal this instead of cloning? (move semantics)
		FInstantMovementEffect* EffectAsBasePtr = reinterpret_cast<FInstantMovementEffect*>(EffectPtr);
		FInstantMovementEffect* ClonedMove = EffectAsBasePtr->Clone();

		P_THIS->QueueInstantMovementEffect_Internal(TSharedPtr<FInstantMovementEffect>(ClonedMove), /*bShouldRollBack =*/ true); // Worker Thread emitted effects can be rolled back because we expect resim to rerun all Worker Thread logic
	}

	P_NATIVE_END;
}

void UChaosMoverSimulation::QueueInstantMovementEffect_Internal(TSharedPtr<FInstantMovementEffect> InstantMovementEffect, bool bShouldRollBack)
{
	// We always use a server frame to schedule, never a time. This is because ChaosMover is always in async mode
    // (if we're not, UChaosMoverBackendComponent::InitializeComponent should have warned us to fix that)
	QueueInstantMovementEffect_Internal(FScheduledInstantMovementEffect{
									FMoverSchedulingInfo{
										/* ServerIssuanceTime  = */ FMoverTime{InternalServerFrame, 0.0},
										/* ServerExecutionTime = */ FMoverTime{InternalServerFrame, 0.0},
										/* bIsFixedDt          = */ true},
									InstantMovementEffect },
								bShouldRollBack);
}

void UChaosMoverSimulation::QueueInstantMovementEffect_Internal(const FScheduledInstantMovementEffect& ScheduledInstantMovementEffect, bool bShouldRollBack)
{
	StateMachine.QueueInstantMovementEffect(FChaosScheduledInstantMovementEffect{ bShouldRollBack, ScheduledInstantMovementEffect });
}

void UChaosMoverSimulation::K2_QueueLayeredMove(const int32& MoveAsRawData)
{
	// This will never be called, the exec version below will be hit instead
	checkNoEntry();
}

DEFINE_FUNCTION(UChaosMoverSimulation::execK2_QueueLayeredMove)
{
	Stack.StepCompiledIn<FStructProperty>(nullptr);
	void* MovePtr = Stack.MostRecentPropertyAddress;
	FStructProperty* StructProp = CastField<FStructProperty>(Stack.MostRecentProperty);

	P_FINISH;

	P_NATIVE_BEGIN;

	const bool bHasValidStructProp = StructProp && StructProp->Struct && StructProp->Struct->IsChildOf(FLayeredMoveBase::StaticStruct());

	if (ensureMsgf((bHasValidStructProp && MovePtr), TEXT("An invalid type (%s) was sent to a QueueLayeredMove node. A struct derived from FLayeredMoveBase is required. No Move will be queued."),
		StructProp ? *GetNameSafe(StructProp->Struct) : *Stack.MostRecentProperty->GetClass()->GetName()))
	{
		// Could we steal this instead of cloning? (move semantics)
		FLayeredMoveBase* MoveAsBasePtr = reinterpret_cast<FLayeredMoveBase*>(MovePtr);
		FLayeredMoveBase* ClonedMove = MoveAsBasePtr->Clone();

		P_THIS->QueueLayeredMove_Internal(TSharedPtr<FLayeredMoveBase>(ClonedMove), /*bShouldRollBack =*/ true); // Worker Thread moves can be rolled back because we expect resim to rerun all Worker Thread logic
	}

	P_NATIVE_END;
}

void UChaosMoverSimulation::QueueLayeredMove_Internal(TSharedPtr<FLayeredMoveBase> LayeredMove, bool bShouldRollBack)
{
	// We always use a server frame to schedule, never a time. This is because ChaosMover is always in async mode
	// (if we're not, UChaosMoverBackendComponent::InitializeComponent should have warned us to fix that)
	QueueLayeredMove_Internal(FScheduledLayeredMove{
						FMoverSchedulingInfo{
							/* ServerIssuanceTime  = */ FMoverTime{InternalServerFrame, 0.0},
							/* ServerExecutionTime = */ FMoverTime{InternalServerFrame, 0.0},
							/* bIsFixedDt          = */ true},
						LayeredMove },
					 bShouldRollBack);
}

void UChaosMoverSimulation::QueueLayeredMove_Internal(const FScheduledLayeredMove& ScheduledLayeredMove, bool bShouldRollBack)
{
	StateMachine.QueueLayeredMove(FChaosScheduledLayeredMove{ bShouldRollBack, ScheduledLayeredMove });
}

void UChaosMoverSimulation::QueueLayeredMoveInstance_Internal(TSubclassOf<ULayeredMoveLogic> MoveLogicClass, TSharedRef<FLayeredMoveInstancedData> Data, bool bShouldRollBack)
{
	ULayeredMoveLogic* FoundLogic = nullptr;
	for (const TObjectPtr<ULayeredMoveLogic>& RegisteredMove : StateMachine.GetRegisteredMoves())
	{
		if (RegisteredMove && RegisteredMove->GetClass() == MoveLogicClass)
		{
			FoundLogic = RegisteredMove;
			break;
		}
	}

	if (!ensure(FoundLogic))
	{
		return;
	}

	TSharedPtr<FLayeredMoveInstance> MoveInst = MakeShared<FLayeredMoveInstance>(Data, FoundLogic);
	FChaosScheduledLayeredMoveInstance Scheduled;
	Scheduled.bShouldRollBack  = bShouldRollBack;
	Scheduled.SchedulingInfo   = FMoverSchedulingInfo(
		/* ServerIssuanceTime  = */ FMoverTime{InternalServerFrame, 0.0},
		/* ServerExecutionTime = */ FMoverTime{InternalServerFrame, 0.0},
		/* bIsFixedDt          = */ true);
	Scheduled.Move = MoveInst;
	StateMachine.QueueLayeredMoveInstance(Scheduled);
}

void UChaosMoverSimulation::SetRegisteredMoves(const TArray<TObjectPtr<ULayeredMoveLogic>>& Moves)
{
	StateMachine.SetRegisteredMoves(Moves);
}

DEFINE_FUNCTION(UChaosMoverSimulation::execK2_QueueMovementModifier)
{
	Stack.StepCompiledIn<FStructProperty>(nullptr);
	void* MovePtr = Stack.MostRecentPropertyAddress;
	FStructProperty* StructProp = CastField<FStructProperty>(Stack.MostRecentProperty);

	P_FINISH;

	P_NATIVE_BEGIN;

	const bool bHasValidStructProp = StructProp && StructProp->Struct && StructProp->Struct->IsChildOf(FMovementModifierBase::StaticStruct());

	if (ensureMsgf((bHasValidStructProp && MovePtr), TEXT("An invalid type (%s) was sent to a QueueMovementModifier node. A struct derived from FMovementModifierBase is required. No modifier will be queued."),
		StructProp ? *GetNameSafe(StructProp->Struct) : *Stack.MostRecentProperty->GetClass()->GetName()))
	{
		// Could we steal this instead of cloning? (move semantics)
		FMovementModifierBase* MoveAsBasePtr = reinterpret_cast<FMovementModifierBase*>(MovePtr);
		FMovementModifierBase* ClonedMove = MoveAsBasePtr->Clone();

		FMovementModifierHandle ModifierID = P_THIS->QueueMovementModifier(TSharedPtr<FMovementModifierBase>(ClonedMove));
		*static_cast<FMovementModifierHandle*>(RESULT_PARAM) = ModifierID;
	}

	P_NATIVE_END;
}

FMovementModifierHandle UChaosMoverSimulation::QueueMovementModifier(TSharedPtr<FMovementModifierBase> Modifier)
{
	return StateMachine.QueueMovementModifier(Modifier);
}

void UChaosMoverSimulation::CancelModifierFromHandle(FMovementModifierHandle ModifierHandle)
{
	StateMachine.CancelModifierFromHandle(ModifierHandle);
}

bool UChaosMoverSimulation::HasGameplayTag(FGameplayTag TagToFind, bool bExactMatch) const
{
	// Check active Movement Mode
	if (TStrongObjectPtr<const UBaseMovementMode> ActiveMovementMode = FindMovementModeByName(CurrentSyncState.MovementMode).Pin())
	{
		if (ActiveMovementMode->HasGameplayTag(TagToFind, bExactMatch))
		{
			return true;
		}
	}

	// Search Movement Modifiers
	for (auto ModifierFromSyncStateIt = CurrentSyncState.MovementModifiers.GetActiveModifiersIterator(); ModifierFromSyncStateIt; ++ModifierFromSyncStateIt)
	{
		if (const TSharedPtr<FMovementModifierBase> ModifierFromSyncState = *ModifierFromSyncStateIt)
		{
			if (ModifierFromSyncState.IsValid() && ModifierFromSyncState->HasGameplayTag(TagToFind, bExactMatch))
			{
				return true;
			}
		}
	}

	// Search Layered Moves
	for (const TSharedPtr<FLayeredMoveBase>& LayeredMove : CurrentSyncState.LayeredMoves.GetActiveMoves())
	{
		if (LayeredMove->HasGameplayTag(TagToFind, bExactMatch))
		{
			return true;
		}
	}

	return false;
}

const FMovementModifierBase* UChaosMoverSimulation::FindMovementModifierByType(const UScriptStruct* DataStructType) const
{
	// Check active modifiers for modifier handle
	for (auto ActiveModifierFromSyncStateIt = CurrentSyncState.MovementModifiers.GetActiveModifiersIterator(); ActiveModifierFromSyncStateIt; ++ActiveModifierFromSyncStateIt)
	{
		const TSharedPtr<FMovementModifierBase> ActiveModifierFromSyncState = *ActiveModifierFromSyncStateIt;

		if (ActiveModifierFromSyncState->GetScriptStruct()->IsChildOf(DataStructType))
		{
			return ActiveModifierFromSyncState.Get();
		}
	}

	return StateMachine.FindQueuedModifierByType(DataStructType);
}

void UChaosMoverSimulation::AttemptTeleport(const FMoverTimeStep& TimeStep, const FTransform& TargetTransform, bool bUseActorRotation, FMoverSyncState& OutputState)
{
	const FMoverDefaultSyncState* DefaultSyncState = OutputState.SyncStateCollection.FindDataByType<FMoverDefaultSyncState>();

	FTransform FromTransform = DefaultSyncState ? DefaultSyncState->GetTransform_WorldSpace() : FTransform::Identity;
	const FVector& ToLocation = TargetTransform.GetLocation();
	FQuat ToRotation = (DefaultSyncState && bUseActorRotation) ? FromTransform.GetRotation() : TargetTransform.GetRotation();
	FTransform TeleportTransform(ToRotation, TargetTransform.GetLocation());
	if (CanTeleport(TeleportTransform, false, OutputState))
	{
		// Then we need to teleport
		Teleport(TeleportTransform, TimeStep, OutputState);

		AddEvent(MakeShared<FTeleportSucceededEventData>(TimeStep, FromTransform.GetLocation(), FromTransform.GetRotation(), ToLocation, ToRotation));

#if !UE_BUILD_SHIPPING
		const FChaosMoverSimulationDefaultInputs* SimInputs = LocalSimInput.FindDataByType<FChaosMoverSimulationDefaultInputs>();
		if (SimInputs && SimInputs->OwningActor)
		{
			UE_LOGF(LogChaosMover, Log, "%ls: Actor %ls teleported to %ls, %ls, teleport frame = %d)",
				*ToString(SimInputs->OwningActor->GetNetMode()), *GetNameSafe(SimInputs->OwningActor),
				*ToLocation.ToString(), *FRotator(ToRotation).ToString(), TimeStep.ServerFrame);
		}
#endif // !UE_BUILD_SHIPPING
	}
	else
	{
		ETeleportFailureReason FailureReason = ETeleportFailureReason::Reason_NotAvailable;
		AddEvent(MakeShared<FTeleportFailedEventData>(TimeStep, FromTransform.GetLocation(), FromTransform.GetRotation(), ToLocation, ToRotation, FailureReason));

#if !UE_BUILD_SHIPPING
		const FChaosMoverSimulationDefaultInputs* SimInputs = LocalSimInput.FindDataByType<FChaosMoverSimulationDefaultInputs>();
		if (SimInputs && SimInputs->OwningActor)
		{
			UE_LOGF(LogChaosMover, Log, "%ls: Actor %ls could NOT teleport to %ls, %ls, teleport frame = %d. Reason = %ls",
				*ToString(SimInputs->OwningActor->GetNetMode()), *GetNameSafe(SimInputs->OwningActor),
				*ToLocation.ToString(), *FRotator(ToRotation).ToString(), TimeStep.ServerFrame,
				*StaticEnum<ETeleportFailureReason>()->GetNameStringByValue(static_cast<int64>(FailureReason)));
		}
#endif // !UE_BUILD_SHIPPING
	}
}

void UChaosMoverSimulation::ApplyAction_Internal(const TInstancedStruct<FNetworkPhysicsActionPayload>& ActionInstance)
{
	if (const FChaosMoverInstantMovementEffectAction* Action = ActionInstance.GetPtr<FChaosMoverInstantMovementEffectAction>())
	{
		if (ensure(Action->Effect.IsValid()))
		{
			TSharedPtr<FInstantMovementEffect> Effect(Action->Effect.Get().Clone());
			QueueInstantMovementEffect_Internal(MoveTemp(Effect), /* bShouldRollBack = */ true);
		}
		return;
	}

	if (const FChaosMoverLayeredMoveAction* Action = ActionInstance.GetPtr<FChaosMoverLayeredMoveAction>())
	{
		if (ensure(Action->Move.IsValid()))
		{
			TSharedPtr<FLayeredMoveBase> Move(Action->Move.Get().Clone());
			QueueLayeredMove_Internal(MoveTemp(Move), /* bShouldRollBack = */ true);
		}
		return;
	}

	if (const FChaosMoverLayeredMoveInstanceAction* Action = ActionInstance.GetPtr<FChaosMoverLayeredMoveInstanceAction>())
	{
		if (ensure(Action->MoveLogicClass && Action->InstancedData.IsValid()))
		{
			TSharedRef<FLayeredMoveInstancedData> Data(Action->InstancedData.Get().Clone());
			QueueLayeredMoveInstance_Internal(Action->MoveLogicClass, MoveTemp(Data), /* bShouldRollBack = */ true);
		}
		return;
	}
}
