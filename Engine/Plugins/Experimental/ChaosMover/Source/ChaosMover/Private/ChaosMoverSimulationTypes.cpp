// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosMover/ChaosMoverSimulationTypes.h"

#include "Chaos/PhysicsObjectInternalInterface.h"
#include "LayeredMoveBase.h"
#include "ChaosMover/ChaosMoverSimulation.h"
#include "ChaosMover/ChaosMoverLog.h"
#include "ChaosMover/ChaosMoverConsoleVariables.h"
#include "MoveLibrary/BasedMovementUtils.h"
#include "Serialization/ArchiveCrc32.h"
#include "Physics/NetworkPhysicsComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosMoverSimulationTypes)

namespace UE::ChaosMover
{
	namespace Blackboard
	{
		const FName GroundDynamicsInfo = TEXT("GroundDynamicsInfo");
	}

	FGroundDynamicsInfo::FGroundDynamicsInfo()
		: LinearVelocity(FVector::ZeroVector)
		, AngularVelocityDegrees(FVector::ZeroVector)
		, bIsMoving(0)
		, bIsDynamic(0)
		, bIsGravityEnabled(0)
	{
	}

	FGroundDynamicsInfo::FGroundDynamicsInfo(const FFloorCheckResult& InFloorResult)
		: FGroundDynamicsInfo()
	{
		if (Chaos::FPhysicsObjectHandle PhysicsObject = InFloorResult.HitResult.PhysicsObject)
		{
			Chaos::FReadPhysicsObjectInterface_Internal Interface = Chaos::FPhysicsObjectInternalInterface::GetRead();
			if (const Chaos::FPBDRigidParticleHandle* ParticleHandle = Interface.GetRigidParticle(PhysicsObject))
			{
				bIsDynamic = ParticleHandle->IsDynamic();

				FVector Offset = InFloorResult.HitResult.ImpactPoint - ParticleHandle->GetTransformXRCom().GetLocation();
				Offset = FVector::VectorPlaneProject(Offset, InFloorResult.HitResult.ImpactNormal);

				LinearVelocity = ParticleHandle->GetV() + ParticleHandle->GetW().Cross(Offset);
				AngularVelocityDegrees = FMath::RadiansToDegrees(ParticleHandle->GetW());

				bIsMoving = (LinearVelocity.SizeSquared() > UE_SMALL_NUMBER) || (AngularVelocityDegrees.SizeSquared() > UE_SMALL_NUMBER);
				bIsGravityEnabled = ParticleHandle->GravityEnabled();
			}
		}
	}
}

void FChaosMoverSimulationDefaultInputs::Reset()
{
	CollisionResponseParams = FCollisionResponseParams();
	CollisionQueryParams = FCollisionQueryParams();
	UpDir = FVector::UpVector;
	Gravity = -980.7 * UpDir;
	PhysicsObjectGravity = 0.0f;
	PawnCollisionHalfHeight = 40.0f;
	PawnCollisionRadius = 30.0f;
	PhysicsObject = nullptr;
	OwningActor = nullptr;
	World = nullptr;
	AsyncNetworkPhysicsComponent = nullptr;
	CollisionChannel = ECC_Pawn;
}

FMoverDataCollection& UE::ChaosMover::GetDebugSimData(UChaosMoverSimulation* Simulation)
{
	check(Simulation);
	return Simulation->GetDebugSimData();
}

bool FChaosMovementBasis::NetSerialize(FArchive & Ar, UPackageMap * Map, bool& bOutSuccess)
{
	//bool bHasBasisLocation = BasisLocation.IsNearlyZero();
	Ar << BasisLocation;
	Ar << BasisRotation;
	bOutSuccess = true;
	return true;
}

FMoverDataStructBase* FChaosMovementBasis::Clone() const
{
	return new FChaosMovementBasis(*this);
}

UScriptStruct* FChaosMovementBasis::GetScriptStruct() const
{
	return StaticStruct();
}

void FChaosMovementBasis::ToString(FAnsiStringBuilderBase& Out) const
{
	Out.Appendf("Movement Basis: X = %s, R = %s (as rotator = %s)\n", *BasisLocation.ToString(), *BasisRotation.ToString(), *FRotator(BasisRotation).ToString());
}

bool FChaosMovementBasis::ShouldReconcile(const FMoverDataStructBase& AuthorityState) const
{
	const FChaosMovementBasis& TypedAuthority = static_cast<const FChaosMovementBasis&>(AuthorityState);
	static float LocationTolerance = 1e-3f;
	static float AngularTolerance = 1e-3f;
	return !BasisLocation.Equals(TypedAuthority.BasisLocation, LocationTolerance)
		|| !FMath::IsNearlyZero(BasisRotation.AngularDistance(TypedAuthority.BasisRotation), AngularTolerance);
}

void FChaosMovementBasis::Interpolate(const FMoverDataStructBase& From, const FMoverDataStructBase& To, float Pct)
{
	*this = static_cast<const FChaosMovementBasis&>((Pct < .5f) ? From : To);
}

void FChaosMovementBasis::Merge(const FMoverDataStructBase& From)
{
	
}

void FChaosMoverTimeStepDebugData::SetTimeStep(const FMoverTimeStep& InTimeStep)
{
	TimeStep = InTimeStep;
	bIsResimulating = InTimeStep.bIsResimulating;
	bIsFirstResimFrame = InTimeStep.bIsFirstResimFrame;
}

FMoverDataStructBase* FChaosMoverTimeStepDebugData::Clone() const
{
	return new FChaosMoverTimeStepDebugData(*this);
}

UScriptStruct* FChaosMoverTimeStepDebugData::GetScriptStruct() const
{
	return StaticStruct();
}

FMoverDataStructBase* FNetworkPhysicsDebugData::Clone() const
{
	return new FNetworkPhysicsDebugData(*this);
}

void FNetworkPhysicsDebugData::Set(const FAsyncNetworkPhysicsComponent* NetworkPhysicsComponent)
{
	if (NetworkPhysicsComponent)
	{
		bIsLocallyControlled = NetworkPhysicsComponent->IsLocallyControlled();
		PhysicsReplicationMode = NetworkPhysicsComponent->GetPhysicsReplicationMode();
		NetworkPhysicsTickOffset = NetworkPhysicsComponent->GetNetworkPhysicsTickOffset();
		LatestReceivedStateFrame = NetworkPhysicsComponent->GetLatestReceivedStateFrame();
		ForwardPredictionTime = NetworkPhysicsComponent->GetForwardPredictionTime();
		CurrentSimProxyInputDecayAtRuntime = NetworkPhysicsComponent->GetCurrentSimProxyInputDecayAtRuntime();
		CurrentInputDecay = NetworkPhysicsComponent->GetCurrentInputDecay();
	}
}

UScriptStruct* FNetworkPhysicsDebugData::GetScriptStruct() const
{
	return StaticStruct();
}

void FChaosWaterResultData::ToString(FAnsiStringBuilderBase& Out) const
{
	Super::ToString(Out);

	Out.Appendf("bSwimmableVolume: %i | ", WaterResult.bSwimmableVolume);
	Out.Appendf("ImmersionDepth: %.2f | ", WaterResult.WaterSplineData.ImmersionDepth);
	Out.Appendf("ImmersionPercent: %.2f | ", WaterResult.WaterSplineData.ImmersionPercent);
	Out.Appendf("WaterDepth: %.2f/n", WaterResult.WaterSplineData.WaterDepth);
	Out.Appendf("HitResult: %s/n", *WaterResult.HitResult.ToString());
}

bool FChaosWaterResultData::ShouldReconcile(const FMoverDataStructBase& AuthorityState) const
{
	return false;
}

void FChaosWaterResultData::Interpolate(const FMoverDataStructBase& From, const FMoverDataStructBase& To, float Pct)
{
	*this = static_cast<const FChaosWaterResultData&>((Pct < 0.5f) ? From : To);
}

void FChaosWaterResultData::Merge(const FMoverDataStructBase& From)
{
}

void FChaosWaterResultData::Decay(float DecayAmount)
{
}

void FChaosNetInstantMovementEffectsQueue::Add(const FScheduledInstantMovementEffect& InScheduledEffect, bool bShouldRollBack, uint8 UniqueID)
{
	TCheckedObjPtr<FInstantMovementEffect> InstantMovementEffect = InScheduledEffect.Effect.Get();
	if (ensure(InstantMovementEffect.IsValid() && InstantMovementEffect->GetScriptStruct()))
	{
		FChaosNetInstantMovementEffect& AddedInstancedEffect = Effects.AddDefaulted_GetRef();
		AddedInstancedEffect.ExecutionServerFrame = InScheduledEffect.SchedulingInfo.ServerExecutionTime.FrameCount;
		AddedInstancedEffect.IssuanceServerFrame = InScheduledEffect.SchedulingInfo.ServerIssuanceTime.FrameCount;
		AddedInstancedEffect.UniqueID = UniqueID;
		AddedInstancedEffect.bShouldRollBack = bShouldRollBack;
		AddedInstancedEffect.Effect.InitializeAsScriptStruct(InstantMovementEffect->GetScriptStruct(), (const uint8*)InScheduledEffect.Effect.Get());
	}
}

FChaosNetInstantMovementEffectsQueue& FChaosNetInstantMovementEffectsQueue::operator=(const TArray<FChaosScheduledInstantMovementEffect>& ScheduledInstantEffects)
{
	Effects.Empty(/*Slack =*/ ScheduledInstantEffects.Num());
	for (const FChaosScheduledInstantMovementEffect& ScheduledInstantEffect : ScheduledInstantEffects)
	{
		Add(ScheduledInstantEffect.ScheduledEffect, ScheduledInstantEffect.bShouldRollBack, 0xFF/* This is used for debugging, unique ID doesn't apply */);
	}
	return *this;
}

bool FChaosNetInstantMovementEffectsQueue::NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess)
{
	bOutSuccess = true;

	uint8 NumEffectsToSerialize;
	if (Ar.IsSaving())
	{
		ensureMsgf(Effects.Num() <= 255, TEXT("More than 255 effects to serialize, some effects will be dropped"));
		NumEffectsToSerialize = FMath::Min(Effects.Num(), 255);
	}
	Ar << NumEffectsToSerialize;

	if (Ar.IsLoading())
	{
		Effects.SetNumZeroed(NumEffectsToSerialize);
	}

	for (int32 EffectIndex = 0; EffectIndex < NumEffectsToSerialize && !Ar.IsError(); ++EffectIndex)
	{
		FChaosNetInstantMovementEffect& ScheduledEffect = Effects[EffectIndex];
		FInstantMovementEffect* Effect = ScheduledEffect.Effect.IsValid() ? &ScheduledEffect.Effect.GetMutable() : nullptr;
		TCheckedObjPtr<UScriptStruct> ScriptStruct = Ar.IsLoading() ? FInstantMovementEffect::StaticStruct() : (Effect ? Effect->GetScriptStruct() : nullptr);

		Ar << ScriptStruct;

		if (ScriptStruct.IsValid())
		{
			// Restrict replication to derived classes of FInstantMovementEffect for security reasons:
			// If FInstantMovementEffectsQueue is replicated through a Server RPC, we need to prevent clients from sending us
			// arbitrary ScriptStructs due to the allocation/reliance on GetCppStructOps below which could trigger a server crash
			// for invalid structs. All provided sources are direct children of FInstantMovementEffect and we never expect to have deep hierarchies
			// so this should not be too costly
			bool bIsDerivedFromBase = false;
			UStruct* CurrentSuperStruct = ScriptStruct->GetSuperStruct();
			while (CurrentSuperStruct)
			{
				if (CurrentSuperStruct == FInstantMovementEffect::StaticStruct())
				{
					bIsDerivedFromBase = true;
					break;
				}
				CurrentSuperStruct = CurrentSuperStruct->GetSuperStruct();
			}

			if (bIsDerivedFromBase)
			{
				Ar << ScheduledEffect.ExecutionServerFrame;
				Ar << ScheduledEffect.IssuanceServerFrame;
				Ar << ScheduledEffect.UniqueID;
				
				if (Ar.IsLoading())
				{
					ScheduledEffect.Effect.InitializeAsScriptStruct(ScriptStruct.Get());
					ScheduledEffect.bShouldRollBack = true; // Networked effects should always roll back, only game thread issued ones don't
					// In the loading case it's only now that Effect can be set to a valid initialized struct
					if (ensure(ScheduledEffect.Effect.IsValid()))
					{
						Effect = &ScheduledEffect.Effect.GetMutable();
					}
				}

				if (ensure(Effect))
				{
					Effect->NetSerialize(Ar);
				}
				else
				{
					UE_LOGF(LogMover, Error, "FInstantMovementEffectsQueue::NetSerialize: Failed to serialize effect");
					Ar.SetError();
					bOutSuccess = false;
					return false;
				}
			}
			else
			{
				UE_LOGF(LogMover, Error, "FInstantMovementEffectsQueue::NetSerialize: ScriptStruct not derived from FInstantMovementEffect attempted to serialize.");
				Ar.SetError();
				bOutSuccess = false;
				return false;
			}
		}
		else if (ScriptStruct.IsError())
		{
			UE_LOGF(LogMover, Error, "FInstantMovementEffectsQueue::NetSerialize: Invalid ScriptStruct serialized.");
			Ar.SetError();
			bOutSuccess = false;
			return false;
		}
	}

	return true;
}

FMoverDataStructBase* FChaosNetInstantMovementEffectsQueue::Clone() const
{
	FChaosNetInstantMovementEffectsQueue* CopyPtr = new FChaosNetInstantMovementEffectsQueue(*this);
	return CopyPtr;
}

void FChaosNetInstantMovementEffectsQueue::ToString(FAnsiStringBuilderBase& Out) const
{
	Out.Appendf("Instant Movement Effects Queue -------------------------------------------------\n");
	for (int32 EffectIndex = 0; EffectIndex < Effects.Num(); ++EffectIndex)
	{
		const FChaosNetInstantMovementEffect& ScheduledEffect = Effects[EffectIndex];
		Out.Appendf("Effect Index %d: ServerFrame %d, %s",
			EffectIndex,
			ScheduledEffect.ExecutionServerFrame,
			ScheduledEffect.Effect.IsValid() ? TCHAR_TO_ANSI(*ScheduledEffect.Effect.Get().ToSimpleString()) : "INVALID INSTANCED STRUCT EFFECT");
		Out.AppendChar('\n');
	}
	Out.Appendf("--------------------------------------------------------------------------------\n");
}

void FChaosNetInstantMovementEffectsQueue::Interpolate(const FMoverDataStructBase& From, const FMoverDataStructBase& To, float Pct)
{
	const FChaosNetInstantMovementEffectsQueue& TypedFrom = static_cast<const FChaosNetInstantMovementEffectsQueue&>(From);
	const FChaosNetInstantMovementEffectsQueue& TypedTo = static_cast<const FChaosNetInstantMovementEffectsQueue&>(To);

	Effects = (Pct < 0.5f) ? TypedFrom.Effects : TypedTo.Effects;
}

UScriptStruct* FChaosNetInstantMovementEffectsQueue::GetScriptStruct() const
{
	return StaticStruct();
}

bool FChaosNetInstantMovementEffectsQueue::ShouldReconcile(const FMoverDataStructBase& AuthorityState) const
{
	const FChaosNetInstantMovementEffectsQueue& TypedAuthority = static_cast<const FChaosNetInstantMovementEffectsQueue&>(AuthorityState);
	if (Effects.Num() != TypedAuthority.Effects.Num())
	{
		return true;
	}

	for (int EffectIndex = 0; EffectIndex < Effects.Num(); ++EffectIndex)
	{
		const FChaosNetInstantMovementEffect& ScheduledEffect = Effects[EffectIndex];
		const FChaosNetInstantMovementEffect& AuthorityScheduledEffect = TypedAuthority.Effects[EffectIndex];

		if (ScheduledEffect.ExecutionServerFrame != AuthorityScheduledEffect.ExecutionServerFrame)
		{
			return true;
		}

		if (ScheduledEffect.Effect.IsValid() != AuthorityScheduledEffect.Effect.IsValid())
		{
			return true;
		}

		if (ScheduledEffect.Effect.IsValid())
		{
			if (ScheduledEffect.Effect.GetScriptStruct() != AuthorityScheduledEffect.Effect.GetScriptStruct())
			{
				return true;
			}

			// This allows us to skip implementing operator== for all movement effects
			FArchiveCrc32 Crc1;
			(const_cast<FChaosNetInstantMovementEffect&>(ScheduledEffect)).Effect.GetMutable().NetSerialize(Crc1);
			FArchiveCrc32 Crc2;
			(const_cast<FChaosNetInstantMovementEffect&>(AuthorityScheduledEffect)).Effect.GetMutable().NetSerialize(Crc2);
			bool bAreEqual = (Crc1.GetCrc() == Crc2.GetCrc());
			if (!bAreEqual)
			{
				return true;
			}
		}
	}

	return false;
}

void FChaosNetInstantMovementEffectsQueue::Merge(const FMoverDataStructBase& From)
{
	const FChaosNetInstantMovementEffectsQueue& TypedFrom = static_cast<const FChaosNetInstantMovementEffectsQueue&>(From);
	Effects.Append(TypedFrom.Effects);
}

void FChaosNetInstantMovementEffectsQueue::Decay(float DecayAmount)
{

}

void FChaosNetLayeredMovesQueue::Add(const FScheduledLayeredMove& InScheduledLayeredMove, bool bShouldRollBack, uint8 UniqueID)
{
	TCheckedObjPtr<FLayeredMoveBase> LayeredMove = InScheduledLayeredMove.Move.Get();
	if (ensure(LayeredMove.IsValid() && LayeredMove->GetScriptStruct()))
	{
		FChaosNetLayeredMove& AddedInstancedEffect = Moves.AddDefaulted_GetRef();
		AddedInstancedEffect.ExecutionServerFrame = InScheduledLayeredMove.SchedulingInfo.ServerExecutionTime.FrameCount;
		AddedInstancedEffect.IssuanceServerFrame = InScheduledLayeredMove.SchedulingInfo.ServerIssuanceTime.FrameCount;
		AddedInstancedEffect.UniqueID = UniqueID;
		AddedInstancedEffect.bShouldRollBack = bShouldRollBack;
		AddedInstancedEffect.Move.InitializeAsScriptStruct(LayeredMove->GetScriptStruct(), (const uint8*)InScheduledLayeredMove.Move.Get());
	}
}

// This function is used for debugging (sending to CVD)
FChaosNetLayeredMovesQueue& FChaosNetLayeredMovesQueue::operator=(const TArray<FChaosScheduledLayeredMove>& ScheduledMoves)
{
	Moves.Empty(/*Slack =*/ ScheduledMoves.Num());
	for (const FChaosScheduledLayeredMove& ScheduledMove : ScheduledMoves)
	{
		Add(ScheduledMove.ScheduledLayeredMove, ScheduledMove.bShouldRollBack, 0xFF/* This is used for debugging, unique ID doesn't apply */);
	}
	return *this;
}

FMoverDataStructBase* FChaosNetLayeredMovesQueue::Clone() const
{
	FChaosNetLayeredMovesQueue* CopyPtr = new FChaosNetLayeredMovesQueue(*this);
	return CopyPtr;
}

bool FChaosNetLayeredMovesQueue::NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess)
{
	bOutSuccess = true;

	uint8 NumMovesToSerialize = 0;
	if (Ar.IsSaving())
	{
		ensureMsgf(Moves.Num() <= 255, TEXT("More than 255 moves to serialize, some moves will be dropped"));
		NumMovesToSerialize = FMath::Min(Moves.Num(), 255);
	}
	Ar << NumMovesToSerialize;

	if (Ar.IsLoading())
	{
		Moves.SetNumZeroed(NumMovesToSerialize);
	}

	for (int32 MoveIndex = 0; MoveIndex < NumMovesToSerialize && !Ar.IsError(); ++MoveIndex)
	{
		FChaosNetLayeredMove& ScheduledMove = Moves[MoveIndex];
		FLayeredMoveBase* Move = ScheduledMove.Move.IsValid() ? &ScheduledMove.Move.GetMutable() : nullptr;
		TCheckedObjPtr<UScriptStruct> ScriptStruct = Ar.IsLoading() ? FLayeredMoveBase::StaticStruct() : (Move ? Move->GetScriptStruct() : nullptr);

		Ar << ScriptStruct;

		if (ScriptStruct.IsValid())
		{
			// Restrict replication to derived classes of FLayeredMoveBase for security reasons:
			// If FLayeredMovesQueue is replicated through a Server RPC, we need to prevent clients from sending us
			// arbitrary ScriptStructs due to the allocation/reliance on GetCppStructOps below which could trigger a server crash
			// for invalid structs. All provided sources are direct children of FLayeredMoveBase and we never expect to have deep hierarchies
			// so this should not be too costly
			bool bIsDerivedFromBase = false;
			UStruct* CurrentSuperStruct = ScriptStruct->GetSuperStruct();
			while (CurrentSuperStruct)
			{
				if (CurrentSuperStruct == FLayeredMoveBase::StaticStruct())
				{
					bIsDerivedFromBase = true;
					break;
				}
				CurrentSuperStruct = CurrentSuperStruct->GetSuperStruct();
			}

			if (bIsDerivedFromBase)
			{
				Ar << ScheduledMove.ExecutionServerFrame;
				Ar << ScheduledMove.IssuanceServerFrame;
				Ar << ScheduledMove.UniqueID;

				if (Ar.IsLoading())
				{
					ScheduledMove.Move.InitializeAsScriptStruct(ScriptStruct.Get());
					ScheduledMove.bShouldRollBack = true; // Networked moves should always roll back, only game thread issued ones don't
					// In the loading case it's only now that Move can be set to a valid initialized struct
					if (ensure(ScheduledMove.Move.IsValid()))
					{
						Move = &ScheduledMove.Move.GetMutable();
					}
				}

				if (ensure(Move))
				{
					Move->NetSerialize(Ar);
				}
				else
				{
					UE_LOGF(LogMover, Error, "FChaosNetLayeredMovesQueue::NetSerialize: Failed to serialize move");
					Ar.SetError();
					bOutSuccess = false;
					return false;
				}
			}
			else
			{
				UE_LOGF(LogMover, Error, "FChaosNetLayeredMovesQueue::NetSerialize: ScriptStruct not derived from FLayeredMoveBase attempted to serialize.");
				Ar.SetError();
				bOutSuccess = false;
				return false;
			}
		}
		else if (ScriptStruct.IsError())
		{
			UE_LOGF(LogMover, Error, "FChaosNetLayeredMovesQueue::NetSerialize: Invalid ScriptStruct serialized.");
			Ar.SetError();
			bOutSuccess = false;
			return false;
		}
	}

	return true;
}

UScriptStruct* FChaosNetLayeredMovesQueue::GetScriptStruct() const
{
	return StaticStruct();
}

void FChaosNetLayeredMovesQueue::ToString(FAnsiStringBuilderBase& Out) const
{
	Out.Appendf("Layered Moves Queue -------------------------------------------------\n");
	for (int32 MoveIndex = 0; MoveIndex < Moves.Num(); ++MoveIndex)
	{
		const FChaosNetLayeredMove& ScheduledMove = Moves[MoveIndex];
		Out.Appendf("Move Index %d: ServerFrame %d, %s",
			MoveIndex,
			ScheduledMove.ExecutionServerFrame,
			ScheduledMove.Move.IsValid() ? TCHAR_TO_ANSI(*ScheduledMove.Move.Get().ToSimpleString()) : "INVALID INSTANCED STRUCT MOVE");
		Out.AppendChar('\n');
	}
	Out.Appendf("--------------------------------------------------------------------------------\n");
}

bool FChaosNetLayeredMovesQueue::ShouldReconcile(const FMoverDataStructBase& AuthorityState) const
{
	const FChaosNetLayeredMovesQueue& TypedAuthority = static_cast<const FChaosNetLayeredMovesQueue&>(AuthorityState);
	if (Moves.Num() != TypedAuthority.Moves.Num())
	{
		return true;
	}

	for (int MoveIndex = 0; MoveIndex < Moves.Num(); ++MoveIndex)
	{
		const FChaosNetLayeredMove& ScheduledMove = Moves[MoveIndex];
		const FChaosNetLayeredMove& AuthorityScheduledMove = TypedAuthority.Moves[MoveIndex];

		if (ScheduledMove.ExecutionServerFrame != AuthorityScheduledMove.ExecutionServerFrame)
		{
			return true;
		}

		if (ScheduledMove.Move.IsValid() != AuthorityScheduledMove.Move.IsValid())
		{
			return true;
		}

		if (ScheduledMove.Move.IsValid())
		{
			if (ScheduledMove.Move.GetScriptStruct() != AuthorityScheduledMove.Move.GetScriptStruct())
			{
				return true;
			}

			// This allows us to skip implementing operator== for all moves
			FArchiveCrc32 Crc1;
			(const_cast<FChaosNetLayeredMove&>(ScheduledMove)).Move.GetMutable().NetSerialize(Crc1);
			FArchiveCrc32 Crc2;
			(const_cast<FChaosNetLayeredMove&>(AuthorityScheduledMove)).Move.GetMutable().NetSerialize(Crc2);
			bool bAreEqual = (Crc1.GetCrc() == Crc2.GetCrc());
			if (!bAreEqual)
			{
				return true;
			}
		}
	}

	return false;
}

void FChaosNetLayeredMovesQueue::Interpolate(const FMoverDataStructBase& From, const FMoverDataStructBase& To, float Pct)
{
	const FChaosNetLayeredMovesQueue& TypedFrom = static_cast<const FChaosNetLayeredMovesQueue&>(From);
	const FChaosNetLayeredMovesQueue& TypedTo = static_cast<const FChaosNetLayeredMovesQueue&>(To);

	Moves = (Pct < 0.5f) ? TypedFrom.Moves : TypedTo.Moves;

}

void FChaosNetLayeredMovesQueue::Merge(const FMoverDataStructBase& From)
{
	const FChaosNetLayeredMovesQueue& TypedFrom = static_cast<const FChaosNetLayeredMovesQueue&>(From);
	Moves.Append(TypedFrom.Moves);
}

void FChaosNetLayeredMovesQueue::Decay(float DecayAmount)
{

}

void FChaosNetLayeredMoveInstancesQueue::Add(const FChaosScheduledLayeredMoveInstance& Scheduled, bool bShouldRollBack, uint8 UniqueID)
{
	if (!ensure(Scheduled.Move.IsValid() && Scheduled.Move->HasLogic()))
	{
		return;
	}

	FChaosNetLayeredMoveInstance& Added = Moves.AddDefaulted_GetRef();
	Added.ExecutionServerFrame = Scheduled.SchedulingInfo.ServerExecutionTime.FrameCount;
	Added.IssuanceServerFrame  = Scheduled.SchedulingInfo.ServerIssuanceTime.FrameCount;
	Added.UniqueID             = UniqueID;
	Added.bShouldRollBack      = bShouldRollBack;
	Added.MoveLogicClass       = const_cast<UClass*>(Scheduled.Move->GetLogicClass());
	Added.Move                 = Scheduled.Move;
}

FChaosNetLayeredMoveInstancesQueue& FChaosNetLayeredMoveInstancesQueue::operator=(const TArray<FChaosScheduledLayeredMoveInstance>& ScheduledMoves)
{
	Moves.Empty(/* Slack = */ ScheduledMoves.Num());
	for (const FChaosScheduledLayeredMoveInstance& ScheduledMove : ScheduledMoves)
	{
		Add(ScheduledMove, ScheduledMove.bShouldRollBack, 0xFF/* debugging -- unique ID does not apply */);
	}
	return *this;
}

FMoverDataStructBase* FChaosNetLayeredMoveInstancesQueue::Clone() const
{
	return new FChaosNetLayeredMoveInstancesQueue(*this);
}

bool FChaosNetLayeredMoveInstancesQueue::NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess)
{
	bOutSuccess = true;

	uint8 NumMovesToSerialize = 0;
	if (Ar.IsSaving())
	{
		ensureMsgf(Moves.Num() <= 255, TEXT("More than 255 instance moves to serialize, some will be dropped"));
		NumMovesToSerialize = FMath::Min(Moves.Num(), 255);
	}
	Ar << NumMovesToSerialize;

	if (Ar.IsLoading())
	{
		Moves.SetNumZeroed(NumMovesToSerialize);
	}

	for (int32 MoveIndex = 0; MoveIndex < NumMovesToSerialize && !Ar.IsError(); ++MoveIndex)
	{
		FChaosNetLayeredMoveInstance& NetMove = Moves[MoveIndex];

		Ar << NetMove.ExecutionServerFrame;
		Ar << NetMove.IssuanceServerFrame;
		Ar << NetMove.UniqueID;

		if (Ar.IsLoading())
		{
			NetMove.Move = MakeShared<FLayeredMoveInstance>(MakeShared<FLayeredMoveInstancedData>(), nullptr);
			NetMove.Move->NetSerialize(Ar);
			NetMove.bShouldRollBack = true; // Networked moves always roll back
			NetMove.MoveLogicClass  = const_cast<UClass*>(NetMove.Move->GetSerializedMoveLogicClass());
		}
		else
		{
			if (ensure(NetMove.Move.IsValid()))
			{
				NetMove.Move->NetSerialize(Ar);
			}
			else
			{
				UE_LOGF(LogMover, Error, "FChaosNetLayeredMoveInstancesQueue::NetSerialize: Move is invalid");
				Ar.SetError();
				bOutSuccess = false;
				return false;
			}
		}
	}

	return true;
}

UScriptStruct* FChaosNetLayeredMoveInstancesQueue::GetScriptStruct() const
{
	return StaticStruct();
}

void FChaosNetLayeredMoveInstancesQueue::ToString(FAnsiStringBuilderBase& Out) const
{
	Out.Appendf("Layered Move Instances Queue -------------------------------------------------\n");
	for (int32 MoveIndex = 0; MoveIndex < Moves.Num(); ++MoveIndex)
	{
		const FChaosNetLayeredMoveInstance& NetMove = Moves[MoveIndex];
		const UClass* LogicClass = NetMove.MoveLogicClass.Get();
		Out.Appendf("Move Index %d: ServerFrame %d, LogicClass %s",
			MoveIndex,
			NetMove.ExecutionServerFrame,
			LogicClass ? TCHAR_TO_ANSI(*LogicClass->GetName()) : "null");
		Out.AppendChar('\n');
	}
	Out.Appendf("--------------------------------------------------------------------------------\n");
}

bool FChaosNetLayeredMoveInstancesQueue::ShouldReconcile(const FMoverDataStructBase& AuthorityState) const
{
	const FChaosNetLayeredMoveInstancesQueue& TypedAuthority = static_cast<const FChaosNetLayeredMoveInstancesQueue&>(AuthorityState);
	if (Moves.Num() != TypedAuthority.Moves.Num())
	{
		return true;
	}

	for (int32 MoveIndex = 0; MoveIndex < Moves.Num(); ++MoveIndex)
	{
		const FChaosNetLayeredMoveInstance& NetMove          = Moves[MoveIndex];
		const FChaosNetLayeredMoveInstance& AuthorityNetMove = TypedAuthority.Moves[MoveIndex];

		if (NetMove.ExecutionServerFrame != AuthorityNetMove.ExecutionServerFrame)
		{
			return true;
		}

		if (NetMove.Move.IsValid() != AuthorityNetMove.Move.IsValid())
		{
			return true;
		}

		if (NetMove.Move.IsValid())
		{
			if (NetMove.Move->GetDataStructType() != AuthorityNetMove.Move->GetDataStructType())
			{
				return true;
			}

			FArchiveCrc32 Crc1;
			const_cast<FLayeredMoveInstance*>(NetMove.Move.Get())->NetSerialize(Crc1);
			FArchiveCrc32 Crc2;
			const_cast<FLayeredMoveInstance*>(AuthorityNetMove.Move.Get())->NetSerialize(Crc2);
			if (Crc1.GetCrc() != Crc2.GetCrc())
			{
				return true;
			}
		}
	}

	return false;
}

void FChaosNetLayeredMoveInstancesQueue::Interpolate(const FMoverDataStructBase& From, const FMoverDataStructBase& To, float Pct)
{
	const FChaosNetLayeredMoveInstancesQueue& TypedFrom = static_cast<const FChaosNetLayeredMoveInstancesQueue&>(From);
	const FChaosNetLayeredMoveInstancesQueue& TypedTo   = static_cast<const FChaosNetLayeredMoveInstancesQueue&>(To);

	Moves = (Pct < 0.5f) ? TypedFrom.Moves : TypedTo.Moves;
}

void FChaosNetLayeredMoveInstancesQueue::Merge(const FMoverDataStructBase& From)
{
	const FChaosNetLayeredMoveInstancesQueue& TypedFrom = static_cast<const FChaosNetLayeredMoveInstancesQueue&>(From);
	Moves.Append(TypedFrom.Moves);
}

void FChaosNetLayeredMoveInstancesQueue::Decay(float DecayAmount)
{
}

bool FAsyncLocalOnlyInstantMovementEffect::ApplyMovementEffect_Async(FApplyMovementEffectParams_Async& ApplyEffectParams, FMoverSyncState& OutputState)
{
	// AsyncFunction will not be set if this is not executing on the instance that queued this effect, nothing will execute
	if (AsyncFunction.IsSet())
	{
		return AsyncFunction(ApplyEffectParams, OutputState);
	}
	return false;
}

FInstantMovementEffect* FAsyncLocalOnlyInstantMovementEffect::Clone() const
{
	return new FAsyncLocalOnlyInstantMovementEffect(*this);
}

UScriptStruct* FAsyncLocalOnlyInstantMovementEffect::GetScriptStruct() const
{
	return FAsyncLocalOnlyInstantMovementEffect::StaticStruct();
}

FString FAsyncLocalOnlyInstantMovementEffect::ToSimpleString() const
{
	return FString::Printf(TEXT("FAsyncLocalOnlyInstantMovementEffect %s"), *OptionalName.ToString());
}

bool FDebugTeleportToInstantMovementEffect::ApplyMovementEffect_Async(FApplyMovementEffectParams_Async& ApplyEffectParams, FMoverSyncState& OutputState)
{
	if (UChaosMoverSimulation* ChaosMoverSimulation = Cast<UChaosMoverSimulation>(ApplyEffectParams.Simulation))
	{
		if (const FMoverDefaultSyncState* DefaultSyncState = OutputState.SyncStateCollection.FindDataByType<FMoverDefaultSyncState>())
		{
			FVector FinalTeleportLocation = DefaultSyncState->GetLocation_WorldSpace();
			FinalTeleportLocation.X = (TeleportLocation.X != MAX_FLT) ? TeleportLocation.X : FinalTeleportLocation.X;
			FinalTeleportLocation.Y = (TeleportLocation.Y != MAX_FLT) ? TeleportLocation.Y : FinalTeleportLocation.Y;
			FinalTeleportLocation.Z = (TeleportLocation.Z != MAX_FLT) ? TeleportLocation.Z : FinalTeleportLocation.Z;
			FRotator FinalTeleportRotation = DefaultSyncState->GetOrientation_WorldSpace();
			FinalTeleportRotation.Yaw = (TeleportRotation.Yaw != MAX_FLT) ? TeleportRotation.Yaw : FinalTeleportRotation.Yaw;
			FinalTeleportRotation.Pitch = (TeleportRotation.Pitch != MAX_FLT) ? TeleportRotation.Pitch : FinalTeleportRotation.Pitch;
			FinalTeleportRotation.Roll = (TeleportRotation.Roll != MAX_FLT) ? TeleportRotation.Roll : FinalTeleportRotation.Roll;
			ChaosMoverSimulation->AttemptTeleport(*ApplyEffectParams.TimeStep, FTransform(FinalTeleportRotation, FinalTeleportLocation), /* bUseActorRotation = */ false, OutputState);
			return true;
		}
	}
	return false;
}

FInstantMovementEffect* FDebugTeleportToInstantMovementEffect::Clone() const
{
	return new FDebugTeleportToInstantMovementEffect(*this);
}

void FDebugTeleportToInstantMovementEffect::NetSerialize(FArchive& Ar)
{
	Ar << TeleportLocation;
	Ar << TeleportRotation;
}

UScriptStruct* FDebugTeleportToInstantMovementEffect::GetScriptStruct() const
{
	return FDebugTeleportToInstantMovementEffect::StaticStruct();
}

FString FDebugTeleportToInstantMovementEffect::ToSimpleString() const
{
	return FString::Printf(TEXT("FDebugTeleportToInstantMovementEffect. TeleportLocation: %s\nTeleportRotation: %s"), *TeleportLocation.ToString(), *TeleportRotation.ToString());
}