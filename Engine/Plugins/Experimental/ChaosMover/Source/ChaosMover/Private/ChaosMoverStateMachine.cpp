// Copyright Epic Games, Inc. All Rights Reserved.


#include "ChaosMover/ChaosMoverStateMachine.h"

#include "MoverGameplayTagLog.h"
#include "Backends/ChaosMoverSubsystem.h"
#include "ChaosMover/ChaosMoverConsoleVariables.h"
#include "ChaosMover/ChaosMoverLog.h"
#include "ChaosMover/ChaosMoverSimulation.h"
#include "MoveLibrary/MovementMixer.h"
#include "MoveLibrary/MovementUtils.h"
#include "MoveLibrary/RollbackBlackboardWrappers.h"
#include "MovementModeTransition.h"
#include "MoverComponent.h"
#include "MoverDeveloperSettings.h"
#include "MoverSimulationTypes.h"
#include "DefaultMovementSet/LayeredMoves/AnimRootMotionWarpingTypes.h"
#include "MoverLog.h"
#include "Templates/SubclassOf.h"
#include "Framework/Threading.h"
#include "Engine/World.h"

#if WITH_CHAOS_VISUAL_DEBUGGER
#include "ChaosVisualDebugger/ChaosVisualDebuggerTrace.h"
#endif

#if !UE_BUILD_SHIPPING
#define NETMODE_TO_STR(SimInputs) (SimInputs && IsValid(SimInputs->OwningActor) ? *ToString(SimInputs->OwningActor->GetNetMode()) : TEXT("UNKNOWN NET MODE"))
#define MOVESHAREDPTR_TO_STR(Move) Move.IsValid() ? *Move->ToSimpleString() : TEXT("INVALID MOVE")
#define EFFECTSHAREDPTR_TO_STR(Effect) Effect.IsValid() ? *Effect->ToSimpleString() : TEXT("INVALID EFFECT")
#define MOVEINSTANCEDSTRUCT_TO_STR(Move) Move.IsValid() ? *Move.Get().ToSimpleString() : TEXT("INVALID MOVE")
#define EFFECTINSTANCEDSTRUCT_TO_STR(Effect) Effect.IsValid() ? *Effect.Get().ToSimpleString() : TEXT("INVALID EFFECT")
#endif

namespace UE::ChaosMover
{
	FMoverStateMachine::FMoverStateMachine()
	{
	}

	FMoverStateMachine::~FMoverStateMachine()
	{
	}

	void FMoverStateMachine::Init(const FInitParams& Params)
	{
		Chaos::EnsureIsInGameThreadContext();

		// Careful, this is called from the GT
		ImmediateMovementModeTransitionWeakPtr = Params.ImmediateMovementModeTransition;
		NullMovementModeWeakPtr = Params.NullMovementMode;
		Simulation = Params.Simulation;

		ClearAllMovementModes();
		ClearAllGlobalTransitions();
	}

	void FMoverStateMachine::SetRegisteredMoves(const TArray<TObjectPtr<ULayeredMoveLogic>>& Moves)
	{
		Chaos::EnsureIsInGameThreadContext();
		CachedRegisteredMoves = Moves;
	}

	void FMoverStateMachine::RegisterMovementMode(FName ModeName, TWeakObjectPtr<UBaseMovementMode> Mode, bool bIsDefaultMode)
	{
		// JAH TODO: add validation and warnings for overwriting modes
		// JAH TODO: add validation of Mode

		Modes.Add(ModeName, Mode);

		if (bIsDefaultMode)
		{
			//JAH TODO: add validation that we are only overriding the default null mode
			DefaultModeName = ModeName;
		}

		Mode->OnRegistered(ModeName, GetSimContext());
	}

	void FMoverStateMachine::UnregisterMovementMode(FName ModeName)
	{
		TWeakObjectPtr<UBaseMovementMode> ModeToUnregisterWeakPtr = Modes.FindAndRemoveChecked(ModeName);
		TStrongObjectPtr<UBaseMovementMode> ModeToUnregister = ModeToUnregisterWeakPtr.Pin();

		if (ModeToUnregister)
		{
			ModeToUnregister->OnUnregistered(GetSimContext());
		}
	}

	void FMoverStateMachine::ClearAllMovementModes()
	{
		for (TPair<FName,TWeakObjectPtr<UBaseMovementMode>>& Element : Modes)
		{
			TStrongObjectPtr<UBaseMovementMode> Mode = Element.Value.Pin();

			if (Mode)
			{
				Mode->OnUnregistered(GetSimContext());
			}
		}
		Modes.Empty();

		ConstructDefaultModes();	// Note that we're resetting to our defaults so we keep the null movement mode
	}

	void FMoverStateMachine::SetDefaultMode(FName NewDefaultModeName)
	{
		check(Modes.Contains(NewDefaultModeName));

		DefaultModeName = NewDefaultModeName;
	}

	FName FMoverStateMachine::GetDefaultModeName() const
	{
		return DefaultModeName;
	}

	void FMoverStateMachine::RegisterGlobalTransition(TWeakObjectPtr<UBaseMovementModeTransition> Transition)
	{
		GlobalTransitions.Add(Transition);

		Transition->OnRegistered();
	}

	void FMoverStateMachine::UnregisterGlobalTransition(TWeakObjectPtr<UBaseMovementModeTransition> Transition)
	{
		Transition->OnUnregistered();

		GlobalTransitions.Remove(Transition);
	}

	void FMoverStateMachine::ClearAllGlobalTransitions()
	{
		for (TWeakObjectPtr<UBaseMovementModeTransition> TransitionWeakPtr : GlobalTransitions)
		{
			TStrongObjectPtr<UBaseMovementModeTransition> Transition = TransitionWeakPtr.Pin();
			if (Transition)
			{
				Transition->OnUnregistered();
			}
		}

		GlobalTransitions.Empty();
	}

	void FMoverStateMachine::QueueNextMode(FName DesiredNextModeName, bool bShouldReenter)
	{
		QueueNextModeInternal(DesiredNextModeName, bShouldReenter, /*bIsFromRollback*/ false);
	}

	void FMoverStateMachine::QueueNextModeInternal(FName DesiredNextModeName, bool bShouldReenter, bool bIsFromRollback)
	{
		TStrongObjectPtr<UImmediateMovementModeTransition> QueuedModeTransition = QueuedModeTransitionWeakPtr.Pin();
		if (QueuedModeTransition)
		{
			if (DesiredNextModeName != NAME_None)
			{
				const FName NextModeName = QueuedModeTransition->GetNextModeName();
				const bool bShouldNextModeReenter = QueuedModeTransition->ShouldReenter();

				if ((NextModeName != NAME_None) &&
					(NextModeName != DesiredNextModeName || bShouldReenter != bShouldNextModeReenter))
				{
					UE_LOGF(LogChaosMover, Log, "%ls (%ls) Overwriting of queued mode change (%ls, reenter: %i) with (%ls, reenter: %i)", *OwnerActorName, *UEnum::GetValueAsString(OwnerActorLocalNetRole), *NextModeName.ToString(), bShouldNextModeReenter, *DesiredNextModeName.ToString(), bShouldReenter);
				}

				if (Modes.Contains(DesiredNextModeName))
				{
					QueuedModeTransition->SetNextMode(DesiredNextModeName, bShouldReenter, bIsFromRollback);
				}
				else
				{
					UE_LOGF(LogChaosMover, Warning, "Attempted to queue an unregistered movement mode: %ls on owner %ls", *DesiredNextModeName.ToString(), *OwnerActorName);
				}
			}
		}
	}

	void FMoverStateMachine::SetModeImmediately(const FMoverTimeStep& TimeStep, FName DesiredModeName, bool bShouldReenter)
	{
		QueueNextModeInternal(DesiredModeName, bShouldReenter, /*bIsFromRollback*/ false);
		AdvanceToNextMode(TimeStep);
	}

	void FMoverStateMachine::ClearQueuedMode()
	{
		TStrongObjectPtr<UImmediateMovementModeTransition> QueuedModeTransition = QueuedModeTransitionWeakPtr.Pin();
		if (QueuedModeTransition)
		{
			QueuedModeTransition->Clear();
		}
	}

	FMovementModifierHandle FMoverStateMachine::QueueMovementModifier(TSharedPtr<FMovementModifierBase> Modifier)
	{
		if (ensure(Modifier.IsValid()))
		{
			QueuedMovementModifiers.Add(Modifier);
			Modifier->GenerateHandle();

			return Modifier->GetHandle();
		}

		return 0;
	}

	void FMoverStateMachine::CancelModifierFromHandle(FMovementModifierHandle ModifierHandle)
	{
		QueuedMovementModifiers.RemoveAll([ModifierHandle, this] (const TSharedPtr<FMovementModifierBase>& Modifier)
			{
				if (Modifier.IsValid())
				{
					if (Modifier->GetHandle() == ModifierHandle)
					{
						return true;
					}
				}
				else
				{
					return true;
				}

				return false;
			});

		ModifiersToCancel.Add(ModifierHandle);
	}

	const FMovementModifierBase* FMoverStateMachine::FindQueuedModifier(FMovementModifierHandle ModifierHandle) const
	{
		for (const TSharedPtr<FMovementModifierBase>& QueuedModifier : QueuedMovementModifiers)
		{
			if (QueuedModifier->GetHandle() == ModifierHandle)
			{
				return QueuedModifier.Get();
			}
		}

		return nullptr;
	}

	const FMovementModifierBase* FMoverStateMachine::FindQueuedModifierByType(const UScriptStruct* ModifierType) const
	{
		for (const TSharedPtr<FMovementModifierBase>& QueuedModifier : QueuedMovementModifiers)
		{
			if (QueuedModifier->GetScriptStruct()->IsChildOf(ModifierType))
			{
				return QueuedModifier.Get();
			}
		}

		return nullptr;
	}

	void FMoverStateMachine::FlushQueuedModifiersToGroup(FMovementModifierGroup& ModifierGroup)
	{
		if (!QueuedMovementModifiers.IsEmpty())
		{
			for (TSharedPtr<FMovementModifierBase>& QueuedModifier : QueuedMovementModifiers)
			{
				ModifierGroup.QueueMovementModifier(QueuedModifier);
			}

			QueuedMovementModifiers.Empty();
		}
	}

	void FMoverStateMachine::FlushModifierCancellationsToGroup(FMovementModifierGroup& ActiveModifierGroup)
	{
		for (FMovementModifierHandle HandleToCancel : ModifiersToCancel)
		{
			ActiveModifierGroup.CancelModifierFromHandle(HandleToCancel);
		}

		ModifiersToCancel.Empty();
	}

	bool FMoverStateMachine::HasAnyInstantEffectsQueued(const FMoverTimeStep& TimeStep) const
	{
		if (!QueuedInstantEffects.IsEmpty())
		{
			for (int EffectIndex = 0; EffectIndex < QueuedInstantEffects.Num(); EffectIndex++)
			{
				const FChaosScheduledInstantMovementEffect& QueuedEffect = QueuedInstantEffects[EffectIndex];
				if (QueuedEffect.ScheduledEffect.SchedulingInfo.ShouldExecuteAtFrame(TimeStep.ServerFrame))
				{
					return true;
				}
			}
		}
		return false;
	}

	void FMoverStateMachine::ReceiveInstantMovementEffects(const FMoverTimeStep& TimeStep, const FChaosNetInstantMovementEffectsQueue* InstantMovementEffectsQueue)
	{
		QUICK_SCOPE_CYCLE_COUNTER(MoverStateMachine_ReceiveInstantMovementEffects);

		InstantMovementEffectsIDHistory.CullOldFrames(TimeStep.ServerFrame, CVars::InstantMovementEffectIDHistorySize);

		// Copy queued instant movement effects from the input to the state machine
		// After this, the state machine works on its own queue, into which it can enqueue instant movement effects while stepping
		for (const FChaosNetInstantMovementEffect& NetInstantMovementEffect : InstantMovementEffectsQueue->Effects)
		{
			if (ensure(NetInstantMovementEffect.Effect.IsValid()))
			{
				if (!InstantMovementEffectsIDHistory.WasIDAlreadySeen(NetInstantMovementEffect.UniqueID))
				{
					QueueInstantMovementEffect(FChaosScheduledInstantMovementEffect{
												NetInstantMovementEffect.bShouldRollBack,
												NetInstantMovementEffect.AsScheduledInstantMovementEffect() });

					InstantMovementEffectsIDHistory.AddID(TimeStep.ServerFrame, NetInstantMovementEffect.UniqueID);

#if !UE_BUILD_SHIPPING
					if (const FChaosMoverSimulationDefaultInputs* SimInputs = Simulation->GetLocalSimInput().FindDataByType<FChaosMoverSimulationDefaultInputs>())
					{
						UE_LOGF(LogChaosMover, Verbose, "(%ls, %ls, Current Frame = %d) FMoverStateMachine::ReceiveInstantMovementEffects: Received Instant Effect (ID=%d), issued at frame %d scheduled for frame %d: %ls",
							NETMODE_TO_STR(SimInputs),
							SimInputs->bIsGeneratingInputsLocally ? TEXT("Input Source") : TEXT("Input Consumer"),
							TimeStep.ServerFrame,
							NetInstantMovementEffect.UniqueID,
							NetInstantMovementEffect.IssuanceServerFrame,
							NetInstantMovementEffect.ExecutionServerFrame,
							EFFECTINSTANCEDSTRUCT_TO_STR(NetInstantMovementEffect.Effect));
					}
#endif // !UE_BUILD_SHIPPING
				}
				else
				{
#if !UE_BUILD_SHIPPING
					if (const FChaosMoverSimulationDefaultInputs* SimInputs = Simulation->GetLocalSimInput().FindDataByType<FChaosMoverSimulationDefaultInputs>())
					{
						UE_LOGF(LogChaosMover, Verbose, "(%ls, %ls, Current Frame = %d) FMoverStateMachine::ReceiveInstantMovementEffects: Skipping Instant Effect (ID=%d), ID already seen!",
							NETMODE_TO_STR(SimInputs),
							SimInputs->bIsGeneratingInputsLocally ? TEXT("Input Source") : TEXT("Input Consumer"),
							TimeStep.ServerFrame,
							NetInstantMovementEffect.UniqueID);
					}
#endif // !UE_BUILD_SHIPPING
				}
			}
		}
	}

	void FMoverStateMachine::ReceiveLayeredMoves(const FMoverTimeStep& TimeStep, const FChaosNetLayeredMovesQueue* LayeredMovesQueue)
	{
		QUICK_SCOPE_CYCLE_COUNTER(MoverStateMachine_ReceiveLayeredMoves);

		LayeredMovesIDHistory.CullOldFrames(TimeStep.ServerFrame, CVars::LayeredMoveIDHistorySize);

		// Copy queued layered moves from the input to the state machine
		// After this, the state machine works on its own queue, into which it can enqueue moves while stepping
		for (const FChaosNetLayeredMove& NetLayeredMove : LayeredMovesQueue->Moves)
		{
			if (ensure(NetLayeredMove.Move.IsValid()))
			{
				if (!LayeredMovesIDHistory.WasIDAlreadySeen(NetLayeredMove.UniqueID))
				{
					QueueLayeredMove(FChaosScheduledLayeredMove{
						NetLayeredMove.bShouldRollBack,
						NetLayeredMove.AsScheduledLayeredMove() });

					LayeredMovesIDHistory.AddID(TimeStep.ServerFrame, NetLayeredMove.UniqueID);

#if !UE_BUILD_SHIPPING
					if (const FChaosMoverSimulationDefaultInputs* SimInputs = Simulation->GetLocalSimInput().FindDataByType<FChaosMoverSimulationDefaultInputs>())
					{
						UE_LOGF(LogChaosMover, Verbose, "(%ls, %ls, Current Frame = %d) FMoverStateMachine::ReceiveLayeredMoves: Received Layered Move (ID=%d), issued at frame %d scheduled for frame %d: %ls",
							NETMODE_TO_STR(SimInputs),
							SimInputs->bIsGeneratingInputsLocally ? TEXT("Input Source") : TEXT("Input Consumer"),
							TimeStep.ServerFrame,
							NetLayeredMove.UniqueID,
							NetLayeredMove.IssuanceServerFrame,
							NetLayeredMove.ExecutionServerFrame,
							MOVEINSTANCEDSTRUCT_TO_STR(NetLayeredMove.Move));
					}
#endif // !UE_BUILD_SHIPPING
				}
				else
				{
#if !UE_BUILD_SHIPPING
					if (const FChaosMoverSimulationDefaultInputs* SimInputs = Simulation->GetLocalSimInput().FindDataByType<FChaosMoverSimulationDefaultInputs>())
					{
						UE_LOGF(LogChaosMover, Verbose, "(%ls, %ls, Current Frame = %d) FMoverStateMachine::ReceiveLayeredMoves: Skipping Layered Move (ID=%d), ID already seen!",
							NETMODE_TO_STR(SimInputs),
							SimInputs->bIsGeneratingInputsLocally ? TEXT("Input Source") : TEXT("Input Consumer"),
							TimeStep.ServerFrame,
							NetLayeredMove.UniqueID);
					}
#endif // !UE_BUILD_SHIPPING
				}
			}
		}
	}

	void FMoverStateMachine::ReceiveLayeredMoveInstances(const FMoverTimeStep& TimeStep, const FChaosNetLayeredMoveInstancesQueue* Queue)
	{
		QUICK_SCOPE_CYCLE_COUNTER(MoverStateMachine_ReceiveLayeredMoveInstances);

		LayeredMoveInstancesIDHistory.CullOldFrames(TimeStep.ServerFrame, CVars::LayeredMoveIDHistorySize);

		for (const FChaosNetLayeredMoveInstance& NetMove : Queue->Moves)
		{
			if (!ensure(NetMove.Move.IsValid()))
			{
				continue;
			}

			if (!LayeredMoveInstancesIDHistory.WasIDAlreadySeen(NetMove.UniqueID))
			{
				FChaosScheduledLayeredMoveInstance Scheduled;
				Scheduled.bShouldRollBack = NetMove.bShouldRollBack;
				Scheduled.SchedulingInfo  = FMoverSchedulingInfo(
					FMoverTime(NetMove.IssuanceServerFrame, /* TimeMs = */ 0.0),
					FMoverTime(NetMove.ExecutionServerFrame, /* TimeMs = */ 0.0),
					/* bIsFixedDt = */ true);
				Scheduled.Move = NetMove.Move;

				QueueLayeredMoveInstance(Scheduled);
				LayeredMoveInstancesIDHistory.AddID(TimeStep.ServerFrame, NetMove.UniqueID);

#if !UE_BUILD_SHIPPING
				if (const FChaosMoverSimulationDefaultInputs* SimInputs = Simulation->GetLocalSimInput().FindDataByType<FChaosMoverSimulationDefaultInputs>())
				{
					UE_LOGF(LogChaosMover, Verbose, "(%ls, %ls, Current Frame = %d) FMoverStateMachine::ReceiveLayeredMoveInstances: Received Move Instance (ID=%d), issued at frame %d scheduled for frame %d",
						NETMODE_TO_STR(SimInputs),
						SimInputs->bIsGeneratingInputsLocally ? TEXT("Input Source") : TEXT("Input Consumer"),
						TimeStep.ServerFrame,
						NetMove.UniqueID,
						NetMove.IssuanceServerFrame,
						NetMove.ExecutionServerFrame);
				}
#endif // !UE_BUILD_SHIPPING
			}
			else
			{
#if !UE_BUILD_SHIPPING
				if (const FChaosMoverSimulationDefaultInputs* SimInputs = Simulation->GetLocalSimInput().FindDataByType<FChaosMoverSimulationDefaultInputs>())
				{
					UE_LOGF(LogChaosMover, Verbose, "(%ls, %ls, Current Frame = %d) FMoverStateMachine::ReceiveLayeredMoveInstances: Skipping Move Instance (ID=%d), ID already seen!",
						NETMODE_TO_STR(SimInputs),
						SimInputs->bIsGeneratingInputsLocally ? TEXT("Input Source") : TEXT("Input Consumer"),
						TimeStep.ServerFrame,
						NetMove.UniqueID);
				}
#endif // !UE_BUILD_SHIPPING
			}
		}
	}

	void FMoverStateMachine::CollectGameplayTags(const FMoverSyncState& SyncState, FGameplayTagContainer& OutTags) const
	{
		if (const TWeakObjectPtr<UBaseMovementMode>* ModeWeakPtr = Modes.Find(SyncState.MovementMode))
		{
			if (TStrongObjectPtr<UBaseMovementMode> Mode = ModeWeakPtr->Pin())
			{
				Mode->GetGameplayTags(OutTags);
			}
		}

		for (auto ModifierIt = SyncState.MovementModifiers.GetActiveModifiersIterator(); ModifierIt; ++ModifierIt)
		{
			(*ModifierIt)->GetGameplayTags(OutTags);
		}

		for (const TSharedPtr<FLayeredMoveBase>& Move : SyncState.LayeredMoves.GetActiveMoves())
		{
			Move->GetGameplayTags(OutTags);
		}
	}

	void FMoverStateMachine::DiffAndEmitGameplayTagEvents(const FGameplayTagContainer& OldTags, const FGameplayTagContainer& NewTags, const FMoverTimeStep& TimeStep, bool bIsCausedByRollback)
	{
		TStrongObjectPtr<UChaosMoverSimulation> SimStrongObjPtr = Simulation.Pin();
		if (!SimStrongObjPtr)
		{
			return;
		}

		for (const FGameplayTag& Tag : NewTags)
		{
			if (!OldTags.HasTagExact(Tag))
			{
				TSharedPtr<FMoverGameplayTagChangeEventData> Event = MakeShared<FMoverGameplayTagChangeEventData>(TimeStep, Tag, /*bWasAdded=*/true);
				Event->Context.bIsCausedByRollback = bIsCausedByRollback;

				// Mover Gameplay Tag Logging
				MOVER_TAG_LOG(LogChaosMover, "[WT:TagDiff] Actor=%s Role=%s Frame=%d IsResim=%d IsRollback=%d Tag=%s Change=Added",
					*SimStrongObjPtr->DebugOwnerName, *SimStrongObjPtr->DebugOwnerRole, TimeStep.ServerFrame, (int32)TimeStep.bIsResimulating, (int32)bIsCausedByRollback, *Tag.ToString());

				SimStrongObjPtr->AddEvent(Event);
			}
		}

		for (const FGameplayTag& Tag : OldTags)
		{
			if (!NewTags.HasTagExact(Tag))
			{
				TSharedPtr<FMoverGameplayTagChangeEventData> Event = MakeShared<FMoverGameplayTagChangeEventData>(TimeStep, Tag, /*bWasAdded=*/false);
				Event->Context.bIsCausedByRollback = bIsCausedByRollback;

				// Mover Gameplay Tag Logging
				MOVER_TAG_LOG(LogChaosMover, "[WT:TagDiff] Actor=%s Role=%s Frame=%d IsResim=%d IsRollback=%d Tag=%s Change=Removed",
					*SimStrongObjPtr->DebugOwnerName, *SimStrongObjPtr->DebugOwnerRole, TimeStep.ServerFrame, (int32)TimeStep.bIsResimulating, (int32)bIsCausedByRollback, *Tag.ToString());

				SimStrongObjPtr->AddEvent(Event);
			}
		}

		// Mover Gameplay Tag Logging
		MOVER_TAG_LOG(LogChaosMover, "[WT:TagDiffState] Actor=%s Role=%s Frame=%d IsResim=%d IsRollback=%d OldTags=[%s] NewTags=[%s]",
			*SimStrongObjPtr->DebugOwnerName, *SimStrongObjPtr->DebugOwnerRole, TimeStep.ServerFrame, (int32)TimeStep.bIsResimulating, (int32)bIsCausedByRollback,
			*OldTags.ToStringSimple(), *NewTags.ToStringSimple());
	}

	void FMoverStateMachine::OnSimulationTick(const FMoverSimContext& SimContext, const FMoverTimeStep& TimeStep, const FMoverTickStartData& StartState, UMoverBlackboard* SimBlackboard, UMovementMixer* MovementMixer, FMoverTickEndData& OutputState)
	{
		QUICK_SCOPE_CYCLE_COUNTER(MoverStateMachine_OnSimulationTick);

		FMoverTimeStep SubTimeStep = TimeStep;
		WorkingSubstepStartData = StartState;
		bool bIsWorkingStartStateReady = true;	// this flag is used to avoid unneeded data copying after substeps

		WorkingSimTickEndData = &OutputState;
		ON_SCOPE_EXIT{ WorkingSimTickEndData = nullptr; };

		if (!ensure(MovementMixer))
		{
			return;
		}

		InternalSimTime = TimeStep;

		// Receive all sim commands (instant movement effects, queued layered moves, etc). Some may be scheduled to happen in the future.

		// Copy queued instant movement effects from the input to the state machine
		// After this, the state machine works on its own queue, into which it can enqueue instant movement effects while stepping
		// Most will come from the networked input
		if (const FChaosNetInstantMovementEffectsQueue* InstantMovementEffectsQueue = StartState.InputCmd.InputCollection.FindDataByType<FChaosNetInstantMovementEffectsQueue>())
		{
			ReceiveInstantMovementEffects(TimeStep, InstantMovementEffectsQueue);
		}
		// Some may come from local sim input, non networked, in the case of instant movement effects queued at non input producing end points (remotely controlled actor on a server or sim proxy on a client)
		if (const FChaosNetInstantMovementEffectsQueue* LocalInstantMovementEffectsQueue = Simulation->GetLocalSimInput().FindDataByType<FChaosNetInstantMovementEffectsQueue>())
		{
			ReceiveInstantMovementEffects(TimeStep, LocalInstantMovementEffectsQueue);
		}
		// Same for layered moves (we need to generalize this to handle any abstract "sim command")
		// Most will come from the networked input
		if (const FChaosNetLayeredMovesQueue* LayeredMovesQueue = StartState.InputCmd.InputCollection.FindDataByType<FChaosNetLayeredMovesQueue>())
		{
			ReceiveLayeredMoves(TimeStep, LayeredMovesQueue);
		}
		// Some may come from local sim input, in the case of moves queued at non input producing end points (remotely controlled actor on a server or sim proxy on a client)
		if (const FChaosNetLayeredMovesQueue* LocalLayeredMovesQueue = Simulation->GetLocalSimInput().FindDataByType<FChaosNetLayeredMovesQueue>())
		{
			ReceiveLayeredMoves(TimeStep, LocalLayeredMovesQueue);
		}
		// Same for FLayeredMoveInstance moves (newer ULayeredMoveLogic-based system)
		if (const FChaosNetLayeredMoveInstancesQueue* InstanceMovesQueue = StartState.InputCmd.InputCollection.FindDataByType<FChaosNetLayeredMoveInstancesQueue>())
		{
			ReceiveLayeredMoveInstances(TimeStep, InstanceMovesQueue);
		}
		if (const FChaosNetLayeredMoveInstancesQueue* LocalInstanceMovesQueue = Simulation->GetLocalSimInput().FindDataByType<FChaosNetLayeredMoveInstancesQueue>())
		{
			ReceiveLayeredMoveInstances(TimeStep, LocalInstanceMovesQueue);
		}

#if WITH_CHAOS_VISUAL_DEBUGGER
		CaptureUnprocessedSimCommandsForCVD();
#endif

		TStrongObjectPtr<UImmediateMovementModeTransition> QueuedModeTransition = QueuedModeTransitionWeakPtr.Pin();
		if (QueuedModeTransition && !QueuedModeTransition->IsSet())
		{
			QueueNextMode(WorkingSubstepStartData.SyncState.MovementMode);
		}

		AdvanceToNextMode(TimeStep);

		int SubStepCount = 0;
		const int32 MaxConsecutiveFullRefundedSubsteps = GetDefault<UMoverDeveloperSettings>()->MaxTimesToRefundSubstep;
		int32 NumConsecutiveFullRefundedSubsteps = 0;

		float TotalUsedMs = 0.0f;
		while (TotalUsedMs < TimeStep.StepMs)
		{
			InternalSimTime.BaseSimTimeMs = SubTimeStep.BaseSimTimeMs;

			if (!bIsWorkingStartStateReady)
			{
				WorkingSubstepStartData.SyncState = OutputState.SyncState;
				WorkingSubstepStartData.AuxState = OutputState.AuxState;
				bIsWorkingStartStateReady = true;
			}

			WorkingSubstepStartData.SyncState.MovementMode = CurrentModeName;

			FMoverDefaultSyncState* OutputSyncState = &OutputState.SyncState.SyncStateCollection.FindOrAddMutableDataByType<FMoverDefaultSyncState>();
			OutputState.SyncState.MovementMode = CurrentModeName;

			OutputState.MovementEndState.ResetToDefaults();

			SubTimeStep.StepMs = TimeStep.StepMs - TotalUsedMs;		// TODO: convert this to an overridable function that can support MaxStepTime, MaxIterations, etc.

			// Transfer any queued moves into the starting state. They'll be started during the move generation.
			FlushQueuedMovesToGroup(TimeStep, WorkingSubstepStartData.SyncState.LayeredMoves);
			OutputState.SyncState.LayeredMoves = WorkingSubstepStartData.SyncState.LayeredMoves;

			// Flush any GT-queued FLayeredMoveInstance moves whose execution frame has arrived.
			FlushQueuedMoveInstancesToGroup(TimeStep, WorkingSubstepStartData.SyncState.LayeredMoveInstances);

			// Mirror instance moves into the output state, then re-link any logic pointers that
			// were lost during network deserialization.
			OutputState.SyncState.LayeredMoveInstances = WorkingSubstepStartData.SyncState.LayeredMoveInstances;
			OutputState.SyncState.LayeredMoveInstances.PopulateMissingActiveMoveLogic(CachedRegisteredMoves);

			FlushQueuedModifiersToGroup(WorkingSubstepStartData.SyncState.MovementModifiers);
			OutputState.SyncState.MovementModifiers = WorkingSubstepStartData.SyncState.MovementModifiers;

			bool bModeSetFromInstantEffect = false;
			// Apply any instant effects that were queued up between ticks
			if (ApplyInstantEffects(WorkingSubstepStartData, SubTimeStep, OutputState.SyncState))
			{
				bModeSetFromInstantEffect = UpdateWorkingStateFromSyncState(SubTimeStep, OutputState.SyncState);
			}

			FMovementModifierParams_Async ModifierParams(Simulation.Get(), &WorkingSubstepStartData.SyncState, &SubTimeStep);

			FMovementModifierGroup& CurrentModifiers = OutputState.SyncState.MovementModifiers;
			FlushModifierCancellationsToGroup(CurrentModifiers);
			TArray<TSharedPtr<FMovementModifierBase>> ActiveModifiers = CurrentModifiers.GenerateActiveModifiers_Async(ModifierParams);
			
			for (TSharedPtr<FMovementModifierBase> Modifier : ActiveModifiers)
			{
				Modifier->OnPreMovement_Async(ModifierParams);
			}

			// Expose the skeletal mesh component's transform relative to the actor root so that async
			// layered moves (e.g. AnimRootMotion) can convert mesh-local root motion into world space
			// without touching game-thread-only objects.
			// TODO: This blackboard write is a workaround for GenerateMove_Async not receiving a
			// simulation context. Once FLayeredMoveBase::GenerateMove_Async is updated to accept a
			// context struct analogous to FMovementModifierParams_Async (which already carries
			// UMoverSimulation*), this data can be read directly from LocalSimInput via that context
			// instead, and this blackboard write can be removed.
			if (const FChaosMoverSimulationDefaultInputs* SimInputs = Simulation->GetLocalSimInput().FindDataByType<FChaosMoverSimulationDefaultInputs>())
			{
				SimBlackboard->Set(AnimRootMotionBlackboard::LastPrimaryVisualComponentRelativeTransform, SimInputs->PrimaryVisualComponentRelativeTransform);
			}
			if (const FMoverMotionWarpingInputs* WarpInputs = StartState.InputCmd.InputCollection.FindDataByType<FMoverMotionWarpingInputs>())
			{
				SimBlackboard->Set(AnimRootMotionBlackboard::LastResolvedMotionWarpTargets, WarpInputs->WarpTargets);
			}

			FLayeredMoveGroup& CurrentLayeredMoves = OutputState.SyncState.LayeredMoves;

			// Gather any layered move contributions
			FProposedMove CombinedLayeredMove;
			CombinedLayeredMove.MixMode = EMoveMixMode::AdditiveVelocity;
			bool bHasLayeredMoveContributions = false;
			MovementMixer->ResetMixerState();

			TArray<TSharedPtr<FLayeredMoveBase>> ActiveMoves = CurrentLayeredMoves.GenerateActiveMoves_Async(SubTimeStep, SimBlackboard);

			// Tick and accumulate all active moves
			// Gather all proposed moves and distill this into a cumulative movement report. May include separate additive vs override moves.
			// TODO: may want to sort by priority or other factors
			for (TSharedPtr<FLayeredMoveBase>& ActiveMove : ActiveMoves)
			{
				FProposedMove MoveStep;
				MoveStep.MixMode = ActiveMove->MixMode;	// Initialize using the move's mixmode, but allow it to be changed in GenerateMove

				if (ActiveMove->GenerateMove_Async(WorkingSubstepStartData, SubTimeStep, SimBlackboard, MoveStep))
				{
					// If this active move is already past it's first tick we don't need to set the preferred mode again
					if (ActiveMove->StartSimTimeMs < SubTimeStep.BaseSimTimeMs)
					{
						MoveStep.PreferredMode = NAME_None;
					}

					// Some modes may handle vertical movement themselves (e.g. falling) during anim root motion
					if (MoveStep.MixMode == EMoveMixMode::OverrideAll
						&& ActiveMove->HasGameplayTag(Mover_AnimRootMotion, /*exact?*/false) 
						&& GetLastKnownGameplayTags().HasTag(Mover_SkipVerticalAnimRootMotion))
					{
						MoveStep.MixMode = EMoveMixMode::OverrideAllExceptVerticalVelocity;
					}

					bHasLayeredMoveContributions = true;
					MovementMixer->MixLayeredMove(*ActiveMove, MoveStep, CombinedLayeredMove);
				}
			}

			// Tick and accumulate all active instanced layered moves
			FLayeredMoveInstanceGroup& CurrentLayeredMoveInstances = OutputState.SyncState.LayeredMoveInstances;
			TArray<TSharedPtr<FLayeredMoveInstance>> ActiveMoveInstances = CurrentLayeredMoveInstances.GenerateActiveMoves_Async(SubTimeStep, SimBlackboard);
			for (TSharedPtr<FLayeredMoveInstance>& ActiveMoveInstance : ActiveMoveInstances)
			{
				FProposedMove MoveStep;
				MoveStep.MixMode = ActiveMoveInstance->GetMixMode();

				if (ActiveMoveInstance->GenerateMove_Async(WorkingSubstepStartData, SubTimeStep, SimBlackboard, MoveStep))
				{
					// If this active move is already past its first tick we don't need to set the preferred mode again
					if (ActiveMoveInstance->GetStartingTimeMs() < SubTimeStep.BaseSimTimeMs)
					{
						MoveStep.PreferredMode = NAME_None;
					}

					bHasLayeredMoveContributions = true;
					MovementMixer->MixLayeredMove(*ActiveMoveInstance, MoveStep, CombinedLayeredMove);
				}
			}
			
			if (const FChaosNetInstantMovementEffectsQueue* LocalInstantMovementEffectsQueue = Simulation->GetLocalSimInput().FindDataByType<FChaosNetInstantMovementEffectsQueue>())
			{
				ReceiveInstantMovementEffects(TimeStep, LocalInstantMovementEffectsQueue);
			}
			
			// Apply any instant effects that were queued from various movement effects - Modifiers, layered moves, etc.
			// Note: This won't apply any instant effects an extra time since we keep track of ID's we've already seen
			if (ApplyInstantEffects(WorkingSubstepStartData, SubTimeStep, OutputState.SyncState))
			{
				bModeSetFromInstantEffect = UpdateWorkingStateFromSyncState(SubTimeStep, OutputState.SyncState);
			}
			
			if (bHasLayeredMoveContributions && !CombinedLayeredMove.PreferredMode.IsNone() && !bModeSetFromInstantEffect)
			{
				SetModeImmediately(SubTimeStep, CombinedLayeredMove.PreferredMode);
				OutputState.SyncState.MovementMode = CurrentModeName;
			}

			// Merge proposed movement from the current mode with movement from layered moves
			if (!CurrentModeName.IsNone() && Modes.Contains(CurrentModeName))
			{
				TStrongObjectPtr<UBaseMovementMode> CurrentMode = Modes[CurrentModeName].Pin();
				if (CurrentMode)
				{
					FProposedMove CombinedMove;
					bool bHasModeMoveContribution = false;

					if (!CVars::bSkipGenerateMoveIfOverridden ||
						!(bHasLayeredMoveContributions && CombinedLayeredMove.MixMode == EMoveMixMode::OverrideAll))
					{
						QUICK_SCOPE_CYCLE_COUNTER(MoverStateMachine_OnSimulationTick_GenerateMoveFromMode);
						CurrentMode->GenerateMove(SimContext, WorkingSubstepStartData, SubTimeStep, OUT CombinedMove);

						bHasModeMoveContribution = true;
					}

					if (bHasModeMoveContribution && bHasLayeredMoveContributions)
					{
						FVector UpDir = FVector::UpVector;
						if (const FChaosMoverSimulationDefaultInputs* DefaultSimInputs = Simulation->GetLocalSimInput().FindDataByType<FChaosMoverSimulationDefaultInputs>())
						{
							UpDir = DefaultSimInputs->UpDir;
						}

						MovementMixer->MixProposedMoves(CombinedLayeredMove, UpDir, CombinedMove);
					}
					else if (bHasLayeredMoveContributions && !bHasModeMoveContribution)
					{
						CombinedMove = CombinedLayeredMove;
					}

					// Apply any layered move finish velocity settings
					if (CurrentLayeredMoves.bApplyResidualVelocity)
					{
						CombinedMove.LinearVelocity = CurrentLayeredMoves.ResidualVelocity;
					}
					if (CurrentLayeredMoves.ResidualClamping >= 0.0f)
					{
						CombinedMove.LinearVelocity = CombinedMove.LinearVelocity.GetClampedToMaxSize(CurrentLayeredMoves.ResidualClamping);
					}
					CurrentLayeredMoves.ResetResidualVelocity();

					// We need to replace this with some async equivalent (calling back to FSimulation? an optional FinalMoveProcessor object, a bit like the optional MoveMixer?)
					// SyncTickParams->MoverComponent->ProcessGeneratedMovement.ExecuteIfBound(SubstepStartData, SubTimeStep, OUT CombinedMove);

					// Execute the combined proposed move
					{
						// WorkingSimTickParams.MovingComps is left empty in the async case, so we don't access resources used by the concurrent gameplay thread
						WorkingSimTickParams.SimContext = SimContext;
						WorkingSimTickParams.StartState = WorkingSubstepStartData;
						WorkingSimTickParams.SimBlackboard = SimBlackboard;
						WorkingSimTickParams.TimeStep = SubTimeStep;
						WorkingSimTickParams.ProposedMove = CombinedMove;

						// Check for any transitions, first those registered with the current movement mode, then global ones that could occur from any mode
						FTransitionEvalResult EvalResult = FTransitionEvalResult::NoTransition;
						TStrongObjectPtr<UBaseMovementModeTransition> TransitionToTrigger;

						for (UBaseMovementModeTransition* Transition : CurrentMode->Transitions)
						{
							if (IsValid(Transition) && ((SubStepCount == 0) || !Transition->bFirstSubStepOnly))
							{
								EvalResult = Transition->Evaluate(WorkingSimTickParams);

								if (!EvalResult.NextMode.IsNone())
								{
									if (EvalResult.NextMode != CurrentModeName || Transition->bAllowModeReentry)
									{
										TransitionToTrigger = TStrongObjectPtr<UBaseMovementModeTransition>(Transition);
										break;
									}
								}
							}
						}

						if (TransitionToTrigger == nullptr)
						{
							for (TWeakObjectPtr<UBaseMovementModeTransition> TransitionWeakPtr : GlobalTransitions)
							{
								TStrongObjectPtr<UBaseMovementModeTransition> Transition = TransitionWeakPtr.Pin();
								if (Transition && ((SubStepCount == 0) || !Transition->bFirstSubStepOnly))
								{
									EvalResult = Transition->Evaluate(WorkingSimTickParams);

									if (!EvalResult.NextMode.IsNone())
									{
										if (EvalResult.NextMode != CurrentModeName || Transition->bAllowModeReentry)
										{
											TransitionToTrigger = Transition;
											break;
										}
									}
								}
							}
						}

						if (TransitionToTrigger && !EvalResult.NextMode.IsNone())
						{
							OutputState.MovementEndState.NextModeName = EvalResult.NextMode;
							OutputState.MovementEndState.RemainingMs = WorkingSimTickParams.TimeStep.StepMs; 	// Pass all remaining time to next mode
							TransitionToTrigger->Trigger(WorkingSimTickParams);
						}
						else
						{
							QUICK_SCOPE_CYCLE_COUNTER(MoverStateMachine_OnSimulationTick_CurrentModeSimulationTick);
							CurrentMode->SimulationTick(WorkingSimTickParams, OutputState);
						}

						OutputState.MovementEndState.RemainingMs = FMath::Clamp(OutputState.MovementEndState.RemainingMs, 0.0f, SubTimeStep.StepMs);
					}

					QueueNextMode(OutputState.MovementEndState.NextModeName);

					// Check if all of the time for this Substep was refunded
					if (FMath::IsNearlyEqual(SubTimeStep.StepMs, OutputState.MovementEndState.RemainingMs, UE_KINDA_SMALL_NUMBER))
					{
						NumConsecutiveFullRefundedSubsteps++;
						// if we've done this sub step a lot before go ahead and just advance time to avoid freezing editor
						if (NumConsecutiveFullRefundedSubsteps >= MaxConsecutiveFullRefundedSubsteps)
						{
							UE_LOGF(LogChaosMover, Warning, "Movement mode %ls and %ls on %ls are stuck giving time back to each other. Overriding to advance to next substep.",
								*CurrentModeName.ToString(),
								*OutputState.MovementEndState.NextModeName.ToString(),
								*OwnerActorName);
							TotalUsedMs += SubTimeStep.StepMs;
						}
					}
					else
					{
						NumConsecutiveFullRefundedSubsteps = 0;
					}

					//GEngine->AddOnScreenDebugMessage(-1, -0.1f, FColor::White, FString::Printf(TEXT("NextModeName: %s  Queued: %s"), *Output.MovementEndState.NextModeName.ToString(), *NextModeName.ToString()));
				}
			}

			// Switch modes if necessary (note that this will allow exit/enter on the same state)
			AdvanceToNextMode(SubTimeStep);
			OutputState.SyncState.MovementMode = CurrentModeName;

			for (TSharedPtr<FMovementModifierBase> Modifier : ActiveModifiers)
			{
				Modifier->OnPostMovement_Async(ModifierParams);
			}

			// Collect all tags from active mode, modifiers, and layered moves, then emit events for any changes
			{
				FGameplayTagContainer NewSimTags;
				CollectGameplayTags(OutputState.SyncState, NewSimTags);
				DiffAndEmitGameplayTagEvents(LastKnownGameplayTags, NewSimTags, SubTimeStep, /*bIsCausedByRollback*/ false);
				LastKnownGameplayTags = MoveTemp(NewSimTags);
			}

			if (const FChaosNetInstantMovementEffectsQueue* LocalInstantMovementEffectsQueue = Simulation->GetLocalSimInput().FindDataByType<FChaosNetInstantMovementEffectsQueue>())
			{
				ReceiveInstantMovementEffects(TimeStep, LocalInstantMovementEffectsQueue);
			}
			
			if (HasAnyInstantEffectsQueued(TimeStep))
			{
				// Apply any instant effects that were queued up during this tick and didn't get handled in a substep
				// Note: This won't apply any instant effects an extra time since we keep track of ID's we've already seen
				if (ApplyInstantEffects(WorkingSubstepStartData, TimeStep, OutputState.SyncState))
				{
					if (CurrentModeName != OutputState.SyncState.MovementMode)
					{
						SetModeImmediately(TimeStep, OutputState.SyncState.MovementMode);
					}
				}
			}
			
			const float RemainingMs = FMath::Clamp(OutputState.MovementEndState.RemainingMs, 0.0f, SubTimeStep.StepMs);
			const float SubstepUsedMs = (SubTimeStep.StepMs - RemainingMs);
			SubTimeStep.BaseSimTimeMs += SubstepUsedMs;
			TotalUsedMs += SubstepUsedMs;
			SubTimeStep.StepMs = RemainingMs;

			bIsWorkingStartStateReady = false;
			++SubStepCount;
		}

		InternalSimTime.BaseSimTimeMs = TimeStep.BaseSimTimeMs + TotalUsedMs;

		// We verify no effects are left that should have been applied
#if !UE_BUILD_SHIPPING
		if (const FChaosMoverSimulationDefaultInputs* SimInputs = Simulation->GetLocalSimInput().FindDataByType<FChaosMoverSimulationDefaultInputs>())
		{
			for (const FChaosScheduledInstantMovementEffect& QueuedEffect : QueuedInstantEffects)
			{
				ensureMsgf(!QueuedEffect.ScheduledEffect.SchedulingInfo.ShouldExecuteAtFrame(TimeStep.ServerFrame),
					TEXT("(%s) An Instant Movement Effect that should have been applied at ServerFrame %d was still in the queue after the simulation was ticked at ServerFrame %d. Effect: %s"),
					NETMODE_TO_STR(SimInputs),
					QueuedEffect.ScheduledEffect.SchedulingInfo.ServerExecutionTime.FrameCount,
					TimeStep.ServerFrame,
					EFFECTSHAREDPTR_TO_STR(QueuedEffect.ScheduledEffect.Effect));
			}
		}
#endif // !UE_BUILD_SHIPPING
	}

	static void LogPTRollbackTagDiff(
		const TWeakObjectPtr<UChaosMoverSimulation>& Simulation,
		const FMoverTimeStep& NewTimeStep,
		const FGameplayTagContainer& OldTags,
		const FGameplayTagContainer& NewSimTags)
	{
		if (MoverCVars::bEnableGameplayTagLog)
		{
			if (TStrongObjectPtr<UChaosMoverSimulation> DbgSim = Simulation.Pin())
			{
				MOVER_TAG_LOG(LogChaosMover, "[WT:RollbackTagDiff] Actor=%s Role=%s Frame=%d IsResim=%d IsRollback=%d | OldTags=[%s] AuthTags=[%s]",
					*DbgSim->DebugOwnerName, *DbgSim->DebugOwnerRole, NewTimeStep.ServerFrame, (int32)NewTimeStep.bIsResimulating, (int32)true, *OldTags.ToStringSimple(), *NewSimTags.ToStringSimple());
			}
		}
	}

	void FMoverStateMachine::OnSimulationRollback(const FMoverTimeStep& NewTimeStep, const FMoverSyncState& InvalidSyncState, const FMoverSyncState& NewSyncState)
	{
		QUICK_SCOPE_CYCLE_COUNTER(MoverStateMachine_OnSimulationRollback);

		InternalSimTime = NewTimeStep;

		ClearQueuedMode();

		if (CurrentModeName != NewSyncState.MovementMode)
		{
			QueueNextModeInternal(NewSyncState.MovementMode, /*bShouldReenter*/ false, /*bIsFromRollback*/ true);
		}

		RollBackQueuedLayeredMoves(NewTimeStep.ServerFrame);
		RollBackQueuedLayeredMoveInstances(NewTimeStep.ServerFrame);
		QueuedMovementModifiers.Empty();
		RollBackQueuedInstantMovementEffects(NewTimeStep.ServerFrame);
		InstantMovementEffectsIDHistory.Rollback(NewTimeStep.ServerFrame);
		LayeredMovesIDHistory.Rollback(NewTimeStep.ServerFrame);
		LayeredMoveInstancesIDHistory.Rollback(NewTimeStep.ServerFrame);

		FMovementModifierParams_Async ModifierParams(Simulation.Get(), &NewSyncState, &NewTimeStep);

		// Check if we have a new modifier in the rolled back sync state
		for (auto ModifierFromRollbackIt = NewSyncState.MovementModifiers.GetActiveModifiersIterator(); ModifierFromRollbackIt; ++ModifierFromRollbackIt)
		{
			bool bContainsModifier = false;
			for (auto ModifierFromCacheIt = InvalidSyncState.MovementModifiers.GetActiveModifiersIterator(); ModifierFromCacheIt; ++ModifierFromCacheIt)
			{
				if (ModifierFromRollbackIt->Get()->Matches(ModifierFromCacheIt->Get()))
				{
					bContainsModifier = true;

					// Rolled back version of the modifier will be missing the handle; we fix that here
					ModifierFromRollbackIt->Get()->OverwriteHandleIfInvalid(ModifierFromCacheIt->Get()->GetHandle());
					break;
				}
			}

			// If modifier is not already present start the new one
			if (!bContainsModifier)
			{
				UE_LOGF(LogChaosMover, Verbose, "Modifier(%ls) was started after a rollback.", *ModifierFromRollbackIt->Get()->ToSimpleString());
				ModifierFromRollbackIt->Get()->OnStart_Async(ModifierParams);
			}
		}

		// Check if the previous state has an active modifier not in the rolled back state
		for (auto ModifierFromCacheIt = InvalidSyncState.MovementModifiers.GetActiveModifiersIterator(); ModifierFromCacheIt; ++ModifierFromCacheIt)
		{
			bool bContainsModifier = false;
			for (auto ModifierFromRollbackIt = NewSyncState.MovementModifiers.GetActiveModifiersIterator(); ModifierFromRollbackIt; ++ModifierFromRollbackIt)
			{
				if (ModifierFromRollbackIt->Get()->Matches(ModifierFromCacheIt->Get()))
				{
					bContainsModifier = true;
					break;
				}
			}

			// If the modifier is not in the rolled back state end it
			if (!bContainsModifier)
			{
				UE_LOGF(LogChaosMover, Log, "Modifier(%ls) was ended after a rollback.", *ModifierFromCacheIt->Get()->ToSimpleString());
				ModifierFromCacheIt->Get()->OnEnd_Async(ModifierParams);
			}
		}

		// Collect tags from the rolled-back state and emit corrective tag events
		{
			FGameplayTagContainer NewSimTags;
			CollectGameplayTags(NewSyncState, NewSimTags);

			// Mover Gameplay Tag Logging
			LogPTRollbackTagDiff(Simulation, NewTimeStep, LastKnownGameplayTags, NewSimTags);

			DiffAndEmitGameplayTagEvents(LastKnownGameplayTags, NewSimTags, NewTimeStep, /*bIsCausedByRollback*/ true);
			LastKnownGameplayTags = MoveTemp(NewSimTags);
		}
	}

	void FMoverStateMachine::RollBackQueuedInstantMovementEffects(int32 NewServerFrame)
	{
		for (int EffectIndex = 0; EffectIndex < QueuedInstantEffects.Num();)
		{
			const FChaosScheduledInstantMovementEffect& QueuedEffect = QueuedInstantEffects[EffectIndex];
			// We roll back all effects which were issued on or after NewServerFrame, except those marked as not rolling back
			// (typically those coming from the game thread that is not being resimulated)
			if (QueuedEffect.ScheduledEffect.SchedulingInfo.ShouldRollBackAtFrame(NewServerFrame) && QueuedEffect.bShouldRollBack)
			{
#if !UE_BUILD_SHIPPING
				if (const FChaosMoverSimulationDefaultInputs* SimInputs = Simulation->GetLocalSimInput().FindDataByType<FChaosMoverSimulationDefaultInputs>())
				{
					UE_LOGF(LogChaosMover, Verbose, "(%ls) Rolling back Instant Movement Effect scheduled [%ls] at frame %d: %ls",
						NETMODE_TO_STR(SimInputs),
						*QueuedEffect.ScheduledEffect.SchedulingInfo.ToString(),
						NewServerFrame,
						EFFECTSHAREDPTR_TO_STR(QueuedEffect.ScheduledEffect.Effect));
				}
#endif // !UE_BUILD_SHIPPING

				QueuedInstantEffects.RemoveAt(EffectIndex);
			}
			else
			{
				EffectIndex++;
			}
		}
	}

	void FMoverStateMachine::RollBackQueuedLayeredMoves(int32 NewServerFrame)
	{
		for (int MoveIndex = 0; MoveIndex < QueuedLayeredMoves.Num();)
		{
			const FChaosScheduledLayeredMove& QueuedMove = QueuedLayeredMoves[MoveIndex];
			// We roll back all moves which were issued on or after NewServerFrame, except those marked as not rolling back
			// (typically those coming from the game thread that is not being resimulated)
			if (QueuedMove.ScheduledLayeredMove.SchedulingInfo.ShouldRollBackAtFrame(NewServerFrame) && QueuedMove.bShouldRollBack)
			{
#if !UE_BUILD_SHIPPING
				if (const FChaosMoverSimulationDefaultInputs* SimInputs = Simulation->GetLocalSimInput().FindDataByType<FChaosMoverSimulationDefaultInputs>())
				{
					UE_LOGF(LogChaosMover, Verbose, "(%ls) Rolling back Layered Move scheduled [%ls] at frame %d: %ls",
						NETMODE_TO_STR(SimInputs),
						*QueuedMove.ScheduledLayeredMove.SchedulingInfo.ToString(),
						NewServerFrame,
						MOVESHAREDPTR_TO_STR(QueuedMove.ScheduledLayeredMove.Move));
				}
#endif // !UE_BUILD_SHIPPING

				QueuedLayeredMoves.RemoveAt(MoveIndex);
			}
			else
			{
				MoveIndex++;
			}
		}
	}

	void FMoverStateMachine::RollBackQueuedLayeredMoveInstances(int32 NewServerFrame)
	{
		for (int32 MoveIndex = 0; MoveIndex < QueuedLayeredMoveInstances.Num(); )
		{
			const FChaosScheduledLayeredMoveInstance& QueuedMove = QueuedLayeredMoveInstances[MoveIndex];
			if (QueuedMove.SchedulingInfo.ShouldRollBackAtFrame(NewServerFrame) && QueuedMove.bShouldRollBack)
			{
#if !UE_BUILD_SHIPPING
				if (const FChaosMoverSimulationDefaultInputs* SimInputs = Simulation->GetLocalSimInput().FindDataByType<FChaosMoverSimulationDefaultInputs>())
				{
					UE_LOGF(LogChaosMover, Verbose, "(%ls) Rolling back Layered Move Instance scheduled [%ls] at frame %d",
						NETMODE_TO_STR(SimInputs),
						*QueuedMove.SchedulingInfo.ToString(),
						NewServerFrame);
				}
#endif // !UE_BUILD_SHIPPING

				QueuedLayeredMoveInstances.RemoveAt(MoveIndex);
			}
			else
			{
				MoveIndex++;
			}
		}
	}

	FMoverSimContext FMoverStateMachine::GetSimContext() const
	{
		FMoverSimContext SimContext;

		SimContext.Simulation = Simulation.Get();
		if (SimContext.Simulation)
		{
			SimContext.Blackboard = Simulation->GetRollbackBlackboardSimWrapper();
		}

		SimContext.SimulationOwner = nullptr;	// This crutch isn't needed for ChaosMover cases

		return SimContext;
	}

	const TWeakObjectPtr<UBaseMovementMode> FMoverStateMachine::GetCurrentMode() const
	{
		if (CurrentModeName != NAME_None && Modes.Contains(CurrentModeName))
		{
			return Modes[CurrentModeName];
		}

		return nullptr;
	}

	const TWeakObjectPtr<UBaseMovementMode> FMoverStateMachine::FindMovementMode(FName ModeName) const
	{
		if (ModeName != NAME_None && Modes.Contains(ModeName))
		{
			return Modes[ModeName];
		}

		return nullptr;
	}

	TWeakObjectPtr<UBaseMovementMode> FMoverStateMachine::FindMovementMode_Mutable(FName ModeName)
	{
		if (ModeName != NAME_None && Modes.Contains(ModeName))
		{
			return Modes[ModeName];
		}

		return nullptr;
	}

	void FMoverStateMachine::QueueLayeredMove(const FChaosScheduledLayeredMove& ScheduledMove)
	{
		Chaos::EnsureIsInPhysicsThreadContext();

		QueuedLayeredMoves.Add(ScheduledMove);

#if WITH_CHAOS_VISUAL_DEBUGGER
		TraceLayeredMove(ScheduledMove);
#endif
	}

	void FMoverStateMachine::QueueLayeredMoveInstance(const FChaosScheduledLayeredMoveInstance& ScheduledMove)
	{
		Chaos::EnsureIsInPhysicsThreadContext();

		QueuedLayeredMoveInstances.Add(ScheduledMove);

#if WITH_CHAOS_VISUAL_DEBUGGER
		TraceLayeredMoveInstance(ScheduledMove);
#endif
	}

	void FMoverStateMachine::QueueInstantMovementEffect(const FChaosScheduledInstantMovementEffect& ScheduledEffect)
	{
		Chaos::EnsureIsInPhysicsThreadContext();

		QueuedInstantEffects.Add(ScheduledEffect);

#if WITH_CHAOS_VISUAL_DEBUGGER
		TraceInstantMovementEffect(ScheduledEffect);
#endif
	}

	void FMoverStateMachine::ConstructDefaultModes()
	{
		RegisterMovementMode(UNullMovementMode::NullModeName, TObjectPtr<UBaseMovementMode>(NullMovementModeWeakPtr.Get()), /*bIsDefaultMode =*/ true);
		DefaultModeName = NAME_None;
		CurrentModeName = UNullMovementMode::NullModeName;

		QueuedModeTransitionWeakPtr = ImmediateMovementModeTransitionWeakPtr;

		ClearQueuedMode();
	}

	void FMoverStateMachine::AdvanceToNextMode(const FMoverTimeStep& TimeStep)
	{
		QUICK_SCOPE_CYCLE_COUNTER(MoverStateMachine_AdvanceToNextMode);

		TStrongObjectPtr<UImmediateMovementModeTransition> QueuedModeTransition = QueuedModeTransitionWeakPtr.Pin();
		if (QueuedModeTransition)
		{
			TStrongObjectPtr<UChaosMoverSimulation> SimStrongObjPtr = Simulation.Pin();
			if (!ensure(SimStrongObjPtr))
			{
				return;
			}

			const FName NextModeName = QueuedModeTransition->GetNextModeName();

			if (NextModeName != NAME_None)
			{
				TWeakObjectPtr<UBaseMovementMode>* FoundNextMovementMode = Modes.Find(NextModeName);
				if (FoundNextMovementMode)
				{
					const bool bShouldNextModeReenter = QueuedModeTransition->ShouldReenter();
					if ((CurrentModeName != NextModeName) || bShouldNextModeReenter)
					{
						UE_LOGF(LogChaosMover, Verbose, "AdvanceToNextMode: %ls (%ls) from %ls to %ls",
							*OwnerActorName, *UEnum::GetValueAsString(OwnerActorLocalNetRole), *CurrentModeName.ToString(), *NextModeName.ToString());

						const FName PreviousModeName = CurrentModeName;
						CurrentModeName = NextModeName;

						FMoverEventContext MoverEventContext;
						MoverEventContext.EventTimeMs = TimeStep.BaseSimTimeMs;
						MoverEventContext.bIsDuringResimulation = TimeStep.bIsResimulating;
						MoverEventContext.ServerFrame = TimeStep.ServerFrame;
						MoverEventContext.bIsCausedByRollback = QueuedModeTransition->IsCausedByRollback();

						FMoverSimContext SimContext = GetSimContext();

						if (PreviousModeName != NAME_None && Modes.Contains(PreviousModeName))
						{
							if (TStrongObjectPtr<UBaseMovementMode> PreviousMode = Modes[PreviousModeName].Pin())
							{
								PreviousMode->Deactivate(MoverEventContext, NextModeName, SimContext);
							}
						}

						// Track last mode change in the blackboard
						FRollbackBlackboardSimWrapper RollbackBlackboard = SimStrongObjPtr->GetRollbackBlackboardSimWrapper();
						FMovementModeChangeRecord ModeChangeRecord;
						ModeChangeRecord.ModeName = CurrentModeName;
						ModeChangeRecord.PrevModeName = PreviousModeName;
						ModeChangeRecord.Frame = InternalSimTime.ServerFrame;
						ModeChangeRecord.SimTimeMs = InternalSimTime.BaseSimTimeMs;

						RollbackBlackboard.TrySet(CommonBlackboard::LastModeChangeRecord, ModeChangeRecord);

						if (TStrongObjectPtr<UBaseMovementMode> CurrentMode = Modes[CurrentModeName].Pin())
						{	
							FMoverSyncState* InProgressSyncState = nullptr;
							FMoverAuxStateContext* InProgressAuxState = nullptr;

							if (WorkingSimTickEndData)
							{
								InProgressSyncState = &WorkingSimTickEndData->SyncState;
								InProgressAuxState = &WorkingSimTickEndData->AuxState;
							}

							CurrentMode->Activate(MoverEventContext, PreviousModeName, SimContext, WorkingSubstepStartData, InProgressSyncState, InProgressAuxState);
						}

						// Notify the simulation of a mode change so it can react accordingly
						TSharedPtr<FMovementModeChangedEventData> ModeChangedEventPtr = MakeShared<FMovementModeChangedEventData>(InternalSimTime, PreviousModeName, NextModeName);
						ModeChangedEventPtr->Context.bIsCausedByRollback = QueuedModeTransition->IsCausedByRollback();

						SimStrongObjPtr->AddEvent(ModeChangedEventPtr);
					}
				}
			}

			ClearQueuedMode();
		}
	}

	void FMoverStateMachine::FlushQueuedMovesToGroup(const FMoverTimeStep& TimeStep, FLayeredMoveGroup& Group)
	{
		for (int MoveIndex = 0; MoveIndex < QueuedLayeredMoves.Num(); )
		{
			const FScheduledLayeredMove& QueuedMove = QueuedLayeredMoves[MoveIndex].ScheduledLayeredMove;
			bool bShouldFlushMove = QueuedMove.SchedulingInfo.ShouldExecuteAtFrame(TimeStep.ServerFrame);

			if (bShouldFlushMove)
			{
				// First enqueue, then remove, to make sure any shared pointer ref doesn't go to 0 and accidentally delete underlying objects. Don't ask me how I know.
				Group.QueueLayeredMove(QueuedMove.Move);
				QueuedLayeredMoves.RemoveAt(MoveIndex);
			}
			else
			{
				MoveIndex++;
			}
		}
	}

	void FMoverStateMachine::FlushQueuedMoveInstancesToGroup(const FMoverTimeStep& TimeStep, FLayeredMoveInstanceGroup& Group)
	{
		for (int32 MoveIndex = 0; MoveIndex < QueuedLayeredMoveInstances.Num(); )
		{
			const FChaosScheduledLayeredMoveInstance& QueuedMove = QueuedLayeredMoveInstances[MoveIndex];
			if (QueuedMove.SchedulingInfo.ShouldExecuteAtFrame(TimeStep.ServerFrame))
			{
				// Re-link logic pointer if it was lost during network deserialization.
				if (QueuedMove.Move.IsValid() && !QueuedMove.Move->HasLogic())
				{
					QueuedMove.Move->PopulateMissingActiveMoveLogic(CachedRegisteredMoves);
				}
				// Enqueue before removing to keep the shared pointer ref count above zero.
				Group.QueueLayeredMove(QueuedMove.Move);
				QueuedLayeredMoveInstances.RemoveAt(MoveIndex);
			}
			else
			{
				MoveIndex++;
			}
		}
	}

	bool FMoverStateMachine::ApplyInstantEffects(const FMoverTickStartData& SubstepStartData, const FMoverTimeStep& TimeStep, FMoverSyncState& OutputState)
	{
		QUICK_SCOPE_CYCLE_COUNTER(MoverStateMachine_ApplyInstantEffects);

		FApplyMovementEffectParams_Async EffectParams_Async;
		EffectParams_Async.StartState = &SubstepStartData;
		EffectParams_Async.TimeStep = &TimeStep;
		EffectParams_Async.Simulation = Simulation.Get();

		bool bInstantMovementEffectApplied = false;

		for (int EffectIndex = 0; EffectIndex < QueuedInstantEffects.Num(); )
		{
			const FScheduledInstantMovementEffect& ScheduledEffect = QueuedInstantEffects[EffectIndex].ScheduledEffect;
			if (ScheduledEffect.SchedulingInfo.ShouldExecuteAtFrame(TimeStep.ServerFrame))
			{
				bInstantMovementEffectApplied |= ScheduledEffect.Effect.IsValid() && ScheduledEffect.Effect->ApplyMovementEffect_Async(EffectParams_Async, OutputState);

#if !UE_BUILD_SHIPPING
				if (const FChaosMoverSimulationDefaultInputs* SimInputs = Simulation->GetLocalSimInput().FindDataByType<FChaosMoverSimulationDefaultInputs>())
				{
					UE_LOGF(LogChaosMover, Verbose, "(%ls) Applying Instant Effect scheduled [%ls] at frame %d: %ls",
						NETMODE_TO_STR(SimInputs),
						*ScheduledEffect.SchedulingInfo.ToString(),
						TimeStep.ServerFrame,
						EFFECTSHAREDPTR_TO_STR(ScheduledEffect.Effect));
				}
#endif // !UE_BUILD_SHIPPING

				// We remove this effect since it has been applied
				QueuedInstantEffects.RemoveAt(EffectIndex);
			}
			else
			{
				EffectIndex++;
#if !UE_BUILD_SHIPPING
				if (const FChaosMoverSimulationDefaultInputs* SimInputs = Simulation->GetLocalSimInput().FindDataByType<FChaosMoverSimulationDefaultInputs>())
				{
					UE_LOGF(LogChaosMover, Verbose, "(%ls) SKIPPING Instant Effect scheduled [%ls] at frame %d: %ls",
						NETMODE_TO_STR(SimInputs),
						*ScheduledEffect.SchedulingInfo.ToString(),
						TimeStep.ServerFrame,
						EFFECTSHAREDPTR_TO_STR(ScheduledEffect.Effect));
				}
#endif // !UE_BUILD_SHIPPING
			}
		}

		return bInstantMovementEffectApplied;
	}

	bool FMoverStateMachine::UpdateWorkingStateFromSyncState(const FMoverTimeStep& TimeStep, FMoverSyncState& OutputState)
	{
		bool bModeSetFromInstantEffect = false;
	
		// Copying over our sync state collection to SubstepStartData so it is effectively the input sync state later for the movement mode. Doing this makes sure state modification from Instant Effects isn't overridden later by the movement mode
		for (auto SyncDataIt = OutputState.SyncStateCollection.GetCollectionDataIterator(); SyncDataIt; ++SyncDataIt)
		{
			if (SyncDataIt->Get())
			{
				WorkingSubstepStartData.SyncState.SyncStateCollection.AddDataByCopy(SyncDataIt->Get());
			}
		}
	
		if (CurrentModeName != OutputState.MovementMode)
		{
			bModeSetFromInstantEffect = true;
			SetModeImmediately(TimeStep, OutputState.MovementMode);
			WorkingSubstepStartData.SyncState.MovementMode = CurrentModeName;
		}
	
		return bModeSetFromInstantEffect;
	}
	
	void FMoverStateMachine::SetOwnerActorName(const FString& InOwnerActorName)
	{
		OwnerActorName = InOwnerActorName;
	}
	
	void FMoverStateMachine::SetOwnerActorLocalNetRole(ENetRole InOwnerActorLocalNetRole)
	{
		OwnerActorLocalNetRole = InOwnerActorLocalNetRole;
	}

	void FMoverStateMachine::FIDHistory::AddID(int32 Frame, uint8 ID)
	{
		TSet<uint8>& IDs = IDsByFrame.FindOrAdd(Frame);
		IDs.Add(ID);
	}

	void FMoverStateMachine::FIDHistory::CullOldFrames(int32 CurrentFrame, int32 MaxAge)
	{
		const int32 OldestFrameToKeep = CurrentFrame - MaxAge;

		TArray<int32, TInlineAllocator<8>> KeysToRemove;
		for (TMap<int32, TSet<uint8>>::TIterator It = IDsByFrame.CreateIterator(); It; ++It)
		{
			if (It.Key() < OldestFrameToKeep)
			{
				KeysToRemove.Add(It.Key());
			}
		}

		for (int32 KeyToRemove : KeysToRemove)
		{
			IDsByFrame.Remove(KeyToRemove);
		}
	}
	
	void FMoverStateMachine::FIDHistory::Rollback(int32 RollbackToFrameInclusive)
	{
		TArray<int32, TInlineAllocator<8>> KeysToRemove;
		for (TMap<int32, TSet<uint8>>::TIterator It = IDsByFrame.CreateIterator(); It; ++It)
		{
			if (It.Key() >= RollbackToFrameInclusive)
			{
				KeysToRemove.Add(It.Key());
			}
		}

		for (int32 KeyToRemove : KeysToRemove)
		{
			IDsByFrame.Remove(KeyToRemove);
		}
	}

	bool FMoverStateMachine::FIDHistory::WasIDAlreadySeen(uint8 ID) const
	{
		for (TMap<int32, TSet<uint8>>::TConstIterator It = IDsByFrame.CreateConstIterator(); It; ++It)
		{
			if (It.Value().Contains(ID))
			{
				return true;
			}
		}
		return false;
	}

#if WITH_CHAOS_VISUAL_DEBUGGER
	void FMoverStateMachine::CaptureUnprocessedSimCommandsForCVD()
	{
		// Record the list of sim commands (instant movement effects, layered moves, etc.) we ended up with at the start of OnSimulationTick
		// These could be a combination of commands we will process on the current frame or that are scheduled to be processed later.
		// The fact that we have scheduled commands that stay queued for a while is the reason we can't simply capture them for recording in QueueInstantMovementEffect, QueueLayeredMove, etc.
		// Taking effects as an example, some of these come from the FChaosNetInstantMovementEffectsQueue found in StartState.InputCmd.InputCollection (transferred in ReceiveInstantMovementEffects)
		// while others come from direct (internal) calls to QueueInstantMovementEffect on the simulation while it steps
		// Since we want to also record those added during OnSimulationTick after this point, we set a flag so any calls to QueueInstantMovementEffect
		// from now till post solve when we trace data to CVD are to be added to DebugInstantMovementEffectsQueue (same for moves, etc.)
		if (FChaosVisualDebuggerTrace::IsTracing())
		{
			if (TStrongObjectPtr<UChaosMoverSimulation> SimStrongObjPtr = Simulation.Pin())
			{
				FMoverDataCollection& DebugSimData = SimStrongObjPtr->GetDebugSimData();

				// Instant Movement Effects
				FChaosNetInstantMovementEffectsQueue& DebugInstantMovementEffectsQueue = DebugSimData.FindOrAddMutableDataByType<FChaosNetInstantMovementEffectsQueue>();
				DebugInstantMovementEffectsQueue = QueuedInstantEffects;
				
				// Layered Moves
				FChaosNetLayeredMovesQueue& DebugLayeredMovesQueue = DebugSimData.FindOrAddMutableDataByType<FChaosNetLayeredMovesQueue>();
				DebugLayeredMovesQueue = QueuedLayeredMoves;

				bCaptureSimCommandsForCVD = true;
			}
		}
	}

	void FMoverStateMachine::TraceInstantMovementEffect(const FChaosScheduledInstantMovementEffect& InScheduledEffect)
	{
		if (bCaptureSimCommandsForCVD && FChaosVisualDebuggerTrace::IsTracing())
		{
			if (TStrongObjectPtr<UChaosMoverSimulation> SimStrongObjPtr = Simulation.Pin())
			{
				FChaosNetInstantMovementEffectsQueue& DebugInstantMovementEffectsQueue = SimStrongObjPtr->GetDebugSimData().FindOrAddMutableDataByType<FChaosNetInstantMovementEffectsQueue>();
				DebugInstantMovementEffectsQueue.Add(InScheduledEffect.ScheduledEffect, InScheduledEffect.bShouldRollBack, 0xFF);
			}
		}
	}

	void FMoverStateMachine::TraceLayeredMove(const FChaosScheduledLayeredMove& InScheduledMove)
	{
		if (bCaptureSimCommandsForCVD && FChaosVisualDebuggerTrace::IsTracing())
		{
			if (TStrongObjectPtr<UChaosMoverSimulation> SimStrongObjPtr = Simulation.Pin())
			{
				FChaosNetLayeredMovesQueue& DebugLayeredMovesQueue = SimStrongObjPtr->GetDebugSimData().FindOrAddMutableDataByType<FChaosNetLayeredMovesQueue>();
				DebugLayeredMovesQueue.Add(InScheduledMove.ScheduledLayeredMove, InScheduledMove.bShouldRollBack, 0xFF);
			}
		}
	}

	void FMoverStateMachine::TraceLayeredMoveInstance(const FChaosScheduledLayeredMoveInstance& InScheduledMove)
	{
		if (bCaptureSimCommandsForCVD && FChaosVisualDebuggerTrace::IsTracing())
		{
			if (TStrongObjectPtr<UChaosMoverSimulation> SimStrongObjPtr = Simulation.Pin())
			{
				FChaosNetLayeredMoveInstancesQueue& DebugLayeredMoveInstancesQueue = SimStrongObjPtr->GetDebugSimData().FindOrAddMutableDataByType<FChaosNetLayeredMoveInstancesQueue>();
				DebugLayeredMoveInstancesQueue.Add(InScheduledMove, InScheduledMove.bShouldRollBack, 0xFF);
			}
		}
	}

	void FMoverStateMachine::OnEndTraceMoverData()
	{
		bCaptureSimCommandsForCVD = false;
	}
#endif

} // End of namespace UE::ChaosMover