// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosMover/Character/ChaosCharacterInputs.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosCharacterInputs)

bool FChaosMoverLaunchInputs::NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess)
{
	Super::NetSerialize(Ar, Map, bOutSuccess);

	Ar << LaunchVelocityOrImpulse;
	Ar << Mode;

	bOutSuccess = true;
	return true;
}

void FChaosMoverLaunchInputs::ToString(FAnsiStringBuilderBase& Out) const
{
	Super::ToString(Out);

	Out.Appendf("LaunchVelocityOrImpulse: X=%.2f Y=%.2f Z=%.2f |", LaunchVelocityOrImpulse.X, LaunchVelocityOrImpulse.Y, LaunchVelocityOrImpulse.Z);
	Out.Appendf("Mode: %u\n", uint32{EnumToUnderlyingType(Mode)});
}

bool FChaosMoverLaunchInputs::ShouldReconcile(const FMoverDataStructBase& AuthorityState) const
{
	const FChaosMoverLaunchInputs& TypedAuthority = static_cast<const FChaosMoverLaunchInputs&>(AuthorityState);
	return Mode != TypedAuthority.Mode || LaunchVelocityOrImpulse != TypedAuthority.LaunchVelocityOrImpulse;
}

void FChaosMoverLaunchInputs::Interpolate(const FMoverDataStructBase& From, const FMoverDataStructBase& To, float Pct)
{
	if (Pct < .5f)
	{
		*this = static_cast<const FChaosMoverLaunchInputs&>(From);
	}
	else
	{
		*this = static_cast<const FChaosMoverLaunchInputs&>(To);
	}
}

void FChaosMoverLaunchInputs::Merge(const FMoverDataStructBase& From)
{

}

bool FChaosMoverCrouchInputs::NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess)
{
	Super::NetSerialize(Ar, Map, bOutSuccess);

	Ar << bWantsToCrouch;

	bOutSuccess = true;
	return true;
}

void FChaosMoverCrouchInputs::ToString(FAnsiStringBuilderBase& Out) const
{
	Super::ToString(Out);

	Out.Appendf("bWantsToCrouch: %hs \n", *LexToString(bWantsToCrouch));
}

bool FChaosMoverCrouchInputs::ShouldReconcile(const FMoverDataStructBase& AuthorityState) const
{
	const FChaosMoverCrouchInputs& TypedAuthority = static_cast<const FChaosMoverCrouchInputs&>(AuthorityState);
	return bWantsToCrouch != TypedAuthority.bWantsToCrouch;
}

void FChaosMoverCrouchInputs::Interpolate(const FMoverDataStructBase& From, const FMoverDataStructBase& To, float Pct)
{
	if (Pct < .5f)
	{
		*this = static_cast<const FChaosMoverCrouchInputs&>(From);
	}
	else
	{
		*this = static_cast<const FChaosMoverCrouchInputs&>(To);
	}
}

void FChaosMoverCrouchInputs::Merge(const FMoverDataStructBase& From)
{
	const FChaosMoverCrouchInputs& TypedFrom = static_cast<const FChaosMoverCrouchInputs&>(From);
	bWantsToCrouch |= TypedFrom.bWantsToCrouch;
}

bool FChaosMovementSettingsOverrides::NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess)
{
	Super::NetSerialize(Ar, Map, bOutSuccess);

	Ar << ModeName;
	Ar << MaxSpeedOverride;
	Ar << AccelerationOverride;

	bOutSuccess = true;
	return true;
}

void FChaosMovementSettingsOverrides::ToString(FAnsiStringBuilderBase& Out) const
{
	Super::ToString(Out);

	Out.Appendf("OverrideModeName: %s |", *ModeName.ToString());
	Out.Appendf("MaxSpeedOverride: %.2f |", MaxSpeedOverride);
	Out.Appendf("MaxSpeedOverride: %.2f \n", AccelerationOverride);
}

bool FChaosMovementSettingsOverrides::ShouldReconcile(const FMoverDataStructBase& AuthorityState) const
{
	const FChaosMovementSettingsOverrides& TypedAuthority = static_cast<const FChaosMovementSettingsOverrides&>(AuthorityState);
	return ModeName != TypedAuthority.ModeName || !FMath::IsNearlyEqual(MaxSpeedOverride, TypedAuthority.MaxSpeedOverride) || !FMath::IsNearlyEqual(AccelerationOverride, TypedAuthority.AccelerationOverride);
}

void FChaosMovementSettingsOverrides::Interpolate(const FMoverDataStructBase& From, const FMoverDataStructBase& To, float Pct)
{
	const FChaosMovementSettingsOverrides& TypedFrom = static_cast<const FChaosMovementSettingsOverrides&>(From);
	const FChaosMovementSettingsOverrides& TypedTo = static_cast<const FChaosMovementSettingsOverrides&>(To);
	
	ModeName = TypedTo.ModeName;
	if (TypedFrom.ModeName == TypedTo.ModeName)
	{
		MaxSpeedOverride = FMath::Lerp(TypedFrom.MaxSpeedOverride, TypedTo.MaxSpeedOverride, Pct);
		AccelerationOverride = FMath::Lerp(TypedFrom.AccelerationOverride, TypedTo.AccelerationOverride, Pct);
	}
	else
	{
		MaxSpeedOverride = TypedTo.MaxSpeedOverride;
		AccelerationOverride = TypedTo.AccelerationOverride;
	}

}

void FChaosMovementSettingsOverrides::Merge(const FMoverDataStructBase& From)
{
}

bool FChaosMovementSettingsOverridesRemover::NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess)
{
	Super::NetSerialize(Ar, Map, bOutSuccess);

	Ar << ModeName;

	bOutSuccess = true;
	return true;
}

void FChaosMovementSettingsOverridesRemover::ToString(FAnsiStringBuilderBase& Out) const
{
	Super::ToString(Out);

	Out.Appendf("eModeName: %s |", *ModeName.ToString());
}

bool FChaosMovementSettingsOverridesRemover::ShouldReconcile(const FMoverDataStructBase& AuthorityState) const
{
	const FChaosMovementSettingsOverridesRemover& TypedAuthority = static_cast<const FChaosMovementSettingsOverridesRemover&>(AuthorityState);
	return ModeName != TypedAuthority.ModeName;
}

void FChaosMovementSettingsOverridesRemover::Interpolate(const FMoverDataStructBase& From, const FMoverDataStructBase& To, float Pct)
{
	const FChaosMovementSettingsOverridesRemover& TypedTo = static_cast<const FChaosMovementSettingsOverridesRemover&>(To);
	ModeName = TypedTo.ModeName;
}

void FChaosMovementSettingsOverridesRemover::Merge(const FMoverDataStructBase& From)
{
}

bool FChaosMoverRequestedMoveInputs::NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess)
{
	Super::NetSerialize(Ar, Map, bOutSuccess);

	Ar << RequestedVelocity;
	Ar << bForceMaxSpeed;
	Ar << bUseAcceleration;

	bOutSuccess = true;
	return true;
}

void FChaosMoverRequestedMoveInputs::ToString(FAnsiStringBuilderBase& Out) const
{
	Super::ToString(Out);

	Out.Appendf("RequestedVelocity: X=%.2f Y=%.2f Z=%.2f |", RequestedVelocity.X, RequestedVelocity.Y, RequestedVelocity.Z);
	Out.Appendf("bForceMaxSpeed: %u |", bForceMaxSpeed);
	Out.Appendf("bUseAcceleration: %u\n", bUseAcceleration);
}

bool FChaosMoverRequestedMoveInputs::ShouldReconcile(const FMoverDataStructBase& AuthorityState) const
{
	const FChaosMoverRequestedMoveInputs& TypedAuthority = static_cast<const FChaosMoverRequestedMoveInputs&>(AuthorityState);
	return RequestedVelocity != TypedAuthority.RequestedVelocity;
}

void FChaosMoverRequestedMoveInputs::Interpolate(const FMoverDataStructBase& From, const FMoverDataStructBase& To, float Pct)
{
	const FChaosMoverRequestedMoveInputs& TypedFrom = static_cast<const FChaosMoverRequestedMoveInputs&>(From);
	const FChaosMoverRequestedMoveInputs& TypedTo = static_cast<const FChaosMoverRequestedMoveInputs&>(To);
	bForceMaxSpeed = Pct < 0.5f ? TypedFrom.bForceMaxSpeed : TypedTo.bForceMaxSpeed;
	bUseAcceleration = Pct < 0.5f ? TypedFrom.bUseAcceleration : TypedTo.bUseAcceleration;
	RequestedVelocity = FMath::Lerp(TypedFrom.RequestedVelocity, TypedTo.RequestedVelocity, Pct);
}

void FChaosMoverRequestedMoveInputs::Merge(const FMoverDataStructBase& From)
{
}
