// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosMover/Character/ChaosCharacterMoverComponent.h"

#include "ChaosMover/Backends/ChaosMoverBackend.h"
#include "ChaosMover/Character/ChaosCharacterInputs.h"
#include "ChaosMover/Character/Modes/ChaosFallingMode.h"
#include "ChaosMover/Character/Modes/ChaosFlyingMode.h"
#include "ChaosMover/Character/Modes/ChaosWalkingMode.h"
#include "ChaosMover/ChaosMoverLog.h"
#include "ChaosMover/ChaosMoverSimulationTypes.h"
#include "DefaultMovementSet/CharacterMoverSimulationTypes.h"
#include "GameFramework/Pawn.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosCharacterMoverComponent)

UChaosCharacterMoverComponent::UChaosCharacterMoverComponent()
{
	// Default movement modes
	MovementModes.Add(DefaultModeNames::Walking, CreateDefaultSubobject<UChaosWalkingMode>(TEXT("DefaultChaosWalkingMode")));
	MovementModes.Add(DefaultModeNames::Falling, CreateDefaultSubobject<UChaosFallingMode>(TEXT("DefaultChaosFallingMode")));
	MovementModes.Add(DefaultModeNames::Flying, CreateDefaultSubobject<UChaosFlyingMode>(TEXT("DefaultChaosFlyingMode")));

	StartingMovementMode = DefaultModeNames::Falling;

	bHandleJump = false;
	bHandleStanceChanges = false;

	BackendClass = UChaosMoverBackendComponent::StaticClass();
}

TArray<FTrajectorySampleInfo> UChaosCharacterMoverComponent::GetPredictedTrajectory(FMoverPredictTrajectoryParams PredictionParams)
{
	const int32 NumSamples = FMath::Max(1, PredictionParams.NumPredictionSamples);
	const float DtSeconds = FMath::Max(0.001f, PredictionParams.SecondsPerSample);
	const TArray<FTrajectorySampleInfo>& CachedDeltas = LatestPredictedTrajectoryData.Deltas;

	SetPredictedTrajectoryParams(NumSamples - 1, DtSeconds, /*bInEnableTrajectoryPrediction*/ true);

	TArray<FTrajectorySampleInfo> Out;

	if (!LastMoverDefaultSyncState)
	{
		Out.AddDefaulted(NumSamples);
	}
	else
	{
		FTrajectorySampleInfo BaseSample;
		const FVector AngVelDeg = LastMoverDefaultSyncState->GetAngularVelocityDegrees_WorldSpace();
		BaseSample.Transform = LastMoverDefaultSyncState->GetTransform_WorldSpace();
		BaseSample.LinearVelocity = LastMoverDefaultSyncState->GetVelocity_WorldSpace();
		BaseSample.AngularVelocity = FRotator(AngVelDeg.X, AngVelDeg.Y, AngVelDeg.Z);
		BaseSample.InstantaneousAcceleration = FVector::ZeroVector;
		BaseSample.SimTimeMs = GetLastTimeStep().BaseSimTimeMs;

		if (CachedDeltas.Num() < 2)
		{
			Out.SetNumUninitialized(NumSamples);
			for (int32 i = 0; i < NumSamples; ++i)
			{
				Out[i] = BaseSample;
				Out[i].SimTimeMs = BaseSample.SimTimeMs + static_cast<double>(i) * static_cast<double>(DtSeconds) * 1000.0;
			}
		}
		else
		{
			const int32 NumDeltaSamples = CachedDeltas.Num();
			Out.SetNumUninitialized(NumDeltaSamples + 1);
			Out[0] = BaseSample;

			Out[1] = CachedDeltas[0];
			Out[1].Transform = LatestPredictedTrajectoryData.BaseTransform;
			for (int32 i = 1; i < NumDeltaSamples; ++i)
			{
				Out[i + 1] = CachedDeltas[i];
				Out[i + 1].Transform = CachedDeltas[i].Transform * Out[i].Transform;
			}
		}
	}

	if (PredictionParams.bUseVisualComponentRoot)
	{
		if (const USceneComponent* VisualComp = GetPrimaryVisualComponent())
		{
			const FTransform VisualRelativeTransform = VisualComp->GetRelativeTransform();
			for (FTrajectorySampleInfo& Sample : Out)
			{
				Sample.Transform = VisualRelativeTransform * Sample.Transform;
			}
		}
	}

	return Out;
}

void UChaosCharacterMoverComponent::SetPredictedTrajectoryParams(int32 InSteps, float InStepSeconds, bool bInEnableTrajectoryPrediction)
{
	PredictedTrajectorySteps = FMath::Max(1, InSteps);
	PredictedTrajectoryStepSeconds = FMath::Max(0.001f, InStepSeconds);
	bEnableTrajectoryPrediction = bInEnableTrajectoryPrediction;
}

void UChaosCharacterMoverComponent::GetPredictedTrajectoryParams(int32& OutSteps, float& OutStepSeconds, bool& bIsTrajectoryPredictionEnabled) const
{
	OutSteps = PredictedTrajectorySteps;
	OutStepSeconds = PredictedTrajectoryStepSeconds;
	bIsTrajectoryPredictionEnabled = bEnableTrajectoryPrediction;
}

void UChaosCharacterMoverComponent::ProduceLocalInput(FMoverDataCollection& OutLocalSimInput) const
{
	if (bEnableTrajectoryPrediction)
	{
		FChaosMoverTrajectoryPredictionInputs& PredInputs = OutLocalSimInput.FindOrAddMutableDataByType<FChaosMoverTrajectoryPredictionInputs>();
		bool bUnused;
		GetPredictedTrajectoryParams(PredInputs.NumPredictionSteps, PredInputs.SecondsPerStep, bUnused);
	}
}

void UChaosCharacterMoverComponent::OnMoverPreSimulationTick(const FMoverTimeStep& TimeStep, const FMoverInputCmdContext& InputCmd)
{
	// This overrides the base class to make sure that it doesn't do anything. We do this
	// because the stance modifier and jump handling of the base is not async physics friendly
}

void UChaosCharacterMoverComponent::DispatchSimulationEvents(const UE::Mover::FSimulationOutputData& OutputData)
{
	Super::DispatchSimulationEvents(OutputData);

	// Flush the stance batch accumulated during this frame's event dispatch.
	// If rollback/resim produced transient stance transitions (e.g. uncrouch then recrouch),
	// only the net final stance is dispatched, suppressing spurious visual/animation callbacks.
	if (PendingBatchFinalStance.IsSet())
	{
		const EStanceMode FinalStance = PendingBatchFinalStance.GetValue();
		if (FinalStance != LastDispatchedStance)
		{
			OnStanceChanged.Broadcast(LastDispatchedStance, FinalStance);
			LastDispatchedStance = FinalStance;
		}
		PendingBatchFinalStance.Reset();
	}
}

void UChaosCharacterMoverComponent::ProcessSimulationEvent(const FMoverSimulationEventData& EventData)
{
	Super::ProcessSimulationEvent(EventData);

	if (const FLandedEventData* LandedData = EventData.CastTo<FLandedEventData>())
	{
		NotifyOnLanded(*LandedData);
	}
	if (const FJumpedEventData* JumpedData = EventData.CastTo<FJumpedEventData>())
	{
		OnJumped.Broadcast(JumpedData->JumpStartHeight);
	}
	if (const FStanceModifiedEventData* StanceModifiedEvent = EventData.CastTo<FStanceModifiedEventData>())
	{
		// Accumulate - dispatch only the net final stance at end of batch (see DispatchSimulationEvents).
		// This prevents rollback/resim transient transitions (e.g. forced uncrouch then recrouch) from
		// causing spurious visual/animation responses on the game thread.
		PendingBatchFinalStance = StanceModifiedEvent->NewStance;
	}
}

void UChaosCharacterMoverComponent::DoQueueNextMode(FName DesiredModeName, bool bShouldReenter)
{
	Super::DoQueueNextMode(DesiredModeName, bShouldReenter);

	if (QueuedModeTransitionName != NAME_None)
	{ 
		const FName NextModeName = QueuedModeTransitionName;
		if (NextModeName != NAME_None)
		{
			UE_LOGF(LogChaosMover, Log, "%ls (%ls) Overwriting of queued mode change (%ls) with (%ls)", *GetNameSafe(GetOwner()), *UEnum::GetValueAsString(GetOwner()->GetLocalRole()), *NextModeName.ToString(), *DesiredModeName.ToString());
		}
	}

	QueuedModeTransitionName = DesiredModeName;
}

FName UChaosCharacterMoverComponent::GetNextMovementModeName() const
{
	if (QueuedModeTransitionName != NAME_None)
	{
		return QueuedModeTransitionName;
	}

	return GetMovementModeName();
}

void UChaosCharacterMoverComponent::ClearQueuedMode()
{
	QueuedModeTransitionName = NAME_None;
}

void UChaosCharacterMoverComponent::SetAdditionalSimulationOutput(const FMoverDataCollection& Data)
{
	Super::SetAdditionalSimulationOutput(Data);

	if (const FFloorResultData* FloorData = Data.FindDataByType<FFloorResultData>())
	{
		bFloorResultSet = true;
		LatestFloorResult = FloorData->FloorResult;
	}

	if (const FChaosWaterResultData* WaterData = Data.FindDataByType<FChaosWaterResultData>())
	{
		bWaterResultSet = true;
		LatestWaterResult = WaterData->WaterResult;
	}

	if (const FChaosMoverPredictedTrajectoryData* PredictedTraj = Data.FindDataByType<FChaosMoverPredictedTrajectoryData>())
	{
		bPredictedTrajectorySet = true;
		LatestPredictedTrajectoryData = *PredictedTraj;
	}
}

bool UChaosCharacterMoverComponent::TryGetLastWaterResult(FWaterCheckResult& OutWaterResult) const
{
	if (bWaterResultSet)
	{
		OutWaterResult = LatestWaterResult;
		return true;
	}
	return false;
}

bool UChaosCharacterMoverComponent::TryGetFloorCheckHitResult(FHitResult& OutHitResult) const
{
	if (bFloorResultSet)
	{
		OutHitResult = LatestFloorResult.HitResult;
		return true;
	}
	else
	{
		return Super::TryGetFloorCheckHitResult(OutHitResult);
	}
}

void UChaosCharacterMoverComponent::ProduceInput(const int32 DeltaTimeMS, FMoverInputCmdContext* Cmd)
{
	ProduceInputImpl(DeltaTimeMS, Cmd);
}

void UChaosCharacterMoverComponent::ProduceServerInput(const int32 DeltaTimeMS, FMoverInputCmdContext* Cmd)
{
	ProduceInputImpl(DeltaTimeMS, Cmd);
}

void UChaosCharacterMoverComponent::ProduceInputImpl(const int32 DeltaTimeMS, FMoverInputCmdContext* Cmd)
{
	if (!ensure(Cmd))
	{
		return;
	}

	Super::ProduceInput(DeltaTimeMS, Cmd);

	// Add launch input data if launch velocity is set
	if (!LaunchVelocityOrImpulse.IsZero())
	{
		FChaosMoverLaunchInputs& LaunchInputs = Cmd->InputCollection.FindOrAddMutableDataByType<FChaosMoverLaunchInputs>();
		LaunchInputs.LaunchVelocityOrImpulse = LaunchVelocityOrImpulse;
		LaunchInputs.Mode = LaunchMode;

		LaunchVelocityOrImpulse = FVector::ZeroVector;
	}

	if (QueuedModeTransitionName != NAME_None)
	{
		FCharacterDefaultInputs& CharacterDefaultInputs = Cmd->InputCollection.FindOrAddMutableDataByType<FCharacterDefaultInputs>();
		CharacterDefaultInputs.SuggestedMovementMode = QueuedModeTransitionName;

		ClearQueuedMode();
	}

	// Always propagate the current held state so resim frames can reconstruct intent.
	{
		FChaosMoverCrouchInputs& CrouchInputs = Cmd->InputCollection.FindOrAddMutableDataByType<FChaosMoverCrouchInputs>();
		CrouchInputs.bWantsToCrouch = bWantsToCrouch;
	}

	if (bCancelMovementOverrides)
	{
		FChaosMovementSettingsOverridesRemover& MovementSettingsRemover = Cmd->InputCollection.FindOrAddMutableDataByType<FChaosMovementSettingsOverridesRemover>();
		MovementSettingsRemover.ModeName = ModeToOverrideSettings;

		bCancelMovementOverrides = false;
	}

	if (bOverrideMovementSettings)
	{
		FChaosMovementSettingsOverrides& MovementSettings = Cmd->InputCollection.FindOrAddMutableDataByType<FChaosMovementSettingsOverrides>();
		MovementSettings.MaxSpeedOverride = MaxSpeedOverride;
		MovementSettings.AccelerationOverride = AccelerationOverride;
		MovementSettings.ModeName = ModeToOverrideSettings;

		bOverrideMovementSettings = false;
	}
}

void UChaosCharacterMoverComponent::Launch(const FVector& VelocityOrImpulse, EChaosMoverVelocityEffectMode Mode)
{
	LaunchVelocityOrImpulse = VelocityOrImpulse;
	LaunchMode = Mode;
}


void UChaosCharacterMoverComponent::NotifyOnLanded(const FLandedEventData& LandedEvent)
{
	OnLanded(LandedEvent);
	OnLandedDelegate.Broadcast(this, LandedEvent);
}

void UChaosCharacterMoverComponent::OnLanded(const FLandedEventData& LandedEvent)
{
	
}

void UChaosCharacterMoverComponent::OverrideMovementSettings(const FChaosMovementSettingsOverrides Overrides)
{
	ModeToOverrideSettings = Overrides.ModeName;
	MaxSpeedOverride = FMath::Max(Overrides.MaxSpeedOverride, 0.0f);
	AccelerationOverride = FMath::Max(Overrides.AccelerationOverride, 0.0f);
	bOverrideMovementSettings = true;
}

void UChaosCharacterMoverComponent::CancelMovementSettingsOverrides(FName ModeName)
{
	ModeToOverrideSettings = ModeName;
	bCancelMovementOverrides = true;
}
