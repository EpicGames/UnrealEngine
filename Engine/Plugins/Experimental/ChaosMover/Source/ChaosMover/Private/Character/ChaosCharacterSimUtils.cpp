// Copyright Epic Games, Inc. All Rights Reserved.

#include "Character/ChaosCharacterSimUtils.h"

#include "Chaos/Character/CharacterGroundConstraint.h"
#include "ChaosMover/Character/ChaosCharacterInputs.h"
#include "ChaosMover/ChaosMoverConsoleVariables.h"
#include "Chaos/Character/CharacterGroundConstraintContainer.h"
#include "Chaos/Character/CharacterGroundConstraintSettings.h"
#include "Chaos/PhysicsObjectInternalInterface.h"
#include "ChaosMover/ChaosMoverSimulation.h"
#include "ChaosMover/ChaosMoverSimulationTypes.h"
#include "Components/PrimitiveComponent.h"
#include "DefaultMovementSet/CharacterMoverSimulationTypes.h"
#include "MoveLibrary/FloorQueryUtils.h"
#include "MoveLibrary/MoverBlackboard.h"
#include "MoveLibrary/WaterMovementUtils.h"
#include "MoverTypes.h"

// ---------------------------------------------------------------------------
// ProcessCharacterInputs
// ---------------------------------------------------------------------------

void UE::ChaosMover::ProcessCharacterInputs(FChaosMoverPreSimContext& Context)
{
	UChaosMoverSimulation* ChaosMoverSim = Cast<UChaosMoverSimulation>(Context.Simulation);
	if (!ChaosMoverSim)
	{
		return;
	}

	// Ensure character inputs exist and have a valid move type.
	FCharacterDefaultInputs& CharacterDefaultInputs =
		Context.InputData.InputCmd.InputCollection.FindOrAddMutableDataByType<FCharacterDefaultInputs>();
	if (CharacterDefaultInputs.GetMoveInputType() == EMoveInputType::Invalid)
	{
		CharacterDefaultInputs.SetMoveInput(EMoveInputType::DirectionalIntent, FVector::ZeroVector);
	}

	// Consume any requested mode transition from the input.
	if (!CharacterDefaultInputs.SuggestedMovementMode.IsNone())
	{
		ChaosMoverSim->QueueNextMovementMode(CharacterDefaultInputs.SuggestedMovementMode);
		CharacterDefaultInputs.SuggestedMovementMode = NAME_None;
	}

	// Apply per-mode speed/acceleration overrides from the input.
	if (const FChaosMovementSettingsOverrides* MovementSettingsOverride =
			Context.InputData.InputCmd.InputCollection.FindDataByType<FChaosMovementSettingsOverrides>())
	{
		TWeakObjectPtr<UBaseMovementMode> ModePtr = !MovementSettingsOverride->ModeName.IsNone()
			? ChaosMoverSim->FindMovementModeByName_Mutable(MovementSettingsOverride->ModeName)
			: ChaosMoverSim->GetCurrentMovementMode_Mutable();

		if (IChaosCharacterMovementModeInterface* ModeInterface = Cast<IChaosCharacterMovementModeInterface>(ModePtr.Get()))
		{
			ModeInterface->OverrideMaxSpeed(MovementSettingsOverride->MaxSpeedOverride);
			ModeInterface->OverrideAcceleration(MovementSettingsOverride->AccelerationOverride);
		}
	}

	if (const FChaosMovementSettingsOverridesRemover* Remover =
			Context.InputData.InputCmd.InputCollection.FindDataByType<FChaosMovementSettingsOverridesRemover>())
	{
		TWeakObjectPtr<UBaseMovementMode> ModePtr = !Remover->ModeName.IsNone()
			? ChaosMoverSim->FindMovementModeByName_Mutable(Remover->ModeName)
			: ChaosMoverSim->GetCurrentMovementMode_Mutable();

		if (IChaosCharacterMovementModeInterface* ModeInterface = Cast<IChaosCharacterMovementModeInterface>(ModePtr.Get()))
		{
			ModeInterface->ClearMaxSpeedOverride();
			ModeInterface->ClearAccelerationOverride();
		}
	}
}

// ---------------------------------------------------------------------------
// UpdateCurrentFloor
// ---------------------------------------------------------------------------

void UE::ChaosMover::UpdateCurrentFloor(FChaosMoverPreSimContext& Context, IChaosCharacterMovementModeInterface& CharacterInterface)
{
	// bEnablePreSimGroundCheck OFF allows saving performance by reusing last frame's
	// result, but the blackboard is invalidated on rollback so the first resim frame
	// must always refresh.
	if (UE::ChaosMover::CVars::bEnablePreSimGroundCheck ||
		(Context.TimeStep.bIsResimulating && Context.TimeStep.bIsFirstResimFrame))
	{
		CharacterInterface.UpdateCurrentFloor(Context.TimeStep);
	}
}

// ---------------------------------------------------------------------------
// ReconcileParticleVelocity
// ---------------------------------------------------------------------------

void UE::ChaosMover::ReconcileParticleVelocity(FChaosMoverPostSimContext& Context, IChaosCharacterMovementModeInterface* CharacterInterface)
{
	FMoverDefaultSyncState* PostSimDefaultSyncState = Context.OutputData.SyncState.SyncStateCollection.FindMutableDataByType<FMoverDefaultSyncState>();
	check(PostSimDefaultSyncState);

	UChaosMoverSimulation* ChaosMoverSim = Cast<UChaosMoverSimulation>(Context.Simulation);
	if (!ChaosMoverSim)
	{
		return;
	}

	Chaos::FPBDRigidParticleHandle* ParticleHandle = ChaosMoverSim->GetControlledParticle();
	if (!ParticleHandle)
	{
		return;
	}

	if (!Context.TimeStep.bIsResimulating && ChaosMoverSim->IsNonResimSimProxy())
	{
		const float DeltaSeconds = Context.TimeStep.StepMs * 0.001f;
		const FVector ExtrapolatedPos = ParticleHandle->GetX() + ParticleHandle->GetV() * DeltaSeconds;
		const Chaos::FRotation3 IntegratedRot = Chaos::FRotation3::IntegrateRotationWithAngularVelocity(
			ParticleHandle->GetR(),
			ParticleHandle->GetW(),
			DeltaSeconds);
		const FRotator ExtrapolatedRot(IntegratedRot);
		const FVector ParticleV = ParticleHandle->GetV();
		const FVector ParticleAngVelDeg = FMath::RadiansToDegrees(ParticleHandle->GetW());
		PostSimDefaultSyncState->SetTransforms_WorldSpace(ExtrapolatedPos, ExtrapolatedRot, ParticleV, ParticleAngVelDeg);
	}
	else
	{
		ParticleHandle->SetV(PostSimDefaultSyncState->GetVelocity_WorldSpace());
		if (CharacterInterface && CharacterInterface->ShouldCharacterRemainUpright())
		{
			ParticleHandle->SetW(FMath::DegreesToRadians(PostSimDefaultSyncState->GetAngularVelocityDegrees_WorldSpace()));
		}
		else
		{
			// Physics owns angular velocity; reflect it back so sync state stays consistent.
			PostSimDefaultSyncState->SetTransforms_WorldSpace(
				PostSimDefaultSyncState->GetLocation_WorldSpace(),
				PostSimDefaultSyncState->GetOrientation_WorldSpace(),
				PostSimDefaultSyncState->GetVelocity_WorldSpace(),
				FMath::RadiansToDegrees(ParticleHandle->GetW()));
		}
	}
}

// ---------------------------------------------------------------------------
// ApplyGroundAndWaterResults
// ---------------------------------------------------------------------------

void UE::ChaosMover::ApplyGroundAndWaterResults(FChaosMoverPostSimContext& Context, IChaosCharacterMovementModeInterface* CharacterInterface)
{
	if (!CharacterInterface)
	{
		return;
	}

	FMoverDefaultSyncState* PostSimDefaultSyncState = Context.OutputData.SyncState.SyncStateCollection.FindMutableDataByType<FMoverDefaultSyncState>();
	check(PostSimDefaultSyncState);

	UChaosMoverSimulation* ChaosMoverSim = Cast<UChaosMoverSimulation>(Context.Simulation);
	if (!ChaosMoverSim)
	{
		return;
	}

	UMoverBlackboard* Blackboard = ChaosMoverSim->GetBlackboard_Mutable();
	check(Blackboard);

	FFloorCheckResult FloorResult;
	const bool bFoundFloorResult = Blackboard->TryGet(CommonBlackboard::LastFloorResult, FloorResult);

	bool bIsInGroundMode = false;
	if (const UBaseMovementMode* CurrentMode = ChaosMoverSim->GetCurrentMovementMode().Get())
	{
		bIsInGroundMode = CurrentMode->HasGameplayTag(Mover_IsOnGround, /* bExactMatch = */ true);
	}
	if (bFoundFloorResult && FloorResult.bBlockingHit && bIsInGroundMode)
	{
		PostSimDefaultSyncState->SetTransforms_WorldSpace(
			PostSimDefaultSyncState->GetLocation_WorldSpace(),
			PostSimDefaultSyncState->GetOrientation_WorldSpace(),
			PostSimDefaultSyncState->GetVelocity_WorldSpace(),
			PostSimDefaultSyncState->GetAngularVelocityDegrees_WorldSpace(),
			FloorResult.HitResult.GetComponent(),
			FloorResult.HitResult.BoneName);
		PostSimDefaultSyncState->MoveDirectionIntent =
			PostSimDefaultSyncState->GetCapturedMovementBaseQuat().UnrotateVector(PostSimDefaultSyncState->MoveDirectionIntent);
	}

	FFloorResultData& FloorData = Context.OutputData.AdditionalOutputData.FindOrAddMutableDataByType<FFloorResultData>();
	FloorData.FloorResult = FloorResult;

	FWaterCheckResult WaterResult;
	Blackboard->TryGet(CommonBlackboard::LastWaterResult, WaterResult);

	FChaosWaterResultData& WaterData = Context.OutputData.AdditionalOutputData.FindOrAddMutableDataByType<FChaosWaterResultData>();
	WaterData.WaterResult = WaterResult;
}

// ---------------------------------------------------------------------------
// ConfigureCharacterGroundConstraint
// ---------------------------------------------------------------------------

void UE::ChaosMover::ConfigureCharacterGroundConstraint(
	FChaosMoverPostSimContext& Context,
	IChaosCharacterMovementModeInterface* CharacterInterface,
	IChaosCharacterConstraintMovementModeInterface* ConstraintInterface)
{
	if (!CharacterInterface || !ConstraintInterface)
	{
		return;
	}

	UChaosMoverSimulation* ChaosMoverSim = Cast<UChaosMoverSimulation>(Context.Simulation);
	if (!ChaosMoverSim)
	{
		return;
	}

	Chaos::FCharacterGroundConstraintHandle* CharacterGroundConstraintHandle = ChaosMoverSim->GetCharacterGroundConstraintHandle();
	if (!CharacterGroundConstraintHandle)
	{
		return;
	}

	if (!ConstraintInterface->ShouldEnableConstraint())
	{
		// Constraint stays disabled (simulation already disabled it upfront).
		return;
	}

	ChaosMoverSim->EnableCharacterGroundConstraint();

	Chaos::FCharacterGroundConstraintSettings& Settings = CharacterGroundConstraintHandle->GetSettings_Mutable();
	const FChaosMoverSimulationDefaultInputs* SimInputs = ChaosMoverSim->GetLocalSimInput().FindDataByType<FChaosMoverSimulationDefaultInputs>();
	if (SimInputs)
	{
		ConstraintInterface->UpdateConstraintSettings(Settings);
		Settings.VerticalAxis = SimInputs->UpDir;
		Settings.TargetHeight = CharacterInterface->GetTargetHeight();

		if (ChaosMoverSim->IsNonResimSimProxy())
		{
			Settings.RadialForceLimit   = 0.0f;
			Settings.FrictionForceLimit = 0.0f;
			Settings.TwistTorqueLimit   = 0.0f;
			Settings.SwingTorqueLimit   = 0.0f;
		}
	}

	// Compute target deltas: how far the game simulation wants to move and rotate the particle.
	FVector TargetDeltaPosition = FVector::ZeroVector;
	float TargetDeltaFacing = 0.0f;
	if (Chaos::FPBDRigidParticleHandle* ParticleHandle = ChaosMoverSim->GetControlledParticle())
	{
		FMoverDefaultSyncState* PostSimDefaultSyncState = Context.OutputData.SyncState.SyncStateCollection.FindMutableDataByType<FMoverDefaultSyncState>();
		check(PostSimDefaultSyncState);

		TargetDeltaPosition = PostSimDefaultSyncState->GetLocation_WorldSpace() - ParticleHandle->GetX();

		FQuat TgtQuat = PostSimDefaultSyncState->GetOrientation_WorldSpace().Quaternion();
		TgtQuat.EnforceShortestArcWith(ParticleHandle->GetR());
		const FQuat QuatRotation = TgtQuat * ParticleHandle->GetR().Inverse();
		const FVector AngularDisplacement = QuatRotation.ToRotationVector();

		FVector UpDir = FVector::UpVector;
		if (SimInputs)
		{
			UpDir = SimInputs->UpDir;
		}
		TargetDeltaFacing = AngularDisplacement.Dot(UpDir);
	}

	UMoverBlackboard* Blackboard = ChaosMoverSim->GetBlackboard_Mutable();
	check(Blackboard);
	FFloorCheckResult FloorResult;
	const bool bFoundFloorResult = Blackboard->TryGet(CommonBlackboard::LastFloorResult, FloorResult);

	if (bFoundFloorResult && FloorResult.bBlockingHit)
	{
		Chaos::FGeometryParticleHandle* GroundParticle = nullptr;
		if (Chaos::FPhysicsObjectHandle GroundPhysicsObject = FloorResult.HitResult.PhysicsObject)
		{
			Chaos::FReadPhysicsObjectInterface_Internal ReadInterface = Chaos::FPhysicsObjectInternalInterface::GetRead();
			if (!ReadInterface.AreAllDisabled({ GroundPhysicsObject }))
			{
				GroundParticle = ReadInterface.GetParticle(GroundPhysicsObject);
				if (ReadInterface.AreAllSleeping({ GroundPhysicsObject }))
				{
					Chaos::FWritePhysicsObjectInterface_Internal WriteInterface = Chaos::FPhysicsObjectInternalInterface::GetWrite();
					WriteInterface.WakeUp({ GroundPhysicsObject });
				}
			}
		}
		CharacterGroundConstraintHandle->SetGroundParticle(GroundParticle);

		float WalkableSlopeCosine = CharacterGroundConstraintHandle->GetSettings().CosMaxWalkableSlopeAngle;
		if (const TStrongObjectPtr<UPrimitiveComponent> PrimComp = FloorResult.HitResult.Component.Pin())
		{
			const FWalkableSlopeOverride& SlopeOverride = PrimComp->GetWalkableSlopeOverride();
			WalkableSlopeCosine = SlopeOverride.ModifyWalkableFloorZ(WalkableSlopeCosine);
		}
		if (!FloorResult.bWalkableFloor)
		{
			WalkableSlopeCosine = 2.0f;
		}

		CharacterGroundConstraintHandle->SetData({
			FloorResult.HitResult.ImpactNormal,
			TargetDeltaPosition,
			TargetDeltaFacing,
			FloorResult.FloorDist,
			WalkableSlopeCosine
		});
	}
	else
	{
		CharacterGroundConstraintHandle->SetGroundParticle(nullptr);
		CharacterGroundConstraintHandle->SetData({
			CharacterGroundConstraintHandle->GetSettings().VerticalAxis,
			Chaos::FVec3::ZeroVector,
			0.0,
			1.0e10,
			0.5f
		});
	}
}
