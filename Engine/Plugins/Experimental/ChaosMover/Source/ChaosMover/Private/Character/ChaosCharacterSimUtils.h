// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

struct FChaosMoverPreSimContext;
struct FChaosMoverPostSimContext;
class IChaosCharacterMovementModeInterface;
class IChaosCharacterConstraintMovementModeInterface;

namespace UE::ChaosMover
{
	/**
	 * Processes character movement inputs for the current tick: coerces invalid input to a
	 * zero directional intent, consumes any pending mode-transition request by queuing it on
	 * the simulation, and applies or removes per-mode speed/acceleration overrides.
	 * No-ops if the Simulation pointer in Context cannot be cast to UChaosMoverSimulation.
	 */
	void ProcessCharacterInputs(FChaosMoverPreSimContext& Context);

	/**
	 * Conditionally runs the character floor sweep and stores the result in the blackboard.
	 * Runs unconditionally when bEnablePreSimGroundCheck is set, or on the first frame of
	 * resimulation (the blackboard is invalidated on rollback so the result must be refreshed).
	 */
	void UpdateCurrentFloor(FChaosMoverPreSimContext& Context, IChaosCharacterMovementModeInterface& CharacterInterface);

	/**
	 * Reconciles particle velocity with game-side sync state. Drives the particle from the
	 * sync state in normal simulation; reads back from the particle for non-resim sim proxies.
	 * CharacterInterface may be null -- null is treated as "no upright opinion": linear velocity
	 * is still applied, angular velocity is left to physics and reflected back into sync state.
	 */
	void ReconcileParticleVelocity(FChaosMoverPostSimContext& Context, IChaosCharacterMovementModeInterface* CharacterInterface);

	/**
	 * Propagates floor and water sensing results into output data and assigns the movement
	 * base on the sync state when in a ground mode with a blocking floor hit.
	 * No-ops if CharacterInterface is null.
	 */
	void ApplyGroundAndWaterResults(FChaosMoverPostSimContext& Context, IChaosCharacterMovementModeInterface* CharacterInterface);

	/**
	 * Configures the character ground constraint for the next physics tick, using the sync
	 * state and floor result to compute target position and facing deltas.
	 * No-ops if either interface pointer is null or ShouldEnableConstraint returns false.
	 */
	void ConfigureCharacterGroundConstraint(
		FChaosMoverPostSimContext& Context,
		IChaosCharacterMovementModeInterface* CharacterInterface,
		IChaosCharacterConstraintMovementModeInterface* ConstraintInterface);
}
