// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosMover/Character/Effects/ChaosCharacterApplyVelocityEffect.h"

#include "Chaos/PhysicsObjectInternalInterface.h"
#include "ChaosMover/ChaosMoverLog.h"
#include "ChaosMover/ChaosMoverSimulation.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosCharacterApplyVelocityEffect)

bool FChaosCharacterApplyVelocityEffect::ApplyMovementEffect_Async(FApplyMovementEffectParams_Async& ApplyEffectParams, FMoverSyncState& OutputState)
{
	check(ApplyEffectParams.StartState);
	const FMoverDefaultSyncState* CurrentSyncState = ApplyEffectParams.StartState->SyncState.SyncStateCollection.FindDataByType<FMoverDefaultSyncState>();
	if (!CurrentSyncState)
	{
		return false;
	}

	const UChaosMoverSimulation* Simulation = Cast<UChaosMoverSimulation>(ApplyEffectParams.Simulation);
	if (!Simulation)
	{
		UE_LOGF(LogChaosMover, Warning, "No Simulation set on ChaosCharacterApplyVelocityEffect");
		return false;
	}

	const FChaosMoverSimulationDefaultInputs* SimInputs = Simulation->GetLocalSimInput().FindDataByType<FChaosMoverSimulationDefaultInputs>();
	if (!SimInputs)
	{
		UE_LOGF(LogChaosMover, Warning, "ChaosCharacterApplyVelocityEffect requires FChaosMoverSimulationDefaultInputs");
		return false;
	}

	// Get the position and orientation. Start by looking in the sync state.
	// If not there, get from the particle corresponding to the updated component
	FVector Position = CurrentSyncState->GetLocation_WorldSpace();
	FRotator Orientation = CurrentSyncState->GetOrientation_WorldSpace();

	FMoverDefaultSyncState& OutputSyncState = OutputState.SyncStateCollection.FindOrAddMutableDataByType<FMoverDefaultSyncState>();

	FVector Velocity = CurrentSyncState->GetVelocity_WorldSpace();
	switch (Mode)
	{
	case EChaosMoverVelocityEffectMode::Impulse:
		{
			Chaos::FReadPhysicsObjectInterface_Internal ReadInterface = Chaos::FPhysicsObjectInternalInterface::GetRead();
			float Mass = ReadInterface.GetMass({ SimInputs->PhysicsObject });
			if (Mass > UE_SMALL_NUMBER)
			{
				Velocity += (1.0f / Mass) * VelocityOrImpulseToApply;
			}
		}
		break;
	case EChaosMoverVelocityEffectMode::AdditiveVelocity:
		Velocity += VelocityOrImpulseToApply;
		break;
	case EChaosMoverVelocityEffectMode::OverrideVelocity:
		Velocity = VelocityOrImpulseToApply;
		break;
	default:
		break;
	}

	OutputSyncState.SetTransforms_WorldSpace(Position, Orientation, Velocity, CurrentSyncState->GetAngularVelocityDegrees_WorldSpace());

	return true;
}

FInstantMovementEffect* FChaosCharacterApplyVelocityEffect::Clone() const
{
	FChaosCharacterApplyVelocityEffect* CopyPtr = new FChaosCharacterApplyVelocityEffect(*this);
	return CopyPtr;
}

void FChaosCharacterApplyVelocityEffect::NetSerialize(FArchive& Ar)
{
	Super::NetSerialize(Ar);

	Ar << VelocityOrImpulseToApply;
	Ar << Mode;
}

UScriptStruct* FChaosCharacterApplyVelocityEffect::GetScriptStruct() const
{
	return FChaosCharacterApplyVelocityEffect::StaticStruct();
}

FString FChaosCharacterApplyVelocityEffect::ToSimpleString() const
{
	return FString::Printf(TEXT("ChaosCharacterApplyVelocityEffect VelocityOrImpulseToApply = %s, Mode = %s"), *VelocityOrImpulseToApply.ToString(), *StaticEnum<EChaosMoverVelocityEffectMode>()->GetValueAsString(Mode));
}

void FChaosCharacterApplyVelocityEffect::AddReferencedObjects(FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(Collector);
}
