// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosMover/Character/Executors/ChaosCharacterAirExecutor.h"

#include "Character/ChaosCharacterSimUtils.h"
#include "Chaos/Character/CharacterGroundConstraint.h"
#include "Chaos/Character/CharacterGroundConstraintSettings.h"
#include "Chaos/PhysicsObjectInternalInterface.h"
#include "ChaosMover/ChaosMoverLog.h"
#include "ChaosMover/ChaosMoverSimulation.h"
#include "ChaosMover/ChaosMoverSimulationTypes.h"
#include "ChaosMover/Character/Settings/SharedChaosCharacterMovementSettings.h"
#include "ChaosMover/Utilities/ChaosMoverQueryUtils.h"
#include "Components/CapsuleComponent.h"
#include "Components/PrimitiveComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "DefaultMovementSet/CharacterMoverSimulationTypes.h"
#include "GameFramework/Character.h"
#include "Math/UnitConversion.h"
#include "MoverComponent.h"
#include "MoveLibrary/FloorQueryUtils.h"
#include "MoveLibrary/MoverBlackboard.h"
#include "MoveLibrary/MovementUtils.h"
#include "MoveLibrary/WaterMovementUtils.h"
#include "MoverSimulationTypes.h"
#include "MoverTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosCharacterAirExecutor)

// ---------------------------------------------------------------------------
// UChaosMoveExecutorBase overrides
// ---------------------------------------------------------------------------

void UChaosCharacterAirExecutor::OnModeRegistered(const FName ModeName)
{
	if (TargetHeightOverride.IsSet())
	{
		TargetHeight = TargetHeightOverride.GetValue();
	}
	else if (const ACharacter* CharacterOwner = GetTypedOuter<ACharacter>())
	{
		TargetHeight = -CharacterOwner->GetMesh()->GetRelativeLocation().Z;
	}

	if (QueryRadiusOverride.IsSet())
	{
		QueryRadius = QueryRadiusOverride.GetValue();
	}
	else if (const ACharacter* CharacterOwner = GetTypedOuter<ACharacter>())
	{
		if (const UCapsuleComponent* CapsuleComp = CharacterOwner->GetCapsuleComponent())
		{
			QueryRadius = FMath::Max(CapsuleComp->GetScaledCapsuleRadius() - 5.0f, 0.0f);
		}
	}

	if (UMoverComponent* MoverComp = GetTypedOuter<UMoverComponent>())
	{
		TObjectPtr<const USharedChaosCharacterMovementSettings> SharedSettingsPtr = MoverComp->FindSharedSettings<USharedChaosCharacterMovementSettings>();
		if (ensureMsgf(SharedSettingsPtr, TEXT("%s: Failed to find USharedChaosCharacterMovementSettings. Movement may not function properly."), *GetPathNameSafe(this)))
		{
			SharedSettingsWeakPtr = TWeakObjectPtr<const USharedChaosCharacterMovementSettings>(SharedSettingsPtr);
		}
	}
}

void UChaosCharacterAirExecutor::OnModeUnregistered()
{
	SharedSettingsWeakPtr.Reset();
	MaxSpeedOverride.Reset();
	AccelerationOverride.Reset();
}

// ---------------------------------------------------------------------------
// ExecuteMove_Async
// ---------------------------------------------------------------------------

void UChaosCharacterAirExecutor::ExecuteMove_Async(const FSimulationTickParams& Params, FMoverTickEndData& OutputState)
{
	QUICK_SCOPE_CYCLE_COUNTER(ChaosCharacterAirExecutor_ExecuteMove);

	if (!Simulation)
	{
		UE_LOG(LogChaosMover, Warning, TEXT("UChaosCharacterAirExecutor (%s): No Simulation set. Check you have a Chaos Backend."), *GetName());
		return;
	}

	const FChaosMoverSimulationDefaultInputs* DefaultSimInputs = Simulation->GetLocalSimInput().FindDataByType<FChaosMoverSimulationDefaultInputs>();
	if (!DefaultSimInputs)
	{
		UE_LOG(LogChaosMover, Warning, TEXT("UChaosCharacterAirExecutor (%s): Requires FChaosMoverSimulationDefaultInputs."), *GetName());
		return;
	}

	const FMoverDefaultSyncState* StartingSyncState = Params.StartState.SyncState.SyncStateCollection.FindDataByType<FMoverDefaultSyncState>();
	if (!StartingSyncState)
	{
		UE_LOG(LogChaosMover, Warning, TEXT("UChaosCharacterAirExecutor (%s): Requires FMoverDefaultSyncState in the sync state collection."), *GetName());
		return;
	}

	const FProposedMove& ProposedMove = Params.ProposedMove;
	FMoverDefaultSyncState& OutputSyncState = OutputState.SyncState.SyncStateCollection.FindOrAddMutableDataByType<FMoverDefaultSyncState>();

	const float DeltaSeconds = Params.TimeStep.StepMs * 0.001f;

	const FRotator TargetOrient = UMovementUtils::ApplyAngularVelocityToRotator(
		StartingSyncState->GetOrientation_WorldSpace(), ProposedMove.AngularVelocityDegrees, DeltaSeconds);

	// The physics simulation applies Z-only gravity acceleration via physics volumes, so we need to account for it here
	const FVector TargetVel = ProposedMove.LinearVelocity - DefaultSimInputs->PhysicsObjectGravity * FVector::UpVector * DeltaSeconds;
	const FVector TargetPos = StartingSyncState->GetLocation_WorldSpace() + TargetVel * DeltaSeconds;

	OutputState.MovementEndState.RemainingMs = 0.0f;
	OutputState.MovementEndState.NextModeName = Params.StartState.SyncState.MovementMode;

	// Derive angular velocity from the quaternion delta between the start and
	// target orientations. This gives physically correct results regardless of
	// rotation speed, matching the convention used by the standalone falling mode.
	FVector TargetAngularVelocity = StartingSyncState->GetAngularVelocityDegrees_WorldSpace();
	if (DeltaSeconds > UE_SMALL_NUMBER)
	{
		const FQuat InitialQuat = StartingSyncState->GetOrientation_WorldSpace().Quaternion();
		FQuat TgtQuat = TargetOrient.Quaternion();
		TgtQuat.EnforceShortestArcWith(InitialQuat);
		const FQuat QuatRotation = TgtQuat * InitialQuat.Inverse();
		const FVector AngularDisplacement = QuatRotation.ToRotationVector();
		TargetAngularVelocity = FMath::RadiansToDegrees(AngularDisplacement / DeltaSeconds);
	}

	OutputSyncState.MoveDirectionIntent = ProposedMove.bHasDirIntent ? ProposedMove.DirectionIntent : FVector::ZeroVector;
	OutputSyncState.SetTransforms_WorldSpace(
		TargetPos,
		TargetOrient,
		TargetVel,
		TargetAngularVelocity);
}

// ---------------------------------------------------------------------------
// IChaosCharacterMovementModeInterface
// ---------------------------------------------------------------------------

float UChaosCharacterAirExecutor::GetMaxWalkSlopeCosine() const
{
	if (TStrongObjectPtr<const USharedChaosCharacterMovementSettings> SharedSettingsStrongPtr = SharedSettingsWeakPtr.Pin())
	{
		return SharedSettingsStrongPtr->GetMaxWalkableSlopeCosine();
	}

	return 0.707f;
}

float UChaosCharacterAirExecutor::GetMaxSpeed() const
{
	if (MaxSpeedOverride.IsSet())
	{
		return MaxSpeedOverride.GetValue();
	}

	if (TStrongObjectPtr<const USharedChaosCharacterMovementSettings> SharedSettingsStrongPtr = SharedSettingsWeakPtr.Pin())
	{
		return SharedSettingsStrongPtr->MaxSpeed;
	}

	UE_LOGF(LogChaosMover, Warning, "UChaosCharacterAirExecutor (%ls): Unable to read MaxSpeed from shared settings.", *GetName());
	return 0.0f;
}

void UChaosCharacterAirExecutor::OverrideMaxSpeed(float Value)
{
	MaxSpeedOverride = Value;
}

void UChaosCharacterAirExecutor::ClearMaxSpeedOverride()
{
	MaxSpeedOverride.Reset();
}

float UChaosCharacterAirExecutor::GetAcceleration() const
{
	if (AccelerationOverride.IsSet())
	{
		return AccelerationOverride.GetValue();
	}

	if (TStrongObjectPtr<const USharedChaosCharacterMovementSettings> SharedSettingsStrongPtr = SharedSettingsWeakPtr.Pin())
	{
		return SharedSettingsStrongPtr->Acceleration;
	}

	UE_LOGF(LogChaosMover, Warning, "UChaosCharacterAirExecutor (%ls): Unable to read Acceleration from shared settings.", *GetName());
	return 0.0f;
}

void UChaosCharacterAirExecutor::OverrideAcceleration(float Value)
{
	AccelerationOverride = Value;
}

void UChaosCharacterAirExecutor::ClearAccelerationOverride()
{
	AccelerationOverride.Reset();
}

void UChaosCharacterAirExecutor::UpdateCurrentFloor(const FMoverTimeStep& TimeStep) const
{
	check(Simulation);

	UMoverBlackboard* SimBlackboard = Simulation->GetBlackboard_Mutable();
	if (!SimBlackboard)
	{
		return;
	}

	bool bFloorCheckSucceeded = false;

	if (const FChaosMoverSimulationDefaultInputs* SimInputs = Simulation->GetLocalSimInput().FindDataByType<FChaosMoverSimulationDefaultInputs>())
	{
		Chaos::FReadPhysicsObjectInterface_Internal ReadInterface = Chaos::FPhysicsObjectInternalInterface::GetRead();
		if (const Chaos::FPBDRigidParticleHandle* UpdatedParticle = ReadInterface.GetRigidParticle(SimInputs->PhysicsObject))
		{
			FFloorCheckResult FloorResult;
			FWaterCheckResult WaterResult;

			const float DeltaSeconds = TimeStep.StepMs * 0.001f;
			const FVector VelocityWithGravity = UpdatedParticle->GetV() + UMovementUtils::ComputeVelocityFromGravity(SimInputs->Gravity, DeltaSeconds);

			UE::ChaosMover::Utils::FFloorSweepParams SweepParams{
				.ResponseParams     = SimInputs->CollisionResponseParams,
				.QueryParams        = SimInputs->CollisionQueryParams,
				.Location           = UpdatedParticle->GetX(),
				.DeltaPos           = VelocityWithGravity * DeltaSeconds,
				.UpDir              = SimInputs->UpDir,
				.World              = SimInputs->World,
				.QueryDistance      = 1.5f * GetTargetHeight(),
				.QueryRadius        = FMath::Min(GetGroundQueryRadius(), FMath::Max(SimInputs->PawnCollisionRadius - 5.0f, 0.0f)),
				.MaxWalkSlopeCosine = GetMaxWalkSlopeCosine(),
				.TargetHeight       = GetTargetHeight(),
				.CollisionChannel   = SimInputs->CollisionChannel
			};

			UE::ChaosMover::Utils::FloorSweep_Internal(SweepParams, FloorResult, WaterResult);

			SimBlackboard->Set(CommonBlackboard::LastFloorResult, FloorResult);
			SimBlackboard->Set(CommonBlackboard::LastWaterResult, WaterResult);
			SimBlackboard->Set(UE::ChaosMover::Blackboard::GroundDynamicsInfo, UE::ChaosMover::FGroundDynamicsInfo(FloorResult));

			bFloorCheckSucceeded = true;
		}
	}

	if (!bFloorCheckSucceeded)
	{
		SimBlackboard->Invalidate(CommonBlackboard::LastFloorResult);
		SimBlackboard->Invalidate(CommonBlackboard::LastWaterResult);
		SimBlackboard->Invalidate(UE::ChaosMover::Blackboard::GroundDynamicsInfo);
	}
}

// ---------------------------------------------------------------------------
// IChaosCharacterConstraintMovementModeInterface
// ---------------------------------------------------------------------------

void UChaosCharacterAirExecutor::UpdateConstraintSettings(Chaos::FCharacterGroundConstraintSettings& ConstraintSettings) const
{
	// Air movement is velocity-based -- the constraint provides upright swing torque only.
	// Radial (positional) and twist forces are intentionally zero.
	ConstraintSettings.RadialForceLimit     = 0.0f;
	ConstraintSettings.TwistTorqueLimit     = 0.0f;
	ConstraintSettings.SwingTorqueLimit     = FUnitConversion::Convert(SwingTorqueLimit, EUnit::NewtonMeters, EUnit::KilogramCentimetersSquaredPerSecondSquared);
	ConstraintSettings.TargetHeight         = TargetHeight;
	ConstraintSettings.FrictionForceLimit   = 0.0f;
	ConstraintSettings.DampingFactor        = 0.0f;
	ConstraintSettings.MotionTargetMassBias = 0.0f;
	ConstraintSettings.RadialForceMotionTargetScaling = 0.0f;
}

// ---------------------------------------------------------------------------
// IChaosPreSimulationTickInterface
// ---------------------------------------------------------------------------

void UChaosCharacterAirExecutor::PreSimulationTick_Async(FChaosMoverPreSimContext& Context)
{
	UE::ChaosMover::ProcessCharacterInputs(Context);
	UE::ChaosMover::UpdateCurrentFloor(Context, *this);
}

// ---------------------------------------------------------------------------
// IChaosPostSimulationTickInterface
// ---------------------------------------------------------------------------

void UChaosCharacterAirExecutor::PostSimulationTick_Async(FChaosMoverPostSimContext& Context)
{
	QUICK_SCOPE_CYCLE_COUNTER(ChaosCharacterAirExecutor_PostSimulationTick);
	UE::ChaosMover::ReconcileParticleVelocity(Context, this);
	UE::ChaosMover::ApplyGroundAndWaterResults(Context, this);
	UE::ChaosMover::ConfigureCharacterGroundConstraint(Context, this, this);
}
