// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosMover/Character/Executors/ChaosCharacterWalkingExecutor.h"

#include "Character/ChaosCharacterSimUtils.h"
#include "Chaos/Character/CharacterGroundConstraintSettings.h"
#include "Chaos/PhysicsObjectInternalInterface.h"
#include "ChaosMover/ChaosMoverLog.h"
#include "ChaosMover/ChaosMoverSimulation.h"
#include "ChaosMover/ChaosMoverSimulationTypes.h"
#include "ChaosMover/Character/Settings/SharedChaosCharacterMovementSettings.h"
#include "ChaosMover/Utilities/ChaosMoverQueryUtils.h"
#include "Components/CapsuleComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "DefaultMovementSet/CharacterMoverSimulationTypes.h"
#include "GameFramework/Character.h"
#include "Math/UnitConversion.h"
#include "MoverComponent.h"
#include "MoveLibrary/FloorQueryUtils.h"
#include "MoveLibrary/MoverBlackboard.h"
#include "MoveLibrary/MovementUtils.h"
#include "MoveLibrary/WaterMovementUtils.h"
#include "MoverSimulationTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosCharacterWalkingExecutor)


void UChaosCharacterWalkingExecutor::OnModeRegistered(const FName ModeName)
{
	if (TargetHeightOverride.IsSet())
	{
		TargetHeight = TargetHeightOverride.GetValue();
	}
	else if (const ACharacter* CharacterOwner = GetTypedOuter<ACharacter>())
	{
		TargetHeight = -CharacterOwner->GetMesh()->GetRelativeLocation().Z;
	}

	if (QueryRadiusOverride.IsSet())
	{
		QueryRadius = QueryRadiusOverride.GetValue();
	}
	else if (const ACharacter* CharacterOwner = GetTypedOuter<ACharacter>())
	{
		if (const UCapsuleComponent* CapsuleComp = CharacterOwner->GetCapsuleComponent())
		{
			QueryRadius = FMath::Max(CapsuleComp->GetScaledCapsuleRadius() - 5.0f, 0.0f);
		}
	}

	if (UMoverComponent* MoverComp = GetTypedOuter<UMoverComponent>())
	{
		TObjectPtr<const USharedChaosCharacterMovementSettings> SharedSettingsPtr = MoverComp->FindSharedSettings<USharedChaosCharacterMovementSettings>();
		if (ensureMsgf(SharedSettingsPtr, TEXT("%s: Failed to find USharedChaosCharacterMovementSettings. Movement may not function properly."), *GetPathNameSafe(this)))
		{
			SharedSettingsWeakPtr = TWeakObjectPtr<const USharedChaosCharacterMovementSettings>(SharedSettingsPtr);
		}
	}
}

void UChaosCharacterWalkingExecutor::OnModeUnregistered()
{
	SharedSettingsWeakPtr.Reset();
	MaxSpeedOverride.Reset();
	AccelerationOverride.Reset();
}

void UChaosCharacterWalkingExecutor::ExecuteMove_Async(const FSimulationTickParams& Params, FMoverTickEndData& OutputState)
{
	QUICK_SCOPE_CYCLE_COUNTER(ChaosCharacterWalkingExecutor_ExecuteMove);

	if (!Simulation)
	{
		UE_LOGF(LogChaosMover, Warning, "UChaosCharacterWalkingExecutor (%ls): No Simulation set. Check you have a Chaos Backend.", *GetName());
		return;
	}

	const FChaosMoverSimulationDefaultInputs* DefaultSimInputs = Simulation->GetLocalSimInput().FindDataByType<FChaosMoverSimulationDefaultInputs>();
	const FCharacterDefaultInputs* CharacterInputs = Params.StartState.InputCmd.InputCollection.FindDataByType<FCharacterDefaultInputs>();
	if (!DefaultSimInputs || !CharacterInputs)
	{
		UE_LOGF(LogChaosMover, Warning, "UChaosCharacterWalkingExecutor (%ls): Requires FChaosMoverSimulationDefaultInputs and FCharacterDefaultInputs.", *GetName());
		return;
	}

	TStrongObjectPtr<const USharedChaosCharacterMovementSettings> SharedSettingsStrongPtr = SharedSettingsWeakPtr.Pin();
	if (!SharedSettingsStrongPtr)
	{
		UE_LOGF(LogChaosMover, Warning, "UChaosCharacterWalkingExecutor (%ls): Unable to read from shared settings (USharedChaosCharacterMovementSettings).", *GetName());
		return;
	}
	const USharedChaosCharacterMovementSettings* SharedSettings = SharedSettingsStrongPtr.Get();
	check(SharedSettings);

	FProposedMove ProposedMove = Params.ProposedMove;
	const FMoverDefaultSyncState* StartingSyncState = Params.StartState.SyncState.SyncStateCollection.FindDataByType<FMoverDefaultSyncState>();
	if (!StartingSyncState)
	{
		UE_LOGF(LogChaosMover, Warning, "UChaosCharacterWalkingExecutor (%ls): Requires FMoverDefaultSyncState in the sync state collection.", *GetName());
		return;
	}

	FMoverDefaultSyncState& OutputSyncState = OutputState.SyncState.SyncStateCollection.FindOrAddMutableDataByType<FMoverDefaultSyncState>();
	OutputSyncState = *StartingSyncState;

	const float DeltaSeconds = Params.TimeStep.StepMs * 0.001f;
	const FVector UpDirection = DefaultSimInputs->UpDir;
	FVector GroundNormal = UpDirection;

	FFloorCheckResult FloorResult;
	UE::ChaosMover::FGroundDynamicsInfo GroundDynamicsInfo;
	if (const UMoverBlackboard* SimBlackboard = Simulation->GetBlackboard())
	{
		SimBlackboard->TryGet(CommonBlackboard::LastFloorResult, FloorResult);
		SimBlackboard->TryGet(UE::ChaosMover::Blackboard::GroundDynamicsInfo, GroundDynamicsInfo);
	}

	if (FloorResult.IsWalkableFloor())
	{
		GroundNormal = FloorResult.HitResult.ImpactNormal;

		FVector TargetPosition = StartingSyncState->GetLocation_WorldSpace();

		// Physics volumes always apply gravity along world Z (FVector::UpVector), not along UpDir.
		const FVector PhysicsObjectGravityVelocity = DefaultSimInputs->PhysicsObjectGravity * FVector::UpVector * DeltaSeconds;

		const FVector ProjectedVelocity = StartingSyncState->GetVelocity_WorldSpace() + DefaultSimInputs->Gravity * DeltaSeconds;
		FVector TargetVelocity = ProjectedVelocity - PhysicsObjectGravityVelocity;

		constexpr float ParallelCosThreshold = 0.999f;
		const bool bNonVerticalVelocity = !FVector::Parallel(TargetVelocity.GetSafeNormal(), UpDirection, ParallelCosThreshold);
		const bool bUseProposedMove = bNonVerticalVelocity || ProposedMove.bHasDirIntent;
		bool bNormalVelocityIntent = false;

		if (bUseProposedMove)
		{
			const FVector ProposedMovePlaneVelocity = ProposedMove.LinearVelocity - ProposedMove.LinearVelocity.ProjectOnToNormal(GroundNormal);

			FVector ProposedNormalVelocity = ProposedMove.LinearVelocity - ProposedMovePlaneVelocity;
			if (ProposedNormalVelocity.SizeSquared() > UE_KINDA_SMALL_NUMBER)
			{
				TargetVelocity += ProposedNormalVelocity - TargetVelocity.ProjectOnToNormal(GroundNormal);
				bNormalVelocityIntent = true;
			}

			TargetPosition += ProposedMovePlaneVelocity * DeltaSeconds;
		}

		FVector ProjectedGroundVelocity = GroundDynamicsInfo.LinearVelocity;
		if (GroundDynamicsInfo.bIsGravityEnabled)
		{
			ProjectedGroundVelocity += PhysicsObjectGravityVelocity;
		}

		const FVector ProjectedRelativeVelocity = TargetVelocity - ProjectedGroundVelocity;
		const float ProjectedRelativeNormalVelocity = FloorResult.HitResult.ImpactNormal.Dot(TargetVelocity - ProjectedGroundVelocity);
		const float ProjectedRelativeVerticalVelocity = GroundNormal.Dot(TargetVelocity - ProjectedGroundVelocity);
		const float VerticalVelocityLimit = bNormalVelocityIntent ? 2.0f / DeltaSeconds : FMath::Abs(GroundNormal.Dot(DefaultSimInputs->Gravity) * DeltaSeconds);

		bool bIsLiftingOffSurface = false;
		if ((ProjectedRelativeNormalVelocity > VerticalVelocityLimit) && GroundDynamicsInfo.bIsMoving && (ProjectedRelativeVerticalVelocity > VerticalVelocityLimit))
		{
			bIsLiftingOffSurface = true;
		}

		const float InitialHeightAboveFloor = FloorResult.FloorDist - GetTargetHeight();
		const float EndHeightAboveFloor = InitialHeightAboveFloor + ProjectedRelativeVerticalVelocity * DeltaSeconds;
		const bool bIsSteppingDown = InitialHeightAboveFloor > UE_KINDA_SMALL_NUMBER;
		const bool bIsWithinReach = EndHeightAboveFloor <= SharedSettings->MaxStepHeight;
		const bool bIsSupported = bIsWithinReach && !bIsLiftingOffSurface;
		const bool bNeedsVerticalVelocityToTarget = bIsSupported && bIsSteppingDown && (EndHeightAboveFloor > 0.0f) && !bIsLiftingOffSurface;
		if (bNeedsVerticalVelocityToTarget)
		{
			TargetVelocity -= FractionalDownwardVelocityToTarget * (EndHeightAboveFloor / DeltaSeconds) * UpDirection;
		}

		if (!GroundDynamicsInfo.bIsMoving)
		{
			TargetPosition -= UpDirection * InitialHeightAboveFloor;
		}

		const FRotator TargetOrientation = UMovementUtils::ApplyAngularVelocityToRotator(StartingSyncState->GetOrientation_WorldSpace(), ProposedMove.AngularVelocityDegrees, DeltaSeconds);
		FVector TargetAngularVelocity = StartingSyncState->GetAngularVelocityDegrees_WorldSpace();

		if (ShouldCharacterRemainUpright() && bShouldApplyAngularVelocityToTarget)
		{
			if (DeltaSeconds > UE_SMALL_NUMBER)
			{
				const FQuat InitialQuat = StartingSyncState->GetOrientation_WorldSpace().Quaternion();
				FQuat TgtQuat = TargetOrientation.Quaternion();
				TgtQuat.EnforceShortestArcWith(InitialQuat);
				const FQuat QuatRotation = TgtQuat * InitialQuat.Inverse();
				const FVector AngularDisplacement = QuatRotation.ToRotationVector();
				TargetAngularVelocity = FMath::RadiansToDegrees(AngularDisplacement / DeltaSeconds);
			}
		}

		OutputSyncState.MoveDirectionIntent = ProposedMove.bHasDirIntent ? ProposedMove.DirectionIntent : FVector::ZeroVector;
		OutputSyncState.SetTransforms_WorldSpace(
			TargetPosition,
			TargetOrientation,
			TargetVelocity,
			TargetAngularVelocity);
	}

	OutputState.MovementEndState.RemainingMs = 0.0f;
	OutputState.MovementEndState.NextModeName = Params.StartState.SyncState.MovementMode;
}

// ---------------------------------------------------------------------------
// IChaosCharacterMovementModeInterface
// ---------------------------------------------------------------------------

float UChaosCharacterWalkingExecutor::GetMaxWalkSlopeCosine() const
{
	if (TStrongObjectPtr<const USharedChaosCharacterMovementSettings> SharedSettingsStrongPtr = SharedSettingsWeakPtr.Pin())
	{
		return SharedSettingsStrongPtr->GetMaxWalkableSlopeCosine();
	}

	return 0.707f;
}

float UChaosCharacterWalkingExecutor::GetMaxSpeed() const
{
	if (MaxSpeedOverride.IsSet())
	{
		return MaxSpeedOverride.GetValue();
	}

	if (TStrongObjectPtr<const USharedChaosCharacterMovementSettings> SharedSettingsStrongPtr = SharedSettingsWeakPtr.Pin())
	{
		return SharedSettingsStrongPtr->MaxSpeed;
	}

	UE_LOGF(LogChaosMover, Warning, "UChaosCharacterWalkingExecutor (%ls): Unable to read MaxSpeed from shared settings.", *GetName());
	return 0.0f;
}

void UChaosCharacterWalkingExecutor::OverrideMaxSpeed(float Value)
{
	MaxSpeedOverride = Value;
}

void UChaosCharacterWalkingExecutor::ClearMaxSpeedOverride()
{
	MaxSpeedOverride.Reset();
}

float UChaosCharacterWalkingExecutor::GetAcceleration() const
{
	if (AccelerationOverride.IsSet())
	{
		return AccelerationOverride.GetValue();
	}

	if (TStrongObjectPtr<const USharedChaosCharacterMovementSettings> SharedSettingsStrongPtr = SharedSettingsWeakPtr.Pin())
	{
		return SharedSettingsStrongPtr->Acceleration;
	}

	UE_LOGF(LogChaosMover, Warning, "UChaosCharacterWalkingExecutor (%ls): Unable to read Acceleration from shared settings.", *GetName());
	return 0.0f;
}

void UChaosCharacterWalkingExecutor::OverrideAcceleration(float Value)
{
	AccelerationOverride = Value;
}

void UChaosCharacterWalkingExecutor::ClearAccelerationOverride()
{
	AccelerationOverride.Reset();
}

void UChaosCharacterWalkingExecutor::UpdateCurrentFloor(const FMoverTimeStep& TimeStep) const
{
	check(Simulation);

	UMoverBlackboard* SimBlackboard = Simulation->GetBlackboard_Mutable();
	if (!SimBlackboard)
	{
		return;
	}

	bool bFloorCheckSucceeded = false;

	if (const FChaosMoverSimulationDefaultInputs* SimInputs = Simulation->GetLocalSimInput().FindDataByType<FChaosMoverSimulationDefaultInputs>())
	{
		Chaos::FReadPhysicsObjectInterface_Internal ReadInterface = Chaos::FPhysicsObjectInternalInterface::GetRead();
		if (const Chaos::FPBDRigidParticleHandle* UpdatedParticle = ReadInterface.GetRigidParticle(SimInputs->PhysicsObject))
		{
			FFloorCheckResult FloorResult;
			FWaterCheckResult WaterResult;

			const float DeltaSeconds = TimeStep.StepMs * 0.001f;
			const FVector VelocityWithGravity = UpdatedParticle->GetV() + UMovementUtils::ComputeVelocityFromGravity(SimInputs->Gravity, DeltaSeconds);

			UE::ChaosMover::Utils::FFloorSweepParams SweepParams{
				.ResponseParams = SimInputs->CollisionResponseParams,
				.QueryParams = SimInputs->CollisionQueryParams,
				.Location = UpdatedParticle->GetX(),
				.DeltaPos = VelocityWithGravity * DeltaSeconds,
				.UpDir = SimInputs->UpDir,
				.World = SimInputs->World,
				.QueryDistance = 1.5f * GetTargetHeight(),
				.QueryRadius = FMath::Min(GetGroundQueryRadius(), FMath::Max(SimInputs->PawnCollisionRadius - 5.0f, 0.0f)),
				.MaxWalkSlopeCosine = GetMaxWalkSlopeCosine(),
				.TargetHeight = GetTargetHeight(),
				.CollisionChannel = SimInputs->CollisionChannel
			};

			UE::ChaosMover::Utils::FloorSweep_Internal(SweepParams, FloorResult, WaterResult);

			SimBlackboard->Set(CommonBlackboard::LastFloorResult, FloorResult);
			SimBlackboard->Set(CommonBlackboard::LastWaterResult, WaterResult);
			SimBlackboard->Set(UE::ChaosMover::Blackboard::GroundDynamicsInfo, UE::ChaosMover::FGroundDynamicsInfo(FloorResult));

			bFloorCheckSucceeded = true;
		}
	}

	if (!bFloorCheckSucceeded)
	{
		SimBlackboard->Invalidate(CommonBlackboard::LastFloorResult);
		SimBlackboard->Invalidate(CommonBlackboard::LastWaterResult);
		SimBlackboard->Invalidate(UE::ChaosMover::Blackboard::GroundDynamicsInfo);
	}
}

// ---------------------------------------------------------------------------
// IChaosCharacterConstraintMovementModeInterface
// ---------------------------------------------------------------------------

void UChaosCharacterWalkingExecutor::UpdateConstraintSettings(Chaos::FCharacterGroundConstraintSettings& ConstraintSettings) const
{
	ConstraintSettings.RadialForceLimit = FUnitConversion::Convert(RadialForceLimit, EUnit::Newtons, EUnit::KilogramCentimetersPerSecondSquared);
	ConstraintSettings.TwistTorqueLimit = FUnitConversion::Convert(TwistTorqueLimit, EUnit::NewtonMeters, EUnit::KilogramCentimetersSquaredPerSecondSquared);
	ConstraintSettings.SwingTorqueLimit = FUnitConversion::Convert(SwingTorqueLimit, EUnit::NewtonMeters, EUnit::KilogramCentimetersSquaredPerSecondSquared);
	ConstraintSettings.TargetHeight = TargetHeight;
	ConstraintSettings.FrictionForceLimit = FUnitConversion::Convert(FrictionForceLimit, EUnit::Newtons, EUnit::KilogramCentimetersPerSecondSquared);
	ConstraintSettings.DampingFactor = GroundDamping;
	ConstraintSettings.MotionTargetMassBias = FractionalGroundReaction;
	ConstraintSettings.RadialForceMotionTargetScaling = FractionalRadialForceLimitScaling;
}

// ---------------------------------------------------------------------------
// IChaosPreSimulationTickInterface
// ---------------------------------------------------------------------------

void UChaosCharacterWalkingExecutor::PreSimulationTick_Async(FChaosMoverPreSimContext& Context)
{
	UE::ChaosMover::ProcessCharacterInputs(Context);
	UE::ChaosMover::UpdateCurrentFloor(Context, *this);
}

// ---------------------------------------------------------------------------
// IChaosPostSimulationTickInterface
// ---------------------------------------------------------------------------

void UChaosCharacterWalkingExecutor::PostSimulationTick_Async(FChaosMoverPostSimContext& Context)
{
	QUICK_SCOPE_CYCLE_COUNTER(ChaosCharacterWalkingExecutor_PostSimulationTick);
	UE::ChaosMover::ReconcileParticleVelocity(Context, this);
	UE::ChaosMover::ApplyGroundAndWaterResults(Context, this);
	UE::ChaosMover::ConfigureCharacterGroundConstraint(Context, this, this);
}
