// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosMover/Character/Modes/ChaosCharacterMovementMode.h"

#include "Character/ChaosCharacterSimUtils.h"
#include "Chaos/Character/CharacterGroundConstraintSettings.h"
#include "Chaos/ContactModification.h"
#include "Chaos/PhysicsObjectInternalInterface.h"
#include "ChaosMover/Character/Settings/SharedChaosCharacterMovementSettings.h"
#include "ChaosMover/ChaosMoverLog.h"
#include "ChaosMover/ChaosMoverSimulation.h"
#include "ChaosMover/Utilities/ChaosMoverQueryUtils.h"
#include "Components/CapsuleComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "GameFramework/Character.h"
#include "Math/UnitConversion.h"
#include "MoveLibrary/FloorQueryUtils.h"
#include "MoveLibrary/MovementUtils.h"
#include "MoveLibrary/WaterMovementUtils.h"
#include "MoverComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosCharacterMovementMode)

UChaosCharacterMovementMode::UChaosCharacterMovementMode(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SharedSettingsClasses.Add(USharedChaosCharacterMovementSettings::StaticClass());
}

void UChaosCharacterMovementMode::OnRegistered(const FName ModeName, const FMoverSimContext& SimContext)
{
	Super::OnRegistered(ModeName, SimContext);

	if (TargetHeightOverride.IsSet())
	{
		TargetHeight = TargetHeightOverride.GetValue();
	}
	else if (const ACharacter* CharacterOwner = GetTypedOuter<ACharacter>())
	{
		TargetHeight = -CharacterOwner->GetMesh()->GetRelativeLocation().Z;
	}

	if (QueryRadiusOverride.IsSet())
	{
		QueryRadius = QueryRadiusOverride.GetValue();
	}

	TObjectPtr<const USharedChaosCharacterMovementSettings> SharedSettingsPtr = GetMoverComponent()->FindSharedSettings<USharedChaosCharacterMovementSettings>();
	if (ensureMsgf(SharedSettingsPtr, TEXT("Failed to find instance of USharedChaosCharacterMovementSettings on %s. Movement may not function properly."), *GetPathNameSafe(this)))
	{
		SharedSettingsWeakPtr = TWeakObjectPtr<const USharedChaosCharacterMovementSettings>(SharedSettingsPtr);
	}
}

void UChaosCharacterMovementMode::SetTargetHeightOverride(float InTargetHeight)
{
	TargetHeightOverride = InTargetHeight;
	TargetHeight = InTargetHeight;
}

void UChaosCharacterMovementMode::ClearTargetHeightOverride()
{
	TargetHeightOverride.Reset();

	if (const ACharacter* CharacterOwner = GetTypedOuter<ACharacter>())
	{
		TargetHeight = -CharacterOwner->GetMesh()->GetRelativeLocation().Z;
	}
	else
	{
		TargetHeight = GetDefault<UChaosCharacterMovementMode>(GetClass())->TargetHeight;
	}
}

void UChaosCharacterMovementMode::SetQueryRadiusOverride(float InQueryRadius)
{
	QueryRadiusOverride = InQueryRadius;
	QueryRadius = InQueryRadius;
}

void UChaosCharacterMovementMode::ClearQueryRadiusOverride()
{
	QueryRadiusOverride.Reset();

	if (const ACharacter* CharacterOwner = GetTypedOuter<ACharacter>())
	{
		if (UCapsuleComponent* CapsuleComp = CharacterOwner->GetCapsuleComponent())
		{
			QueryRadius = FMath::Max(CapsuleComp->GetScaledCapsuleRadius() - 5.0f, 0.0f);
			return;
		}
	}
	
	QueryRadius = GetDefault<UChaosCharacterMovementMode>(GetClass())->QueryRadius;
}

void UChaosCharacterMovementMode::PreSimulationTick_Async(FChaosMoverPreSimContext& Context)
{
	UE::ChaosMover::ProcessCharacterInputs(Context);
	UE::ChaosMover::UpdateCurrentFloor(Context, *this);
}

void UChaosCharacterMovementMode::PostSimulationTick_Async(FChaosMoverPostSimContext& Context)
{
	check(Simulation);
	UE::ChaosMover::ReconcileParticleVelocity(Context, this);
	UE::ChaosMover::ApplyGroundAndWaterResults(Context, this);
	UE::ChaosMover::ConfigureCharacterGroundConstraint(Context, this, this);
}

void UChaosCharacterMovementMode::UpdateConstraintSettings(Chaos::FCharacterGroundConstraintSettings& ConstraintSettings) const
{
	ConstraintSettings.RadialForceLimit = FUnitConversion::Convert(RadialForceLimit, EUnit::Newtons, EUnit::KilogramCentimetersPerSecondSquared);
	ConstraintSettings.TwistTorqueLimit = FUnitConversion::Convert(TwistTorqueLimit, EUnit::NewtonMeters, EUnit::KilogramCentimetersSquaredPerSecondSquared);
	ConstraintSettings.SwingTorqueLimit = FUnitConversion::Convert(SwingTorqueLimit, EUnit::NewtonMeters, EUnit::KilogramCentimetersSquaredPerSecondSquared);
	ConstraintSettings.TargetHeight = TargetHeight;
}

float UChaosCharacterMovementMode::GetMaxWalkSlopeCosine() const
{
	if (TStrongObjectPtr<const USharedChaosCharacterMovementSettings> SharedSettingsStrongPtr = SharedSettingsWeakPtr.Pin())
	{
		return SharedSettingsStrongPtr->GetMaxWalkableSlopeCosine();
	}

	return 0.707f;
}

float UChaosCharacterMovementMode::GetMaxSpeed() const
{
	if (MaxSpeedOverride.IsSet())
	{
		return MaxSpeedOverride.GetValue();
	}
	else if (TStrongObjectPtr<const USharedChaosCharacterMovementSettings> SharedSettingsStrongPtr = SharedSettingsWeakPtr.Pin())
	{
		return SharedSettingsStrongPtr->MaxSpeed;
	}

	UE_LOGF(LogChaosMover, Warning, "Invalid max speed on ChaosCharacterMoverComponent");
	return 0.0f;
}

void UChaosCharacterMovementMode::OverrideMaxSpeed(float Value)
{
	MaxSpeedOverride = Value;
}

void UChaosCharacterMovementMode::ClearMaxSpeedOverride()
{
	MaxSpeedOverride.Reset();
}

float UChaosCharacterMovementMode::GetAcceleration() const
{
	if (AccelerationOverride.IsSet())
	{
		return AccelerationOverride.GetValue();
	}
	else if (TStrongObjectPtr<const USharedChaosCharacterMovementSettings> SharedSettingsStrongPtr = SharedSettingsWeakPtr.Pin())
	{
		return SharedSettingsStrongPtr->Acceleration;
	}
	
	UE_LOGF(LogChaosMover, Warning, "Invalid acceleration on ChaosCharacterMoverComponent");
	return 0.0f;
}

void UChaosCharacterMovementMode::OverrideAcceleration(float Value)
{
	AccelerationOverride = Value;
}

void UChaosCharacterMovementMode::ClearAccelerationOverride()
{
	AccelerationOverride.Reset();
}

void UChaosCharacterMovementMode::ModifyContacts(const FMoverTimeStep& TimeStep, const UE::ChaosMover::FSimulationInputData& InputData, const UE::ChaosMover::FSimulationOutputData& OutputData, Chaos::FCollisionContactModifier& Modifier) const
{
	Super::ModifyContacts(TimeStep, InputData, OutputData, Modifier);

	if (FrictionOverrideMode == ECharacterMoverFrictionOverrideMode::DoNotOverride)
	{
		return;
	}

	check(Simulation);

	// Get the updated (character) particle
	Chaos::FGeometryParticleHandle* UpdatedParticle = nullptr;
	const FChaosMoverSimulationDefaultInputs* SimInputs = Simulation->GetLocalSimInput().FindDataByType<FChaosMoverSimulationDefaultInputs>();
	if (SimInputs)
	{
		Chaos::FReadPhysicsObjectInterface_Internal ReadInterface = Chaos::FPhysicsObjectInternalInterface::GetRead();
		UpdatedParticle = ReadInterface.GetParticle(SimInputs->PhysicsObject);
	}

	if (!UpdatedParticle)
	{
		return;
	}

	for (Chaos::FContactPairModifier& PairModifier : Modifier.GetContacts(UpdatedParticle))
	{
		bool bOverrideToZero = false;

		switch (FrictionOverrideMode)
		{
		case ECharacterMoverFrictionOverrideMode::AlwaysOverrideToZero:
			bOverrideToZero = true;
			break;
		case ECharacterMoverFrictionOverrideMode::OverrideToZeroWhenMoving:
		{
			const FCharacterDefaultInputs* CharacterInputs = InputData.InputCmd.InputCollection.FindDataByType<FCharacterDefaultInputs>();
			if (CharacterInputs)
			{
				constexpr float MinInput = 0.1f;
				bOverrideToZero = CharacterInputs->GetMoveInput().SizeSquared() > MinInput* MinInput;
			}
			break;
		}
			
		default:
			break;
		}

		if (bOverrideToZero)
		{
			PairModifier.ModifyStaticFriction(0.0f);
			PairModifier.ModifyDynamicFriction(0.0f);
		}
	}
}

void UChaosCharacterMovementMode::UpdateCurrentFloor(const FMoverTimeStep& TimeStep) const
{
	check(Simulation);

	UMoverBlackboard* SimBlackboard = Simulation->GetBlackboard_Mutable();
	if (!SimBlackboard)
	{
		return;
	}

	bool bFloorCheckSucceeded = false;

	if (const FChaosMoverSimulationDefaultInputs* SimInputs = Simulation->GetLocalSimInput().FindDataByType<FChaosMoverSimulationDefaultInputs>())
	{
		Chaos::FReadPhysicsObjectInterface_Internal ReadInterface = Chaos::FPhysicsObjectInternalInterface::GetRead();
		if (const Chaos::FPBDRigidParticleHandle* UpdatedParticle = ReadInterface.GetRigidParticle(SimInputs->PhysicsObject))
		{
			FFloorCheckResult FloorResult;
			FWaterCheckResult WaterResult;

			const float DeltaSeconds = TimeStep.StepMs * 0.001f;
			const FVector VelocityWithGravity = UpdatedParticle->GetV() + UMovementUtils::ComputeVelocityFromGravity(SimInputs->Gravity, DeltaSeconds);

			UE::ChaosMover::Utils::FFloorSweepParams SweepParams{
				.ResponseParams = SimInputs->CollisionResponseParams,
				.QueryParams = SimInputs->CollisionQueryParams,
				.Location = UpdatedParticle->GetX(),
				.DeltaPos = VelocityWithGravity * DeltaSeconds,
				.UpDir = SimInputs->UpDir,
				.World = SimInputs->World,
				.QueryDistance = 1.5f * GetTargetHeight(),
				.QueryRadius = FMath::Min(GetGroundQueryRadius(), FMath::Max(SimInputs->PawnCollisionRadius - 5.0f, 0.0f)),
				.MaxWalkSlopeCosine = GetMaxWalkSlopeCosine(),
				.TargetHeight = GetTargetHeight(),
				.CollisionChannel = SimInputs->CollisionChannel
			};

			UE::ChaosMover::Utils::FloorSweep_Internal(SweepParams, FloorResult, WaterResult);

			SimBlackboard->Set(CommonBlackboard::LastFloorResult, FloorResult);
			SimBlackboard->Set(CommonBlackboard::LastWaterResult, WaterResult);
			SimBlackboard->Set(UE::ChaosMover::Blackboard::GroundDynamicsInfo, UE::ChaosMover::FGroundDynamicsInfo(FloorResult));

			bFloorCheckSucceeded = true;
		}
	}

	if (!bFloorCheckSucceeded)
	{
		SimBlackboard->Invalidate(CommonBlackboard::LastFloorResult);
		SimBlackboard->Invalidate(CommonBlackboard::LastWaterResult);
		SimBlackboard->Invalidate(UE::ChaosMover::Blackboard::GroundDynamicsInfo);
	}
}
