// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosMover/Character/Modes/ChaosFallingMode.h"

#include "ChaosMover/ChaosMoverConsoleVariables.h"
#include "ChaosMover/ChaosMoverLog.h"
#include "ChaosMover/ChaosMoverSimulation.h"
#include "ChaosMover/Character/ChaosCharacterInputs.h"
#include "ChaosMover/Character/Settings/SharedChaosCharacterMovementSettings.h"
#include "ChaosMover/Character/Transitions/ChaosCharacterLandingCheck.h"
#include "ChaosMover/Utilities/ChaosMoverQueryUtils.h"
#include "MoveLibrary/AirMovementUtils.h"
#include "MoveLibrary/ConstrainedMoveUtils.h"
#include "MoveLibrary/FloorQueryUtils.h"
#include "MoveLibrary/MovementUtils.h"
#include "MoveLibrary/WaterMovementUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosFallingMode)

UChaosFallingMode::UChaosFallingMode(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bCancelVerticalSpeedOnLanding(true)
	, AirControlPercentage(0.4f)
	, FallingDeceleration(200.0f)
	, FallingLateralFriction(0.0f)
	, FallingAngularFriction(30.0f)
	, OverTerminalSpeedFallingDeceleration(800.0f)
	, TerminalMovementPlaneSpeed(1500.0f)
	, bShouldClampTerminalVerticalSpeed(true)
	, VerticalFallingDeceleration(4000.0f)
	, TerminalVerticalSpeed(2000.0f)
{
	bSupportsAsync = true;

	GameplayTags.AddTag(Mover_IsInAir);
	GameplayTags.AddTag(Mover_IsFalling);
	GameplayTags.AddTag(Mover_SkipVerticalAnimRootMotion);

	RadialForceLimit = 0.0f;
	SwingTorqueLimit = 3000.0f;
	TwistTorqueLimit = 0.0f;

	Transitions.Add(CreateDefaultSubobject<UChaosCharacterLandingCheck>(TEXT("DefaultLandingCheck")));
}

void UChaosFallingMode::GenerateMove_Implementation(const FMoverSimContext& SimContext, const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, FProposedMove& OutProposedMove) const
{
	QUICK_SCOPE_CYCLE_COUNTER(ChaosFallingMode_GenerateMove);

	if (!Simulation)
	{
		UE_LOGF(LogChaosMover, Warning, "No Simulation set on UChaosFallingMode. Check you have a Chaos Backend");
		return;
	}

	const FChaosMoverSimulationDefaultInputs* DefaultSimInputs = Simulation->GetLocalSimInput().FindDataByType<FChaosMoverSimulationDefaultInputs>();
	const FCharacterDefaultInputs* CharacterInputs = StartState.InputCmd.InputCollection.FindDataByType<FCharacterDefaultInputs>();
	if (!DefaultSimInputs || !CharacterInputs)
	{
		UE_LOGF(LogChaosMover, Warning, "ChaosFallingMode requires FChaosMoverSimulationDefaultInputs and FCharacterDefaultInputs");
		return;
	}

	TStrongObjectPtr<const USharedChaosCharacterMovementSettings> SharedSettingsStrongPtr = SharedSettingsWeakPtr.Pin();
	if (!SharedSettingsStrongPtr)
	{
		UE_LOGF(LogChaosMover, Warning, "ChaosFallingMode unable to read from shared settings (USharedChaosCharacterMovementSettings)");
		return;
	}
	const USharedChaosCharacterMovementSettings* SharedSettings = SharedSettingsStrongPtr.Get();
	check(SharedSettings);

	const FMoverDefaultSyncState* StartingSyncState = StartState.SyncState.SyncStateCollection.FindDataByType<FMoverDefaultSyncState>();
	check(StartingSyncState);

	const float DeltaSeconds = TimeStep.StepMs * 0.001f;

	// We don't want velocity limits to take the falling velocity component into account, since it is handled 
	// separately by the terminal velocity of the environment.
	const FVector StartVelocity = StartingSyncState->GetVelocity_WorldSpace();
	const FVector StartHorizontalVelocity = FVector::VectorPlaneProject(StartVelocity, DefaultSimInputs->UpDir);

	FFreeMoveParams Params;

	const FChaosMoverRequestedMoveInputs* RequestedMove = StartState.InputCmd.InputCollection.FindDataByType<FChaosMoverRequestedMoveInputs>();
	if (RequestedMove)
	{
		Params.MoveInputType = EMoveInputType::DirectionalIntent;
		const float MaxSpeed = GetMaxSpeed();
		const FVector DirInput = MaxSpeed > 0 ? (RequestedMove->RequestedVelocity / MaxSpeed).GetSafeNormal() : FVector::ZeroVector;
		constexpr bool bMaintainInputMagnitude = true;
		Params.MoveInput = UPlanarConstraintUtils::ConstrainDirectionToPlane(FPlanarConstraint(), DirInput, bMaintainInputMagnitude);
	}
	else
	{
		Params.MoveInputType = CharacterInputs->GetMoveInputType();
		constexpr bool bMaintainInputMagnitude = true;
		Params.MoveInput = UPlanarConstraintUtils::ConstrainDirectionToPlane(FPlanarConstraint(), CharacterInputs->GetMoveInput_WorldSpace(), bMaintainInputMagnitude);
	}

	Params.MoveInput *= AirControlPercentage;
	// Don't care about up axis input since falling - if up input matters that should probably be a different movement mode
	Params.MoveInput = FVector::VectorPlaneProject(Params.MoveInput, DefaultSimInputs->UpDir);

	FRotator IntendedOrientation_WorldSpace;

	// If there's no intent from input to change orientation, use the current orientation integrated with the current angular velocity
	if (!CharacterInputs || CharacterInputs->OrientationIntent.IsNearlyZero() || !bShouldApplyAngularVelocityToTarget)
	{
		IntendedOrientation_WorldSpace = StartingSyncState->GetOrientation_WorldSpace();

		if (ShouldCharacterRemainUpright())
		{
			IntendedOrientation_WorldSpace = FRotator(FQuat::FindBetweenVectors(IntendedOrientation_WorldSpace.RotateVector(DefaultSimInputs->UpDir), DefaultSimInputs->UpDir) * IntendedOrientation_WorldSpace.Quaternion());
		}

		const float AngularFrictionFactor = FMath::Clamp(1.0f - FallingAngularFriction * DeltaSeconds, 0.0f, 1.0f);
		FVector AngularVelocityToApply = AngularFrictionFactor * StartingSyncState->GetAngularVelocityDegrees_WorldSpace();
		if (ShouldCharacterRemainUpright())
		{
			AngularVelocityToApply = AngularVelocityToApply.ProjectOnToNormal(DefaultSimInputs->UpDir);
		}
		IntendedOrientation_WorldSpace = FRotator(UMovementUtils::ApplyAngularVelocityToQuat(IntendedOrientation_WorldSpace.Quaternion(), AngularVelocityToApply, DeltaSeconds));
	}
	else
	{
		if (ShouldCharacterRemainUpright())
		{
			IntendedOrientation_WorldSpace = CharacterInputs->GetOrientationIntentDir_WorldSpace().ToOrientationRotator();
			IntendedOrientation_WorldSpace = FRotator(FQuat::FindBetweenVectors(IntendedOrientation_WorldSpace.RotateVector(DefaultSimInputs->UpDir), DefaultSimInputs->UpDir) * IntendedOrientation_WorldSpace.Quaternion());
		}
		else
		{
			IntendedOrientation_WorldSpace = StartingSyncState->GetOrientation_WorldSpace();
		}
	}

	FQuat WorldToGravityTransform = FQuat::FindBetweenNormals(FVector::UpVector, DefaultSimInputs->UpDir);
	IntendedOrientation_WorldSpace = UMovementUtils::ApplyGravityToOrientationIntent(IntendedOrientation_WorldSpace, WorldToGravityTransform, bShouldCharacterRemainUpright);

	Params.OrientationIntent = IntendedOrientation_WorldSpace;
	Params.PriorVelocity = StartHorizontalVelocity;
	Params.PriorOrientation = StartingSyncState->GetOrientation_WorldSpace();
	Params.DeltaSeconds = DeltaSeconds;
	Params.TurningRate = SharedSettings->TurningRate;
	Params.TurningBoost = SharedSettings->TurningBoost;
	Params.MaxSpeed = GetMaxSpeed();
	Params.Acceleration = GetAcceleration();
	Params.Deceleration = FallingDeceleration;
	Params.WorldToGravityQuat = WorldToGravityTransform;
	Params.bUseAccelerationForVelocityMove = SharedSettings->bUseAccelerationForVelocityMove;
	Params.Friction = FallingLateralFriction;

	// Check if any current velocity values are over our terminal velocity - if so limit the move input in that direction and apply OverTerminalVelocityFallingDeceleration
	if (Params.MoveInput.Dot(StartVelocity) > 0 && StartHorizontalVelocity.Size() >= TerminalMovementPlaneSpeed)
	{
		Params.Deceleration = OverTerminalSpeedFallingDeceleration;
	}

	OutProposedMove = UAirMovementUtils::ComputeControlledFreeMove(Params);
	const FVector VelocityWithGravity = bIgnoreGravity ? StartVelocity : StartVelocity + UMovementUtils::ComputeVelocityFromGravity(DefaultSimInputs->Gravity, DeltaSeconds);

	//  If we are going faster than TerminalVerticalVelocity apply VerticalFallingDeceleration otherwise reset Z velocity to before we applied deceleration 
	if (VelocityWithGravity.GetAbs().Dot(DefaultSimInputs->UpDir) > TerminalVerticalSpeed)
	{
		if (bShouldClampTerminalVerticalSpeed)
		{
			const float ClampedVerticalSpeed = FMath::Sign(VelocityWithGravity.Dot(DefaultSimInputs->UpDir)) * TerminalVerticalSpeed;
			UMovementUtils::SetGravityVerticalComponent(OutProposedMove.LinearVelocity, ClampedVerticalSpeed, DefaultSimInputs->UpDir);
		}
		else
		{
			float DesiredDeceleration = FMath::Abs(TerminalVerticalSpeed - VelocityWithGravity.GetAbs().Dot(DefaultSimInputs->UpDir)) / DeltaSeconds;
			float DecelerationToApply = FMath::Min(DesiredDeceleration, VerticalFallingDeceleration);
			DecelerationToApply = FMath::Sign(VelocityWithGravity.Dot(DefaultSimInputs->UpDir)) * DecelerationToApply * DeltaSeconds;
			FVector MaxUpDirVelocity = VelocityWithGravity * DefaultSimInputs->UpDir - (DefaultSimInputs->UpDir * DecelerationToApply);

			UMovementUtils::SetGravityVerticalComponent(OutProposedMove.LinearVelocity, MaxUpDirVelocity.Dot(DefaultSimInputs->UpDir), DefaultSimInputs->UpDir);
		}
	}
	else
	{
		UMovementUtils::SetGravityVerticalComponent(OutProposedMove.LinearVelocity, VelocityWithGravity.Dot(DefaultSimInputs->UpDir), DefaultSimInputs->UpDir);
	}


	if (!UE::ChaosMover::CVars::bEnablePreSimGroundCheck)
	{
		UMoverBlackboard* SimBlackboard = Simulation->GetBlackboard_Mutable();
		if (SimBlackboard)
		{
			FFloorCheckResult FloorResult;
			FWaterCheckResult WaterResult;

			UE::ChaosMover::Utils::FFloorSweepParams SweepParams{
				.ResponseParams = DefaultSimInputs->CollisionResponseParams,
				.QueryParams = DefaultSimInputs->CollisionQueryParams,
				.Location = StartingSyncState->GetLocation_WorldSpace(),
				.DeltaPos = OutProposedMove.LinearVelocity * DeltaSeconds,
				.UpDir = DefaultSimInputs->UpDir,
				.World = DefaultSimInputs->World,
				.QueryDistance = 1.5f * GetTargetHeight(),
				.QueryRadius = FMath::Min(GetGroundQueryRadius(), FMath::Max(DefaultSimInputs->PawnCollisionRadius - 5.0f, 0.0f)),
				.MaxWalkSlopeCosine = GetMaxWalkSlopeCosine(),
				.TargetHeight = GetTargetHeight(),
				.CollisionChannel = DefaultSimInputs->CollisionChannel
			};

			UE::ChaosMover::Utils::FloorSweep_Internal(SweepParams, FloorResult, WaterResult);

			SimBlackboard->Set(CommonBlackboard::LastFloorResult, FloorResult);
			SimBlackboard->Set(CommonBlackboard::LastWaterResult, WaterResult);
			SimBlackboard->Set(UE::ChaosMover::Blackboard::GroundDynamicsInfo, UE::ChaosMover::FGroundDynamicsInfo(FloorResult));
		}
	}
}

void UChaosFallingMode::SimulationTick_Implementation(const FSimulationTickParams& Params, FMoverTickEndData& OutputState)
{
	QUICK_SCOPE_CYCLE_COUNTER(ChaosFallingMode_SimulationTick);

	if (!Simulation)
	{
		UE_LOGF(LogChaosMover, Warning, "No Simulation set on UChaosFallingMode. Check you have a Chaos Backend");
		return;
	}

	const FChaosMoverSimulationDefaultInputs* DefaultSimInputs = Simulation->GetLocalSimInput().FindDataByType<FChaosMoverSimulationDefaultInputs>();
	if (!DefaultSimInputs)
	{
		UE_LOGF(LogChaosMover, Warning, "ChaosFallingMode requires FChaosMoverSimulationDefaultInputs");
		return;
	}

	const FMoverDefaultSyncState* StartingSyncState = Params.StartState.SyncState.SyncStateCollection.FindDataByType<FMoverDefaultSyncState>();
	check(StartingSyncState);

	FProposedMove ProposedMove = Params.ProposedMove;
	FMoverDefaultSyncState& OutputSyncState = OutputState.SyncState.SyncStateCollection.FindOrAddMutableDataByType<FMoverDefaultSyncState>();

	const float DeltaSeconds = Params.TimeStep.StepMs * 0.001f;

	const FRotator TargetOrient = UMovementUtils::ApplyAngularVelocityToRotator(StartingSyncState->GetOrientation_WorldSpace(), ProposedMove.AngularVelocityDegrees, DeltaSeconds);

	// The physics simulation applies Z-only gravity acceleration via physics volumes, so we need to account for it here 
	FVector TargetVel = ProposedMove.LinearVelocity - DefaultSimInputs->PhysicsObjectGravity * FVector::UpVector * DeltaSeconds;
	FVector TargetPos = StartingSyncState->GetLocation_WorldSpace() + TargetVel * DeltaSeconds;

	OutputState.MovementEndState.RemainingMs = 0.0f;
	OutputState.MovementEndState.NextModeName = Params.StartState.SyncState.MovementMode;

	FVector TargetAngularVelocity = StartingSyncState->GetAngularVelocityDegrees_WorldSpace();
	if (DeltaSeconds > UE_SMALL_NUMBER)
	{
		const FQuat InitialQuat = StartingSyncState->GetOrientation_WorldSpace().Quaternion();
		FQuat TgtQuat = TargetOrient.Quaternion();
		TgtQuat.EnforceShortestArcWith(InitialQuat);
		const FQuat QuatRotation = TgtQuat * InitialQuat.Inverse();
		const FVector AngularDisplacement = QuatRotation.ToRotationVector();
		TargetAngularVelocity = FMath::RadiansToDegrees(AngularDisplacement / DeltaSeconds);
	}
	
	OutputSyncState.MoveDirectionIntent = ProposedMove.bHasDirIntent ? ProposedMove.DirectionIntent : FVector::ZeroVector;
	OutputSyncState.SetTransforms_WorldSpace(
		TargetPos,
		TargetOrient,
		TargetVel,
		TargetAngularVelocity);
}