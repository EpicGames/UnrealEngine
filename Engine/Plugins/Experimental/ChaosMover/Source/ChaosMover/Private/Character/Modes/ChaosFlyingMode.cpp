// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosMover/Character/Modes/ChaosFlyingMode.h"

#include "ChaosMover/ChaosMoverLog.h"
#include "ChaosMover/ChaosMoverSimulation.h"
#include "ChaosMover/Character/Settings/SharedChaosCharacterMovementSettings.h"
#include "ChaosMover/Character/Transitions/ChaosCharacterLandingCheck.h"
#include "MoveLibrary/AirMovementUtils.h"
#include "MoveLibrary/MovementUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosFlyingMode)

UChaosFlyingMode::UChaosFlyingMode(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSupportsAsync = true;

	GameplayTags.AddTag(Mover_IsInAir);
	GameplayTags.AddTag(Mover_IsFlying);

	RadialForceLimit = 0.0f;
	SwingTorqueLimit = 3000.0f;
	TwistTorqueLimit = 0.0f;
}

void UChaosFlyingMode::GenerateMove_Implementation(const FMoverSimContext& SimContext, const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, FProposedMove& OutProposedMove) const
{
	QUICK_SCOPE_CYCLE_COUNTER(ChaosFlyingMode_GenerateMove);

	if (!Simulation)
	{
		UE_LOGF(LogChaosMover, Warning, "No Simulation set on UChaosFlyingMode. Check you have a Chaos Backend");
		return;
	}

	const FChaosMoverSimulationDefaultInputs* DefaultSimInputs = Simulation->GetLocalSimInput().FindDataByType<FChaosMoverSimulationDefaultInputs>();
	const FCharacterDefaultInputs* CharacterInputs = StartState.InputCmd.InputCollection.FindDataByType<FCharacterDefaultInputs>();
	if (!DefaultSimInputs || !CharacterInputs)
	{
		UE_LOGF(LogChaosMover, Warning, "ChaosFlyingMode requires FChaosMoverSimulationDefaultInputs and FCharacterDefaultInputs");
		return;
	}

	TStrongObjectPtr<const USharedChaosCharacterMovementSettings> SharedSettingsStrongPtr = SharedSettingsWeakPtr.Pin();
	if (!SharedSettingsStrongPtr)
	{
		UE_LOGF(LogChaosMover, Warning, "ChaosFlyingMode unable to read from shared settings (USharedChaosCharacterMovementSettings)");
		return;
	}
	const USharedChaosCharacterMovementSettings* SharedSettings = SharedSettingsStrongPtr.Get();
	check(SharedSettings);

	const FMoverDefaultSyncState* StartingSyncState = StartState.SyncState.SyncStateCollection.FindDataByType<FMoverDefaultSyncState>();
	check(StartingSyncState);

	const float DeltaSeconds = TimeStep.StepMs * 0.001f;

	FFreeMoveParams Params;
	if (CharacterInputs)
	{
		Params.MoveInputType = CharacterInputs->GetMoveInputType();
		Params.MoveInput = CharacterInputs->GetMoveInput_WorldSpace();
		//const bool bMaintainInputMagnitude = true;
		//Params.MoveInput = UPlanarConstraintUtils::ConstrainDirectionToPlane(FPlanarConstraint(), CharacterInputs->GetMoveInput_WorldSpace(), bMaintainInputMagnitude);
	}
	else
	{
		Params.MoveInputType = EMoveInputType::None;
		Params.MoveInput = FVector::ZeroVector;
	}

	FRotator IntendedOrientation_WorldSpace;
	// If there's no intent from input to change orientation, use the current orientation
	if (!CharacterInputs || CharacterInputs->OrientationIntent.IsNearlyZero())
	{
		IntendedOrientation_WorldSpace = StartingSyncState->GetOrientation_WorldSpace();
	}
	else
	{
		IntendedOrientation_WorldSpace = CharacterInputs->GetOrientationIntentDir_WorldSpace().ToOrientationRotator();
	}

	FQuat WorldToGravityTransform = FQuat::FindBetweenNormals(FVector::UpVector, DefaultSimInputs->UpDir);

	IntendedOrientation_WorldSpace = UMovementUtils::ApplyGravityToOrientationIntent(IntendedOrientation_WorldSpace, WorldToGravityTransform, bShouldCharacterRemainUpright);

	Params.OrientationIntent = IntendedOrientation_WorldSpace;
	Params.PriorVelocity = StartingSyncState->GetVelocity_WorldSpace();
	Params.PriorOrientation = StartingSyncState->GetOrientation_WorldSpace();
	Params.TurningRate = SharedSettings->TurningRate;
	Params.TurningBoost = SharedSettings->TurningBoost;
	Params.MaxSpeed = GetMaxSpeed();
	Params.Acceleration = GetAcceleration();
	Params.Deceleration = SharedSettings->Deceleration;
	Params.DeltaSeconds = DeltaSeconds;
	Params.WorldToGravityQuat = WorldToGravityTransform;
	Params.bUseAccelerationForVelocityMove = SharedSettings->bUseAccelerationForVelocityMove;

	OutProposedMove = UAirMovementUtils::ComputeControlledFreeMove(Params);

	// Don't do floor checks in flying mode so just clear any previous results from the blackboard
	if (UMoverBlackboard* SimBlackboard = Simulation->GetBlackboard_Mutable())
	{
		SimBlackboard->Invalidate(CommonBlackboard::LastFloorResult);
		SimBlackboard->Invalidate(CommonBlackboard::LastWaterResult);
		SimBlackboard->Invalidate(UE::ChaosMover::Blackboard::GroundDynamicsInfo);
	}
}

void UChaosFlyingMode::SimulationTick_Implementation(const FSimulationTickParams& Params, FMoverTickEndData& OutputState)
{
	QUICK_SCOPE_CYCLE_COUNTER(ChaosFlyingMode_SimulationTick);

	if (!Simulation)
	{
		UE_LOGF(LogChaosMover, Warning, "No Simulation set on UChaosFlyingMode. Check you have a Chaos Backend");
		return;
	}

	const FChaosMoverSimulationDefaultInputs* DefaultSimInputs = Simulation->GetLocalSimInput().FindDataByType<FChaosMoverSimulationDefaultInputs>();
	if (!DefaultSimInputs)
	{
		UE_LOGF(LogChaosMover, Warning, "ChaosFlyingMode requires FChaosMoverSimulationDefaultInputs");
		return;
	}

	const FMoverDefaultSyncState* StartingSyncState = Params.StartState.SyncState.SyncStateCollection.FindDataByType<FMoverDefaultSyncState>();
	check(StartingSyncState);

	FProposedMove ProposedMove = Params.ProposedMove;
	FMoverDefaultSyncState& OutputSyncState = OutputState.SyncState.SyncStateCollection.FindOrAddMutableDataByType<FMoverDefaultSyncState>();

	const float DeltaSeconds = Params.TimeStep.StepMs * 0.001f;

	const FRotator TargetOrient = UMovementUtils::ApplyAngularVelocityToRotator(StartingSyncState->GetOrientation_WorldSpace(), ProposedMove.AngularVelocityDegrees, DeltaSeconds);

	// The physics simulation applies Z-only gravity acceleration via physics volumes, so we need to account for it here 
	FVector TargetVel = ProposedMove.LinearVelocity - DefaultSimInputs->PhysicsObjectGravity * FVector::UpVector * DeltaSeconds;
	FVector TargetPos = StartingSyncState->GetLocation_WorldSpace() + TargetVel * DeltaSeconds;

	OutputState.MovementEndState.RemainingMs = 0.0f;
	OutputState.MovementEndState.NextModeName = Params.StartState.SyncState.MovementMode;
	OutputSyncState.MoveDirectionIntent = ProposedMove.bHasDirIntent ? ProposedMove.DirectionIntent : FVector::ZeroVector;
	OutputSyncState.SetTransforms_WorldSpace(
		TargetPos,
		TargetOrient,
		TargetVel,
		ProposedMove.AngularVelocityDegrees);
}

void UChaosFlyingMode::UpdateCurrentFloor(const FMoverTimeStep& TimeStep) const
{
	if (UMoverBlackboard* SimBlackboard = Simulation->GetBlackboard_Mutable())
	{
		SimBlackboard->Invalidate(CommonBlackboard::LastFloorResult);
		SimBlackboard->Invalidate(CommonBlackboard::LastWaterResult);
		SimBlackboard->Invalidate(UE::ChaosMover::Blackboard::GroundDynamicsInfo);
	}
}