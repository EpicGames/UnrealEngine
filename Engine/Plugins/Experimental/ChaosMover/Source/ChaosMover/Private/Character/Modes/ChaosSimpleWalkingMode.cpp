// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosMover/Character/Modes/ChaosSimpleWalkingMode.h"

#include "ChaosMover/ChaosMoverLog.h"
#include "ChaosMover/ChaosMoverSimulation.h"
#include "ChaosMover/ChaosMoverSimulationTypes.h"
#include "ChaosMover/Character/ChaosCharacterInputs.h"
#include "ChaosMover/Character/Settings/SharedChaosCharacterMovementSettings.h"
#include "MoveLibrary/FloorQueryUtils.h"
#include "MoveLibrary/MovementUtils.h"
#include "MoveLibrary/WaterMovementUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosSimpleWalkingMode)

void UChaosSimpleWalkingMode::GenerateMove_Implementation(const FMoverSimContext& SimContext, const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, FProposedMove& OutProposedMove) const
{
	QUICK_SCOPE_CYCLE_COUNTER(ChaosSimpleWalkingMode_GenerateMove);

	if (!Simulation)
	{
		UE_LOG(LogChaosMover, Warning, TEXT("No Simulation set on ChaosSimpleWalkingMode. Check you have a Chaos Backend"));
		return;
	}

	const FChaosMoverSimulationDefaultInputs* DefaultSimInputs = Simulation->GetLocalSimInput().FindDataByType<FChaosMoverSimulationDefaultInputs>();
	const FCharacterDefaultInputs* CharacterInputs = StartState.InputCmd.InputCollection.FindDataByType<FCharacterDefaultInputs>();
	if (!DefaultSimInputs || !CharacterInputs)
	{
		UE_LOG(LogChaosMover, Warning, TEXT("ChaosSimpleWalkingMode requires FChaosMoverSimulationDefaultInputs and FCharacterDefaultInputs"));
		return;
	}

	TStrongObjectPtr<const USharedChaosCharacterMovementSettings> SharedSettingsStrongPtr = SharedSettingsWeakPtr.Pin();
	if (!SharedSettingsStrongPtr)
	{
		UE_LOG(LogChaosMover, Warning, TEXT("ChaosSimpleWalkingMode unable to read from shared settings (USharedChaosCharacterMovementSettings)"));
		return;
	}
	const USharedChaosCharacterMovementSettings* SharedSettings = SharedSettingsStrongPtr.Get();
	check(SharedSettings);

	const FMoverDefaultSyncState* StartingSyncState = StartState.SyncState.SyncStateCollection.FindDataByType<FMoverDefaultSyncState>();
	check(StartingSyncState);

	const float DeltaSeconds = TimeStep.StepMs * 0.001f;
	if (DeltaSeconds <= FLT_EPSILON)
	{
		return;
	}

	const FVector UpDirection = DefaultSimInputs->UpDir;

	// Try to use the floor as the basis for the intended move direction (i.e. try to walk along slopes, rather than into them)
	UMoverBlackboard* SimBlackboard = Simulation->GetBlackboard_Mutable();
	FFloorCheckResult LastFloorResult;
	UE::ChaosMover::FGroundDynamicsInfo GroundDynamicsInfo;
	FVector MovementNormal = UpDirection;
	bool bWalkableFloor = false;

	if (SimBlackboard)
	{
		SimBlackboard->TryGet(CommonBlackboard::LastFloorResult, LastFloorResult);
		SimBlackboard->TryGet(UE::ChaosMover::Blackboard::GroundDynamicsInfo, GroundDynamicsInfo);

		bWalkableFloor = LastFloorResult.IsWalkableFloor();

		if (!bMaintainHorizontalGroundVelocity && bWalkableFloor)
		{
			MovementNormal = LastFloorResult.HitResult.ImpactNormal;
		}
	}

	FRotator IntendedOrientation_WorldSpace;
	if (CharacterInputs->OrientationIntent.IsNearlyZero())
	{
		IntendedOrientation_WorldSpace = StartingSyncState->GetOrientation_WorldSpace();
	}
	else
	{
		IntendedOrientation_WorldSpace = CharacterInputs->GetOrientationIntentDir_WorldSpace().ToOrientationRotator();
	}

	const FQuat WorldToGravityTransform = FQuat::FindBetweenNormals(FVector::UpVector, UpDirection);
	IntendedOrientation_WorldSpace = UMovementUtils::ApplyGravityToOrientationIntent(IntendedOrientation_WorldSpace, WorldToGravityTransform, bShouldCharacterRemainUpright);

	// Current relative (to ground) velocity
	const FVector CurrentVelocity = StartingSyncState->GetVelocity_WorldSpace() - GroundDynamicsInfo.LinearVelocity;
	const FVector CurrentVelocityInMovementPlane = FVector::VectorPlaneProject(CurrentVelocity, MovementNormal);

	// Desired velocity from inputs
	FVector DesiredVelocity = CharacterInputs->GetMoveInput_WorldSpace();
	const EMoveInputType MoveInputType = CharacterInputs->GetMoveInputType();

	// Remove vertical component (relative to up) but preserve input magnitude.
	const float DesiredVelMag = DesiredVelocity.Length();
	DesiredVelocity -= DesiredVelocity.ProjectOnTo(UpDirection);
	const float DesiredVel2DSq = DesiredVelocity.SquaredLength();
	if (DesiredVel2DSq > UE_SMALL_NUMBER)
	{
		DesiredVelocity *= DesiredVelMag / FMath::Sqrt(DesiredVel2DSq);
	}

	// Constrain to the current walking surface plane (eg slopes) while preserving the intended input magnitude.
	const float DesiredVelMagAfterUp = DesiredVelocity.Length();
	DesiredVelocity = FVector::VectorPlaneProject(DesiredVelocity, MovementNormal);
	const float DesiredVelPlaneSq = DesiredVelocity.SquaredLength();
	if (DesiredVelPlaneSq > UE_SMALL_NUMBER)
	{
		DesiredVelocity *= DesiredVelMagAfterUp / FMath::Sqrt(DesiredVelPlaneSq);
	}

	// Convert to a target velocity in the movement plane.
	const float MaxMoveSpeed = GetMaxSpeed();
	switch (MoveInputType)
	{
	case EMoveInputType::DirectionalIntent:
		OutProposedMove.DirectionIntent = DesiredVelocity;
		DesiredVelocity *= MaxMoveSpeed;
		break;
	case EMoveInputType::Velocity:
		DesiredVelocity = DesiredVelocity.GetClampedToMaxSize(MaxMoveSpeed);
		OutProposedMove.DirectionIntent = MaxMoveSpeed > UE_KINDA_SMALL_NUMBER ? (DesiredVelocity / MaxMoveSpeed) : FVector::ZeroVector;
		break;
	case EMoveInputType::None:
	case EMoveInputType::Invalid:
	default:
		OutProposedMove.DirectionIntent = FVector::ZeroVector;
		DesiredVelocity = FVector::ZeroVector;
		break;
	}

	DesiredVelocity = FVector::VectorPlaneProject(DesiredVelocity, MovementNormal);
	OutProposedMove.bHasDirIntent = !OutProposedMove.DirectionIntent.IsNearlyZero();

	const FQuat CurrentFacing = StartingSyncState->GetOrientation_WorldSpace().Quaternion();
	const FQuat DesiredFacing = IntendedOrientation_WorldSpace.Quaternion();

	OutProposedMove.LinearVelocity = CurrentVelocityInMovementPlane;
	OutProposedMove.AngularVelocityDegrees = StartingSyncState->GetAngularVelocityDegrees_WorldSpace();

	// Allow derived classes to mutate StartState (for persistent smoothing state), matching Mover's pattern.
	UChaosSimpleWalkingMode* MutableMode = const_cast<UChaosSimpleWalkingMode*>(this);
	FMoverTickStartData& MutableStartState = const_cast<FMoverTickStartData&>(StartState);
	MutableMode->GenerateWalkMove(MutableStartState, DeltaSeconds, DesiredVelocity, DesiredFacing, CurrentFacing, OutProposedMove.AngularVelocityDegrees, OutProposedMove.LinearVelocity);

	// Maintain horizontal velocity up ramps if requested (same behavior as ChaosWalkingMode).
	if (bMaintainHorizontalGroundVelocity && bWalkableFloor)
	{
		const double CosAngleBetweenUpAndNormal = UpDirection.Dot(LastFloorResult.HitResult.ImpactNormal);
		if (CosAngleBetweenUpAndNormal > UE_SMALL_NUMBER)
		{
			OutProposedMove.LinearVelocity -= UpDirection * OutProposedMove.LinearVelocity.Dot(LastFloorResult.HitResult.ImpactNormal) / CosAngleBetweenUpAndNormal;
		}
	}

	// Special case for standing on a rotating platform: if there is no input and the proposed move is small enough, snap it to zero.
	if ((GroundDynamicsInfo.AngularVelocityDegrees.Dot(UpDirection) > 1.0f) && CharacterInputs->GetMoveInput_WorldSpace().IsNearlyZero())
	{
		const float ErrorLimit = MaxMoveSpeed * 0.1f;
		if (OutProposedMove.LinearVelocity.SizeSquared() < ErrorLimit * ErrorLimit)
		{
			OutProposedMove.LinearVelocity = FVector::ZeroVector;
		}
	}

	// Update the floor result and check the proposed move to prevent movement onto unwalkable surfaces (same behavior as ChaosWalkingMode).
	if (SimBlackboard)
	{
		FVector OutDeltaPos = FVector::ZeroVector;
		FFloorCheckResult FloorResult;
		FWaterCheckResult WaterResult;
		GetFloorAndCheckMovement(*StartingSyncState, OutProposedMove, *DefaultSimInputs, DeltaSeconds, FloorResult, WaterResult, OutDeltaPos);

		OutProposedMove.LinearVelocity = (DeltaSeconds > 0.0f) ? (OutDeltaPos / DeltaSeconds) : FVector::ZeroVector;

		GroundDynamicsInfo = UE::ChaosMover::FGroundDynamicsInfo(FloorResult);
		SimBlackboard->Set(CommonBlackboard::LastFloorResult, FloorResult);
		SimBlackboard->Set(UE::ChaosMover::Blackboard::GroundDynamicsInfo, GroundDynamicsInfo);
		SimBlackboard->Set(CommonBlackboard::LastWaterResult, WaterResult);
	}
}

void UChaosSimpleWalkingMode::GenerateWalkMove_Implementation(FMoverTickStartData& StartState, float DeltaSeconds, const FVector& DesiredVelocity,
	const FQuat& DesiredFacing, const FQuat& CurrentFacing, FVector& InOutAngularVelocityDegrees, FVector& InOutVelocity)
{
	InOutVelocity = DesiredVelocity;

	const FQuat ToFacing = CurrentFacing.Inverse() * DesiredFacing;
	InOutAngularVelocityDegrees = DeltaSeconds > 0.0f ? FMath::RadiansToDegrees(ToFacing.ToRotationVector() / DeltaSeconds) : FVector::ZeroVector;
}

