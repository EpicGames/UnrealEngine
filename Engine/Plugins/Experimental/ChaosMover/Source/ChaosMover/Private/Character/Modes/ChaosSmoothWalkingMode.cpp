// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosMover/Character/Modes/ChaosSmoothWalkingMode.h"
#include "ChaosSmoothWalkingState.h"

#include "Math/SpringMath.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosSmoothWalkingMode)

void UChaosSmoothWalkingMode::SimulationTick_Implementation(const FSimulationTickParams& Params, FMoverTickEndData& OutputState)
{
	Super::SimulationTick_Implementation(Params, OutputState);

	// Copy spring state into output state (GenerateWalkMove mutates the start state collection).
	if (const FChaosSmoothWalkingState* InState = Params.StartState.SyncState.SyncStateCollection.FindDataByType<FChaosSmoothWalkingState>())
	{
		FChaosSmoothWalkingState& OutState = OutputState.SyncState.SyncStateCollection.FindOrAddMutableDataByType<FChaosSmoothWalkingState>();
		OutState = *InState;
	}
}

void UChaosSmoothWalkingMode::GenerateWalkMove_Implementation(FMoverTickStartData& StartState, float DeltaSeconds, const FVector& DesiredVelocity,
	const FQuat& DesiredFacing, const FQuat& CurrentFacing, FVector& InOutAngularVelocityDegrees, FVector& InOutVelocity)
{
	if (DeltaSeconds <= FLT_EPSILON)
	{
		return;
	}

	const FMoverDefaultSyncState* StartingSyncState = StartState.SyncState.SyncStateCollection.FindDataByType<FMoverDefaultSyncState>();
	if (!ensure(StartingSyncState))
	{
		return;
	}

	// Find or add persistent spring state
	bool bStateAdded = false;
	FChaosSmoothWalkingState& SpringState = StartState.SyncState.SyncStateCollection.FindOrAddMutableDataByType<FChaosSmoothWalkingState>(bStateAdded);

	if (bStateAdded)
	{
		SpringState.SpringVelocity = InOutVelocity;
		SpringState.SpringAcceleration = FVector::ZeroVector;
		SpringState.IntermediateVelocity = InOutVelocity;
		SpringState.IntermediateFacing = CurrentFacing;
		SpringState.IntermediateAngularVelocity = FVector::ZeroVector;
	}

	// How much did our internal spring velocity match the actual velocity last frame?
	const float VelocityMatch = FMath::Clamp(
		SpringState.SpringVelocity.Dot(InOutVelocity) /
		FMath::Max(InOutVelocity.Length() * SpringState.SpringVelocity.Length(), UE_SMALL_NUMBER),
		0.0f, 1.0f);

	// If outside influences (collisions, pushes) changed our velocity, smoothly reset intermediate velocity towards it.
	FMath::ExponentialSmoothingApprox(
		SpringState.IntermediateVelocity,
		InOutVelocity,
		DeltaSeconds,
		(OutsideInfluenceSmoothingTime + UE_KINDA_SMALL_NUMBER) / FMath::Max(1.0f - VelocityMatch, UE_KINDA_SMALL_NUMBER));

	// Update spring velocity based on real velocity
	SpringState.SpringVelocity = InOutVelocity;

	// Rotate intermediate velocity towards desired direction
	if (TurningStrength > 0.0f && !DesiredVelocity.IsNearlyZero())
	{
		FMath::ExponentialSmoothingApprox(
			SpringState.IntermediateVelocity,
			DesiredVelocity.GetSafeNormal() * SpringState.IntermediateVelocity.Length(),
			DeltaSeconds,
			SpringMath::StrengthToSmoothingTime(TurningStrength));
	}

	// Determine whether we are accelerating or decelerating
	const bool bIsAccelerating = (1.01f * DesiredVelocity.SquaredLength()) > SpringState.SpringVelocity.SquaredLength();
	const float LateralAccelerationMagnitude = bIsAccelerating ? (1.0f - DirectionalAccelerationFactor) * Acceleration : Deceleration;
	const float DirectionalAccelerationMagnitude = bIsAccelerating ? DirectionalAccelerationFactor * Acceleration : 0.0f;

	const float PreviousVelocityLength = SpringState.IntermediateVelocity.Length();
	const FVector VelocityDifference = DesiredVelocity - SpringState.IntermediateVelocity;

	const FVector LateralAccelerationVector =
		VelocityDifference.GetSafeNormal() *
		FMath::Min(LateralAccelerationMagnitude, VelocityDifference.Length() / FMath::Max(DeltaSeconds, UE_SMALL_NUMBER));

	const FVector DirectionalAccelerationVector = DesiredVelocity.GetSafeNormal() * DirectionalAccelerationMagnitude;
	const FVector DesiredAcceleration = LateralAccelerationVector + DirectionalAccelerationVector;

	// Integrate desired acceleration to estimate the next intermediate target velocity (avoid overshoot).
	FVector NextVelocity = (VelocityDifference.Dot(DesiredAcceleration * DeltaSeconds) < VelocityDifference.SquaredLength())
		? (SpringState.IntermediateVelocity + DesiredAcceleration * DeltaSeconds)
		: DesiredVelocity;

	NextVelocity = NextVelocity.GetClampedToMaxSize(FMath::Max(PreviousVelocityLength, DesiredVelocity.Length()));

	const float VelocitySmoothingTime = bIsAccelerating ? AccelerationSmoothingTime : DecelerationSmoothingTime;
	const float VelocitySmoothingCompensation = bIsAccelerating ? AccelerationSmoothingCompensation : DecelerationSmoothingCompensation;

	const float LagSeconds = DeltaSeconds + (VelocitySmoothingCompensation * VelocitySmoothingTime);

	FVector TrackVelocity = (VelocityDifference.Dot(DesiredAcceleration * LagSeconds) < VelocityDifference.SquaredLength())
		? (SpringState.IntermediateVelocity + DesiredAcceleration * LagSeconds)
		: DesiredVelocity;

	TrackVelocity = TrackVelocity.GetClampedToMaxSize(FMath::Max(PreviousVelocityLength, DesiredVelocity.Length()));

	// Apply critical damping spring to velocity
	SpringMath::CriticalSpringDamper(SpringState.SpringVelocity, SpringState.SpringAcceleration, TrackVelocity, VelocitySmoothingTime, DeltaSeconds);

	// Deadzone snap
	if ((DesiredVelocity - SpringState.SpringVelocity).SquaredLength() < FMath::Square(VelocityDeadzoneThreshold))
	{
		SpringState.SpringVelocity = DesiredVelocity;

		if (SpringState.SpringAcceleration.SquaredLength() < FMath::Square(AccelerationDeadzoneThreshold))
		{
			SpringState.SpringAcceleration = FVector::ZeroVector;
		}
	}

	InOutVelocity = SpringState.SpringVelocity;
	SpringState.IntermediateVelocity = NextVelocity;

	FVector CurrentAngularVelocityRadians = FMath::DegreesToRadians(InOutAngularVelocityDegrees);
	FQuat UpdatedFacing = CurrentFacing;

	// Facing smoothing: single or double spring
	if (bSmoothFacingWithDoubleSpring)
	{
		SpringMath::CriticalSpringDamperQuat(SpringState.IntermediateFacing, SpringState.IntermediateAngularVelocity, DesiredFacing, FacingSmoothingTime / 2.0f, DeltaSeconds);
		SpringMath::CriticalSpringDamperQuat(UpdatedFacing, CurrentAngularVelocityRadians, SpringState.IntermediateFacing, FacingSmoothingTime / 2.0f, DeltaSeconds);
	}
	else
	{
		SpringState.IntermediateFacing = DesiredFacing;
		SpringState.IntermediateAngularVelocity = CurrentAngularVelocityRadians;
		SpringMath::CriticalSpringDamperQuat(UpdatedFacing, CurrentAngularVelocityRadians, DesiredFacing, FacingSmoothingTime, DeltaSeconds);
	}

	// Deadzone snap for facing
	if (DesiredFacing.AngularDistance(UpdatedFacing) < FMath::DegreesToRadians(FacingDeadzoneThreshold))
	{
		CurrentAngularVelocityRadians = DeltaSeconds > 0.0f
			? ((CurrentFacing.Inverse() * UpdatedFacing).GetShortestArcWith(FQuat::Identity)).ToRotationVector() / DeltaSeconds
			: FVector::ZeroVector;

		SpringState.IntermediateFacing = DesiredFacing;

		if (CurrentAngularVelocityRadians.SquaredLength() < FMath::Square(FMath::DegreesToRadians(AngularVelocityDeadzoneThreshold)))
		{
			SpringState.IntermediateAngularVelocity = FVector::ZeroVector;
		}
	}

	InOutAngularVelocityDegrees = FMath::RadiansToDegrees(CurrentAngularVelocityRadians);
}

