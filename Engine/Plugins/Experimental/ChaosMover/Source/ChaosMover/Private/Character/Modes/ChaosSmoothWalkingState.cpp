// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosSmoothWalkingState.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosSmoothWalkingState)

namespace ChaosSmoothWalkingStateErrorTolerance
{
	constexpr float VelocityErrorTolerance = 10.f;
	constexpr float AngularVelocityErrorTolerance = 10.f;
	constexpr float AccelerationErrorTolerance = 50.f;
	constexpr float FacingDegreeErrorTolerance = 10.0f;
}

UScriptStruct* FChaosSmoothWalkingState::GetScriptStruct() const
{
	return StaticStruct();
}

FMoverDataStructBase* FChaosSmoothWalkingState::Clone() const
{
	return new FChaosSmoothWalkingState(*this);
}

bool FChaosSmoothWalkingState::NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess)
{
	bool bSuccess = Super::NetSerialize(Ar, Map, bOutSuccess);

	Ar << SpringVelocity;
	Ar << SpringAcceleration;
	Ar << IntermediateVelocity;
	Ar << IntermediateFacing;
	Ar << IntermediateAngularVelocity;

	return bSuccess;
}

void FChaosSmoothWalkingState::ToString(FAnsiStringBuilderBase& Out) const
{
	Super::ToString(Out);

	Out.Appendf("SpringVelocity=%s SpringAcceleration=%s IntVel=%s IntFac=%s IntAng=%s\n",
		*SpringVelocity.ToCompactString(),
		*SpringAcceleration.ToCompactString(),
		*IntermediateVelocity.ToCompactString(),
		*IntermediateFacing.ToString(),
		*IntermediateAngularVelocity.ToString());
}

bool FChaosSmoothWalkingState::ShouldReconcile(const FMoverDataStructBase& AuthorityState) const
{
	const FChaosSmoothWalkingState* Authority = static_cast<const FChaosSmoothWalkingState*>(&AuthorityState);

	return (!(SpringVelocity - Authority->SpringVelocity).IsNearlyZero(ChaosSmoothWalkingStateErrorTolerance::VelocityErrorTolerance) ||
			!(SpringAcceleration - Authority->SpringAcceleration).IsNearlyZero(ChaosSmoothWalkingStateErrorTolerance::AccelerationErrorTolerance) ||
			!(IntermediateVelocity - Authority->IntermediateVelocity).IsNearlyZero(ChaosSmoothWalkingStateErrorTolerance::VelocityErrorTolerance) ||
			(FMath::RadiansToDegrees(IntermediateFacing.AngularDistance(Authority->IntermediateFacing)) > ChaosSmoothWalkingStateErrorTolerance::FacingDegreeErrorTolerance) ||
			!(IntermediateAngularVelocity - Authority->IntermediateAngularVelocity).IsNearlyZero(ChaosSmoothWalkingStateErrorTolerance::AngularVelocityErrorTolerance));
}

void FChaosSmoothWalkingState::Interpolate(const FMoverDataStructBase& From, const FMoverDataStructBase& To, float Pct)
{
	const FChaosSmoothWalkingState* FromState = static_cast<const FChaosSmoothWalkingState*>(&From);
	const FChaosSmoothWalkingState* ToState = static_cast<const FChaosSmoothWalkingState*>(&To);

	SpringVelocity = FMath::Lerp(FromState->SpringVelocity, ToState->SpringVelocity, Pct);
	SpringAcceleration = FMath::Lerp(FromState->SpringAcceleration, ToState->SpringAcceleration, Pct);
	IntermediateVelocity = FMath::Lerp(FromState->IntermediateVelocity, ToState->IntermediateVelocity, Pct);
	IntermediateFacing = FQuat::Slerp(FromState->IntermediateFacing, ToState->IntermediateFacing, Pct);
	IntermediateAngularVelocity = FMath::Lerp(FromState->IntermediateAngularVelocity, ToState->IntermediateAngularVelocity, Pct);
}

