// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/StringBuilder.h"
#include "MoverTypes.h"

#include "ChaosSmoothWalkingState.generated.h"

class UPackageMap;

/**
 * Internal state data for Chaos smooth walking.
 * Stored in the SyncStateCollection to remain stable across rollback/resim and replication.
 */
USTRUCT()
struct FChaosSmoothWalkingState : public FMoverDataStructBase
{
	GENERATED_BODY()

	virtual UScriptStruct* GetScriptStruct() const override;
	virtual FMoverDataStructBase* Clone() const override;
	virtual bool NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess) override;
	virtual void ToString(FAnsiStringBuilderBase& Out) const override;
	virtual bool ShouldReconcile(const FMoverDataStructBase& AuthorityState) const override;
	virtual void Interpolate(const FMoverDataStructBase& From, const FMoverDataStructBase& To, float Pct) override;

	// Velocity of internal velocity spring
	UPROPERTY(BlueprintReadOnly, Category = "ChaosMover|Experimental")
	FVector SpringVelocity = FVector::ZeroVector;

	// Acceleration of internal velocity spring
	UPROPERTY(BlueprintReadOnly, Category = "ChaosMover|Experimental")
	FVector SpringAcceleration = FVector::ZeroVector;

	// Intermediate velocity which the velocity spring tracks as a target
	UPROPERTY(BlueprintReadOnly, Category = "ChaosMover|Experimental")
	FVector IntermediateVelocity = FVector::ZeroVector;

	// Intermediate facing direction when using a double spring
	UPROPERTY(BlueprintReadOnly, Category = "ChaosMover|Experimental")
	FQuat IntermediateFacing = FQuat::Identity;

	// Angular velocity of the intermediate spring when using a double spring
	UPROPERTY(BlueprintReadOnly, Category = "ChaosMover|Experimental")
	FVector IntermediateAngularVelocity = FVector::ZeroVector;
};

