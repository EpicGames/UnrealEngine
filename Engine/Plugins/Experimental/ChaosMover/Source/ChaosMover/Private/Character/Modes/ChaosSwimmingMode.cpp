// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosMover/Character/Modes/ChaosSwimmingMode.h"

#include "ChaosMover/ChaosMoverConsoleVariables.h"
#include "ChaosMover/ChaosMoverLog.h"
#include "ChaosMover/ChaosMoverSimulation.h"
#include "ChaosMover/Character/Settings/SharedChaosCharacterMovementSettings.h"
#include "ChaosMover/Character/Transitions/ChaosCharacterJumpCheck.h"
#include "ChaosMover/Character/Transitions/ChaosCharacterWaterCheck.h"
#include "ChaosMover/Utilities/ChaosMoverQueryUtils.h"
#include "MoveLibrary/MovementUtils.h"
#include "MoveLibrary/FloorQueryUtils.h"
#include "MoveLibrary/WaterMovementUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosSwimmingMode)

UChaosSwimmingMode::UChaosSwimmingMode(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSupportsAsync = true;
	GameplayTags.AddTag(Mover_IsSwimming);

	RadialForceLimit = 0.0f;
	SwingTorqueLimit = 3000.0f;
	TwistTorqueLimit = 0.0f;

	Transitions.Add(CreateDefaultSubobject<UChaosCharacterWaterCheck>(TEXT("DefaultWaterCheck")));
}

void UChaosSwimmingMode::GenerateMove_Implementation(const FMoverSimContext& SimContext, const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, FProposedMove& OutProposedMove) const
{
	QUICK_SCOPE_CYCLE_COUNTER(ChaosSwimmingMode_GenerateMove);

	if (!Simulation)
	{
		UE_LOGF(LogChaosMover, Warning, "No Simulation set on UChaosSwimmingMode. Check you have a Chaos Backend");
		return;
	}

	const FChaosMoverSimulationDefaultInputs* DefaultSimInputs = Simulation->GetLocalSimInput().FindDataByType<FChaosMoverSimulationDefaultInputs>();
	if (!ensureMsgf(DefaultSimInputs, TEXT("ChaosSwimmingMode requires FChaosMoverSimulationDefaultInputs")))
	{
		return;
	}
	const FCharacterDefaultInputs* CharacterInputs = StartState.InputCmd.InputCollection.FindDataByType<FCharacterDefaultInputs>();
	if (!ensureMsgf(CharacterInputs, TEXT("ChaosSwimmingMode requires FCharacterDefaultInputs")))
	{
		return;
	}	
	const FMoverDefaultSyncState* StartingSyncState = StartState.SyncState.SyncStateCollection.FindDataByType<FMoverDefaultSyncState>();
	if (!ensureMsgf(StartingSyncState, TEXT("ChaosSwimmingMode requires FMoverDefaultSyncState")))
	{
		return;
	}

	TStrongObjectPtr<const USharedChaosCharacterMovementSettings> SharedSettingsStrongPtr = SharedSettingsWeakPtr.Pin();
	if (!SharedSettingsStrongPtr)
	{
		UE_LOGF(LogChaosMover, Warning, "ChaosSwimmingMode unable to read from shared settings (USharedChaosCharacterMovementSettings)");
		return;
	}
	const USharedChaosCharacterMovementSettings* SharedSettings = SharedSettingsStrongPtr.Get();
	check(SharedSettings);

	const float DeltaSeconds = TimeStep.StepMs * 0.001f;	
	const float HalfHeight = DefaultSimInputs->PawnCollisionHalfHeight;
	FVector Velocity = StartingSyncState->GetVelocity_WorldSpace();
	
	UMoverBlackboard* SimBlackboard = Simulation->GetBlackboard_Mutable();
	FFloorCheckResult FloorResult;
	FWaterCheckResult WaterResult;
	
	if (UE::ChaosMover::CVars::bEnablePreSimGroundCheck)
	{
		SimBlackboard->TryGet(CommonBlackboard::LastFloorResult, FloorResult);
		SimBlackboard->TryGet(CommonBlackboard::LastWaterResult, WaterResult);
	}
	else
	{
		SimBlackboard->Invalidate(CommonBlackboard::LastFloorResult);
		SimBlackboard->Invalidate(CommonBlackboard::LastWaterResult);

		const float QueryDistance = 2.0f * HalfHeight;
		FVector DeltaPos = Velocity * DeltaSeconds;
		UE::ChaosMover::Utils::FFloorSweepParams SweepParams{
			.ResponseParams = DefaultSimInputs->CollisionResponseParams,
			.QueryParams = DefaultSimInputs->CollisionQueryParams,
			.Location = StartingSyncState->GetLocation_WorldSpace(),
			.DeltaPos = DeltaPos,
			.UpDir = DefaultSimInputs->UpDir,
			.World = DefaultSimInputs->World,
			.QueryDistance = QueryDistance,
			.QueryRadius = FMath::Min(GetGroundQueryRadius(), FMath::Max(DefaultSimInputs->PawnCollisionRadius - 5.0f, 0.0f)),
			.MaxWalkSlopeCosine = GetMaxWalkSlopeCosine(),
			.TargetHeight = GetTargetHeight(),
			.CollisionChannel = DefaultSimInputs->CollisionChannel
		};

		UE::ChaosMover::Utils::FloorSweep_Internal(SweepParams, FloorResult, WaterResult);

		SimBlackboard->Set(CommonBlackboard::LastFloorResult, FloorResult);
		SimBlackboard->Set(UE::ChaosMover::Blackboard::GroundDynamicsInfo, UE::ChaosMover::FGroundDynamicsInfo(FloorResult));
	}

	if (WaterResult.IsSwimmableVolume())
	{
		FUpdateWaterSplineDataParams SplineParams;
		SplineParams.TargetImmersionDepth = SwimmingIdealImmersionDepth;
		SplineParams.WaterVelocityDepthForMax = SurfaceSwimmingWaterControlSettings.WaterVelocityDepthForMax;
		SplineParams.WaterVelocityMinMultiplier = SurfaceSwimmingWaterControlSettings.WaterVelocityMinMultiplier;
		SplineParams.PlayerVelocity = StartingSyncState->GetVelocity_WorldSpace();
		SplineParams.CapsuleHalfHeight = HalfHeight;
		SplineParams.PlayerLocation = StartingSyncState->GetLocation_WorldSpace();

		UWaterMovementUtils::UpdateWaterSplineData(SplineParams, WaterResult);
	}

	SimBlackboard->Set(CommonBlackboard::LastWaterResult, WaterResult);
	
	// Buoyancy Bobbing
	{
		const FWaterFlowSplineData& WaterData = WaterResult.WaterSplineData;
		
		const float ReciprocalHeight = 0.5f / HalfHeight;
		const float ReciprocalOriginalHeight = 0.5f / OriginalHalfHeight;
		const FVector Location = StartingSyncState->GetLocation_WorldSpace();

		const float ImmersionDepth = WaterData.ImmersionDepth + HalfHeight;
		const float ImmersionPercent = FMath::Clamp(ImmersionDepth * ReciprocalHeight, UE_KINDA_SMALL_NUMBER, 1.f);
		const float IdealDepth = SwimmingIdealImmersionDepth + HalfHeight;
		const float IdealImmersionPercent = FMath::Clamp(IdealDepth * ReciprocalOriginalHeight, UE_KINDA_SMALL_NUMBER, 1.f);

		const float GravityForce = DefaultSimInputs->Gravity.Z;
		
		// Using IdealDepth, we compute the Buoyancy such that (Buoyancy * IdealImmersionPercent) = -G force
		const float BuoyancyForce = -GravityForce / IdealImmersionPercent;
	
		// Now calculate the actual force in the correct ratio
		float BobbingForce = (BuoyancyForce * ImmersionPercent) + GravityForce;
		const float MaxForce = SurfaceSwimmingWaterControlSettings.BobbingMaxForce;
		BobbingForce = FMath::Clamp(BobbingForce, -MaxForce, MaxForce);

		Velocity.Z += BobbingForce * DeltaSeconds;

		// Vertical fluid friction for bobbing
		if (!FMath::IsNearlyZero(Velocity.Z, (FVector::FReal)0.1f))
		{
			const float IdealDepthTolerance = SurfaceSwimmingWaterControlSettings.BobbingIdealDepthTolerance;
			if (FMath::Sign(Velocity.Z) != FMath::Sign(BobbingForce) || FMath::IsNearlyEqual(ImmersionDepth, IdealDepth, IdealDepthTolerance))
			{
				float FluidFriction = 0.f;
				float ExpDrag = 0.f;
				const bool bFullySubmerged = ImmersionPercent > 1.f;
				if (Velocity.Z > 0)
				{
					FluidFriction = SurfaceSwimmingWaterControlSettings.BobbingFrictionUp;
					ExpDrag = SurfaceSwimmingWaterControlSettings.BobbingExpDragUp;
				}
				else
				{
					// Different drag when fully immersed and moving down (mainly controls how far you go when falling in fast)
					FluidFriction = bFullySubmerged ? SurfaceSwimmingWaterControlSettings.BobbingFrictionDownSubmerged : SurfaceSwimmingWaterControlSettings.BobbingFrictionDown;
					ExpDrag = bFullySubmerged ? SurfaceSwimmingWaterControlSettings.BobbingExpDragDownSubmerged : SurfaceSwimmingWaterControlSettings.BobbingExpDragDown;
				}
	
				FluidFriction *= SurfaceSwimmingWaterControlSettings.BobbingFrictionMultiplier;
				ExpDrag *= SurfaceSwimmingWaterControlSettings.BobbingExpDragMultiplier;
	
				Velocity.Z = Velocity.Z * (1.f - FMath::Min(FluidFriction * DeltaSeconds, 1.f));
				Velocity.Z = Velocity.Z * (1.f - FMath::Min(FMath::Abs(Velocity.Z) * FMath::Square(ExpDrag) * DeltaSeconds, 1.f));
			}
		}
	}

	// Vertical speed limit in Water
	{
		const float MaxVerticalWaterSpeedUp = SurfaceSwimmingWaterControlSettings.MaxSpeedUp;
		const float MaxVerticalWaterSpeedDown = SurfaceSwimmingWaterControlSettings.MaxSpeedDown;
	
		Velocity.Z = FMath::Clamp(Velocity.Z, -FMath::Abs(MaxVerticalWaterSpeedDown), FMath::Abs(MaxVerticalWaterSpeedUp));
	}

	// Calculate and apply the requested move here
	{
		// Force from Water flow's WaterVelocity 
		const float MaxWaterForce = SurfaceSwimmingWaterControlSettings.MaxWaterForce;
		const float WaterForceMultiplier = SurfaceSwimmingWaterControlSettings.WaterForceMultiplier * SurfaceSwimmingWaterControlSettings.WaterForceSecondMultiplier;
		const FVector WaterVelocity = WaterResult.WaterSplineData.WaterVelocity;
		const FVector WaterAcceleration = (WaterVelocity * WaterForceMultiplier).GetClampedToMaxSize(MaxWaterForce);
		const float WaterSpeed = WaterVelocity.Size();
	
	    // Consider player input
		FRotator IntendedOrientation_WorldSpace;
		if (!CharacterInputs || CharacterInputs->OrientationIntent.IsNearlyZero())
		{
			IntendedOrientation_WorldSpace = StartingSyncState->GetOrientation_WorldSpace();
		}
		else
		{
			IntendedOrientation_WorldSpace = CharacterInputs->GetOrientationIntentDir_WorldSpace().ToOrientationRotator();
		}

		const FQuat WorldToGravityTransform = FQuat::FindBetweenNormals(FVector::UpVector, DefaultSimInputs->UpDir);
		IntendedOrientation_WorldSpace = UMovementUtils::ApplyGravityToOrientationIntent(IntendedOrientation_WorldSpace, WorldToGravityTransform, bShouldCharacterRemainUpright);
		
		FWaterMoveParams Params;
		if (CharacterInputs)
		{
			Params.MoveInputType = CharacterInputs->GetMoveInputType();

			const bool bMaintainInputMagnitude = true;
			Params.MoveInput = CharacterInputs->GetMoveInput_WorldSpace();
			// Params.MoveInput = UPlanarConstraintUtils::ConstrainDirectionToPlane(MoverComp->GetPlanarConstraint(), CharacterInputs->GetMoveInput_WorldSpace(), bMaintainInputMagnitude);
		}
		else
		{
			Params.MoveInputType = EMoveInputType::None;
			Params.MoveInput = FVector::ZeroVector;
		}
		
		Params.OrientationIntent = IntendedOrientation_WorldSpace;
		Params.PriorVelocity = StartingSyncState->GetVelocity_WorldSpace();
		Params.PriorOrientation = StartingSyncState->GetOrientation_WorldSpace();
		Params.TurningRate = SharedSettings->TurningRate;
		Params.TurningBoost = SharedSettings->TurningBoost;
		Params.MaxSpeed = GetMaxSpeed();
		Params.Acceleration = GetAcceleration();
		Params.Deceleration = SharedSettings->Deceleration;
		Params.DeltaSeconds = DeltaSeconds;
		Params.MoveSpeed = WaterSpeed;
		Params.MoveAcceleration = WaterAcceleration;
		Params.WorldToGravityQuat = WorldToGravityTransform;

		// Calculate the move
		OutProposedMove = UWaterMovementUtils::ComputeControlledWaterMove(Params);
		
		// Use Z Velocity calculated earlier (Buoyancy, Friction and Terminal Velocity) for the move's Z component
		OutProposedMove.LinearVelocity.Z = Velocity.Z;
	}
}

void UChaosSwimmingMode::SimulationTick_Implementation(const FSimulationTickParams& Params, FMoverTickEndData& OutputState)
{
	QUICK_SCOPE_CYCLE_COUNTER(ChaosSwimmingMode_SimulationTick);

	if (!Simulation)
	{
		UE_LOGF(LogChaosMover, Warning, "No Simulation set on UChaosSwimmingMode. Check you have a Chaos Backend");
		return;
	}

	const FChaosMoverSimulationDefaultInputs* DefaultSimInputs = Simulation->GetLocalSimInput().FindDataByType<FChaosMoverSimulationDefaultInputs>();
	if (!ensureMsgf(DefaultSimInputs != nullptr, TEXT("No FChaosMoverSimulationDefaultInputs in UChaosSwimmingMode::SimulationTick_Implementation, skipping")))
	{
		return;
	}
	const FMoverTickStartData& StartState = Params.StartState;
	const FCharacterDefaultInputs* CharacterInputs = StartState.InputCmd.InputCollection.FindDataByType<FCharacterDefaultInputs>();
	if (!ensureMsgf(CharacterInputs, TEXT("ChaosSwimmingMode requires FCharacterDefaultInputs")))
	{
		return;
	}
	const FMoverDefaultSyncState* StartingSyncState = StartState.SyncState.SyncStateCollection.FindDataByType<FMoverDefaultSyncState>();
	if (!ensureMsgf(StartingSyncState, TEXT("ChaosSwimmingMode requires FMoverDefaultSyncState")))
	{
		return;
	}

	FProposedMove ProposedMove = Params.ProposedMove;
	const FVector UpDir = DefaultSimInputs->UpDir;

	FMoverDefaultSyncState& OutputSyncState = OutputState.SyncState.SyncStateCollection.FindOrAddMutableDataByType<FMoverDefaultSyncState>();
	OutputSyncState = *StartingSyncState;

	OutputSyncState.MoveDirectionIntent = ProposedMove.bHasDirIntent ? ProposedMove.DirectionIntent : FVector::ZeroVector;

	const float DeltaSeconds = Params.TimeStep.StepMs * 0.001f;

	FRotator TargetOrient = StartingSyncState->GetOrientation_WorldSpace();
	TargetOrient = UMovementUtils::ApplyAngularVelocityToRotator(TargetOrient, ProposedMove.AngularVelocityDegrees, DeltaSeconds);

	FVector TargetVel = ProposedMove.LinearVelocity;
	TargetVel -= DefaultSimInputs->PhysicsObjectGravity * DefaultSimInputs->UpDir * DeltaSeconds;;

	FVector TargetPos = StartingSyncState->GetLocation_WorldSpace();
	TargetPos += TargetVel * DeltaSeconds;

	OutputSyncState.SetTransforms_WorldSpace(
		TargetPos,
		TargetOrient,
		TargetVel,
		ProposedMove.AngularVelocityDegrees);

	OutputState.MovementEndState.RemainingMs = 0.0f;
	OutputState.MovementEndState.NextModeName = Params.StartState.SyncState.MovementMode;
}