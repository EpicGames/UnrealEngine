// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosMover/Character/Modes/ChaosWalkingMode.h"

#include "Chaos/Character/CharacterGroundConstraintSettings.h"
#include "Chaos/ContactModification.h"
#include "Chaos/PhysicsObjectInternalInterface.h"
#include "ChaosMover/ChaosMoverLog.h"
#include "ChaosMover/ChaosMoverSimulation.h"
#include "ChaosMover/ChaosMoverSimulationTypes.h"
#include "ChaosMover/Character/ChaosCharacterInputs.h"
#include "ChaosMover/Character/Settings/SharedChaosCharacterMovementSettings.h"
#include "ChaosMover/Character/Transitions/ChaosCharacterFallingCheck.h"
#include "ChaosMover/Character/Transitions/ChaosCharacterJumpCheck.h"
#include "ChaosMover/Utilities/ChaosGroundMovementUtils.h"
#include "ChaosMover/Utilities/ChaosMoverQueryUtils.h"
#include "Math/UnitConversion.h"
#include "MoveLibrary/MovementUtils.h"
#include "MoveLibrary/BasedMovementUtils.h"
#include "MoveLibrary/FloorQueryUtils.h"
#include "MoveLibrary/GroundMovementUtils.h"
#include "MoveLibrary/MovementUtils.h"
#include "MoveLibrary/WaterMovementUtils.h"
#include "Chaos/ParticleHandle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosWalkingMode)

bool bVelocityPaddingWalkingModeRollbackUseNewCode = true;
FAutoConsoleVariableRef CVarVelocityPaddingWalkingModeRollbackUseNewCode(TEXT("ChaosMover.WalkingMode.VelocityPaddingUseNewCode"),
	bVelocityPaddingWalkingModeRollbackUseNewCode, TEXT("If true use the new code path for calculating the VelocityPadding in ChaosWalkingMode\n"));

UChaosWalkingMode::UChaosWalkingMode(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSupportsAsync = true;
	GameplayTags.AddTag(Mover_IsOnGround);

	RadialForceLimit = 2000.0f;
	SwingTorqueLimit = 3000.0f;
	TwistTorqueLimit = 1500.0f;

	Transitions.Add(CreateDefaultSubobject<UChaosCharacterFallingCheck>(TEXT("DefaultFallingCheck")));
}

void UChaosWalkingMode::UpdateConstraintSettings(Chaos::FCharacterGroundConstraintSettings& ConstraintSettings) const
{
	Super::UpdateConstraintSettings(ConstraintSettings);
	ConstraintSettings.FrictionForceLimit = FUnitConversion::Convert(FrictionForceLimit, EUnit::Newtons, EUnit::KilogramCentimetersPerSecondSquared);
	ConstraintSettings.DampingFactor = GroundDamping;
	ConstraintSettings.MotionTargetMassBias = FractionalGroundReaction;
	ConstraintSettings.RadialForceMotionTargetScaling = FractionalRadialForceLimitScaling;
}

void UChaosWalkingMode::GenerateMove_Implementation(const FMoverSimContext& SimContext, const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, FProposedMove& OutProposedMove) const
{
	QUICK_SCOPE_CYCLE_COUNTER(ChaosWalkingMode_GenerateMove);

	if (!Simulation)
	{
		UE_LOGF(LogChaosMover, Warning, "No Simulation set on ChaosWalkingMode. Check you have a Chaos Backend");
		return;
	}

	const FChaosMoverSimulationDefaultInputs* DefaultSimInputs = Simulation->GetLocalSimInput().FindDataByType<FChaosMoverSimulationDefaultInputs>();
	const FCharacterDefaultInputs* CharacterInputs = StartState.InputCmd.InputCollection.FindDataByType<FCharacterDefaultInputs>();
	if (!DefaultSimInputs || !CharacterInputs)
	{
		UE_LOGF(LogChaosMover, Warning, "ChaosWalkingMode requires FChaosMoverSimulationDefaultInputs and FCharacterDefaultInputs");
		return;
	}

	TStrongObjectPtr<const USharedChaosCharacterMovementSettings> SharedSettingsStrongPtr = SharedSettingsWeakPtr.Pin();
	if (!SharedSettingsStrongPtr)
	{
		UE_LOGF(LogChaosMover, Warning, "ChaosWalkingMode unable to read from shared settings (USharedChaosCharacterMovementSettings)");
		return;
	}
	const USharedChaosCharacterMovementSettings* SharedSettings = SharedSettingsStrongPtr.Get();
	check(SharedSettings);

	const FMoverDefaultSyncState* StartingSyncState = StartState.SyncState.SyncStateCollection.FindDataByType<FMoverDefaultSyncState>();
	check(StartingSyncState);

	const float DeltaSeconds = TimeStep.StepMs * 0.001f;
	const FVector UpDirection = DefaultSimInputs->UpDir;

	// Try to use the floor as the basis for the intended move direction (i.e. try to walk along slopes, rather than into them)
	UMoverBlackboard* SimBlackboard = Simulation->GetBlackboard_Mutable();
	FFloorCheckResult LastFloorResult;
	UE::ChaosMover::FGroundDynamicsInfo GroundDynamicsInfo;
	FVector MovementNormal = UpDirection;
	bool bWalkableFloor = false;

	if (SimBlackboard)
	{
		SimBlackboard->TryGet(CommonBlackboard::LastFloorResult, LastFloorResult);
		SimBlackboard->TryGet(UE::ChaosMover::Blackboard::GroundDynamicsInfo, GroundDynamicsInfo);

		bWalkableFloor = LastFloorResult.IsWalkableFloor();

		if (!bMaintainHorizontalGroundVelocity && SimBlackboard->TryGet(CommonBlackboard::LastFloorResult, LastFloorResult) && bWalkableFloor)
		{
			MovementNormal = LastFloorResult.HitResult.ImpactNormal;
		}
	}

	FRotator IntendedOrientation_WorldSpace;
	// If there's no intent from input to change orientation, use the current orientation
	if (!CharacterInputs || CharacterInputs->OrientationIntent.IsNearlyZero())
	{
		IntendedOrientation_WorldSpace = StartingSyncState->GetOrientation_WorldSpace();
	}
	else
	{
		IntendedOrientation_WorldSpace = CharacterInputs->GetOrientationIntentDir_WorldSpace().ToOrientationRotator();
	}

	const FQuat WorldToGravityTransform = FQuat::FindBetweenNormals(FVector::UpVector, DefaultSimInputs->UpDir);
	IntendedOrientation_WorldSpace = UMovementUtils::ApplyGravityToOrientationIntent(IntendedOrientation_WorldSpace, WorldToGravityTransform, bShouldCharacterRemainUpright);
	const FVector CurrentVelocity = StartingSyncState->GetVelocity_WorldSpace() - GroundDynamicsInfo.LinearVelocity;

	const FChaosMoverRequestedMoveInputs* RequestedMove = StartState.InputCmd.InputCollection.FindDataByType<FChaosMoverRequestedMoveInputs>();
	if (RequestedMove)
	{
		FRequestedMoveParams Params;

		Params.PriorVelocity = FVector::VectorPlaneProject(CurrentVelocity, MovementNormal);
		Params.PriorOrientation = StartingSyncState->GetOrientation_WorldSpace();
		Params.GroundNormal = MovementNormal;
		Params.TurningRate = SharedSettings->TurningRate;
		Params.MaxSpeed = GetMaxSpeed();
		Params.Acceleration = GetAcceleration();
		Params.Deceleration = SharedSettings->Deceleration;
		Params.DeltaSeconds = DeltaSeconds;
		Params.WorldToGravityQuat = WorldToGravityTransform;

		Params.bRequestedMoveWithMaxSpeed = RequestedMove->bForceMaxSpeed;
		Params.bShouldComputeAcceleration = RequestedMove->bUseAcceleration;
		Params.RequestedVelocity = RequestedMove->RequestedVelocity;

		OutProposedMove = UChaosGroundMovementUtils::ComputeRequestedMove(Params);
	}
	else
	{
		FGroundMoveParams Params;

		if (CharacterInputs)
		{
			Params.MoveInputType = CharacterInputs->GetMoveInputType();
			Params.MoveInput = CharacterInputs->GetMoveInput_WorldSpace();
		}
		else
		{
			Params.MoveInputType = EMoveInputType::None;
			Params.MoveInput = FVector::ZeroVector;
		}

		Params.OrientationIntent = IntendedOrientation_WorldSpace;
		Params.PriorVelocity = FVector::VectorPlaneProject(CurrentVelocity, MovementNormal);
		Params.PriorOrientation = StartingSyncState->GetOrientation_WorldSpace();
		Params.GroundNormal = MovementNormal;
		Params.TurningRate = SharedSettings->TurningRate;
		Params.TurningBoost = SharedSettings->TurningBoost;
		Params.MaxSpeed = GetMaxSpeed();
		Params.Acceleration = GetAcceleration();
		Params.Deceleration = SharedSettings->Deceleration;
		Params.DeltaSeconds = DeltaSeconds;
		Params.WorldToGravityQuat = WorldToGravityTransform;
		Params.UpDirection = UpDirection;
		Params.bUseAccelerationForVelocityMove = SharedSettings->bUseAccelerationForVelocityMove;

		if (Params.MoveInput.SizeSquared() > 0.f && !UMovementUtils::IsExceedingMaxSpeed(Params.PriorVelocity, GetMaxSpeed()))
		{
			Params.Friction = SharedSettings->GroundFriction;
		}
		else
		{
			Params.Friction = SharedSettings->bUseSeparateBrakingFriction ? SharedSettings->BrakingFriction : SharedSettings->GroundFriction;
			Params.Friction *= SharedSettings->BrakingFrictionFactor;
		}

		OutProposedMove = UGroundMovementUtils::ComputeControlledGroundMove(Params);
	}

	if (bMaintainHorizontalGroundVelocity && bWalkableFloor)
	{
		// So far have assumed we are on level ground, so now add velocity up the slope
		const double CosAngleBetweenUpAndNormal = UpDirection.Dot(LastFloorResult.HitResult.ImpactNormal);
		if (CosAngleBetweenUpAndNormal > UE_SMALL_NUMBER)
		{
			OutProposedMove.LinearVelocity -= UpDirection * OutProposedMove.LinearVelocity.Dot(LastFloorResult.HitResult.ImpactNormal) / CosAngleBetweenUpAndNormal;
		}
	}

	// Special case for standing on a rotating platform. Since the relative ground velocity is not quite right
	// in this case just set the propose move to zero if there is no input and the proposed move is small enough
	if ((GroundDynamicsInfo.AngularVelocityDegrees.Dot(DefaultSimInputs->UpDir) > 1.0f) && CharacterInputs->GetMoveInput_WorldSpace().IsNearlyZero())
	{
		const float ErrorLimit = GetMaxSpeed() * 0.1f;
		if (OutProposedMove.LinearVelocity.SizeSquared() < ErrorLimit * ErrorLimit)
		{
			OutProposedMove.LinearVelocity = FVector::ZeroVector;
		}
	}

	if (SimBlackboard)
	{
		// Update the floor result and check the proposed move to prevent movement onto unwalkable surfaces
		FVector OutDeltaPos = FVector::ZeroVector;
		FFloorCheckResult FloorResult;
		FWaterCheckResult WaterResult;
		GetFloorAndCheckMovement(*StartingSyncState, OutProposedMove, *DefaultSimInputs, DeltaSeconds, FloorResult, WaterResult, OutDeltaPos);

		OutProposedMove.LinearVelocity = (DeltaSeconds > 0.0f) ? (OutDeltaPos / DeltaSeconds) : FVector::ZeroVector;

		GroundDynamicsInfo = UE::ChaosMover::FGroundDynamicsInfo(FloorResult);
		SimBlackboard->Set(CommonBlackboard::LastFloorResult, FloorResult);
		SimBlackboard->Set(UE::ChaosMover::Blackboard::GroundDynamicsInfo, GroundDynamicsInfo);
		SimBlackboard->Set(CommonBlackboard::LastWaterResult, WaterResult);
	};
}

void UChaosWalkingMode::SimulationTick_Implementation(const FSimulationTickParams& Params, FMoverTickEndData& OutputState)
{
	QUICK_SCOPE_CYCLE_COUNTER(ChaosWalkingMode_SimulationTick);

	if (!Simulation)
	{
		UE_LOGF(LogChaosMover, Warning, "No Simulation set on ChaosWalkingMode. Check you have a Chaos Backend");
		return;
	}

	const FChaosMoverSimulationDefaultInputs* DefaultSimInputs = Simulation->GetLocalSimInput().FindDataByType<FChaosMoverSimulationDefaultInputs>();
	const FCharacterDefaultInputs* CharacterInputs = Params.StartState.InputCmd.InputCollection.FindDataByType<FCharacterDefaultInputs>();
	if (!DefaultSimInputs || !CharacterInputs)
	{
		UE_LOGF(LogChaosMover, Warning, "ChaosWalkingMode requires FChaosMoverSimulationDefaultInputs and FCharacterDefaultInputs");
		return;
	}

	TStrongObjectPtr<const USharedChaosCharacterMovementSettings> SharedSettingsStrongPtr = SharedSettingsWeakPtr.Pin();
	if (!SharedSettingsStrongPtr)
	{
		UE_LOGF(LogChaosMover, Warning, "UChaosWalkingMode unable to read from shared settings (USharedChaosCharacterMovementSettings)");
		return;
	}
	const USharedChaosCharacterMovementSettings* SharedSettings = SharedSettingsStrongPtr.Get();
	check(SharedSettings);

	FProposedMove ProposedMove = Params.ProposedMove;
	const FMoverDefaultSyncState* StartingSyncState = Params.StartState.SyncState.SyncStateCollection.FindDataByType<FMoverDefaultSyncState>();
	check(StartingSyncState);

	FMoverDefaultSyncState& OutputSyncState = OutputState.SyncState.SyncStateCollection.FindOrAddMutableDataByType<FMoverDefaultSyncState>();
	OutputSyncState = *StartingSyncState;

	// Update the ground based on the proposed move
	const float DeltaSeconds = Params.TimeStep.StepMs * 0.001f;
	const FVector UpDirection = DefaultSimInputs->UpDir;
	FVector GroundNormal = UpDirection;

	FFloorCheckResult FloorResult;
	UE::ChaosMover::FGroundDynamicsInfo GroundDynamicsInfo;
	if (const UMoverBlackboard* SimBlackboard = Simulation->GetBlackboard())
	{
		SimBlackboard->TryGet(CommonBlackboard::LastFloorResult, FloorResult);
		SimBlackboard->TryGet(UE::ChaosMover::Blackboard::GroundDynamicsInfo, GroundDynamicsInfo);
	}

	if (FloorResult.IsWalkableFloor())
	{
		GroundNormal = FloorResult.HitResult.ImpactNormal;

		FVector TargetPosition = StartingSyncState->GetLocation_WorldSpace();

		// The base movement mode does not apply gravity in walking mode so apply here.
		// Also remove the gravity that will be applied by the physics simulation.
		// This is so that the gravity in this mode will be consistent with the gravity
		// set on the mover, not the default physics gravity
		// Physics volumes always apply gravity along world Z (FVector::UpVector), not along UpDir.
		const FVector PhysicsObjectGravityVelocity = DefaultSimInputs->PhysicsObjectGravity * FVector::UpVector * DeltaSeconds;

		const FVector ProjectedVelocity = StartingSyncState->GetVelocity_WorldSpace() + DefaultSimInputs->Gravity * DeltaSeconds;
		FVector TargetVelocity = ProjectedVelocity - PhysicsObjectGravityVelocity;

		// If we have movement intent, non-vertical current velocity, or a proposed horizontal velocity, use the proposed move plane velocity.
		// Otherwise just fall with gravity.
		constexpr float ParallelCosThreshold = 0.999f;
		const bool bNonVerticalVelocity = !FVector::Parallel(TargetVelocity.GetSafeNormal(), UpDirection, ParallelCosThreshold);
		const bool bHasProposedHorizontalVelocity = !FVector::VectorPlaneProject(ProposedMove.LinearVelocity, UpDirection).IsNearlyZero();
		const bool bUseProposedMove = bNonVerticalVelocity || ProposedMove.bHasDirIntent || bHasProposedHorizontalVelocity;
		bool bNormalVelocityIntent = false;

		if (bUseProposedMove)
		{
			const FVector ProposedMovePlaneVelocity = ProposedMove.LinearVelocity - ProposedMove.LinearVelocity.ProjectOnToNormal(GroundNormal);

			// If there is velocity intent in the normal direction then use the velocity from the proposed move. Otherwise
			// retain the previous vertical velocity
			FVector ProposedNormalVelocity = ProposedMove.LinearVelocity - ProposedMovePlaneVelocity;
			if (ProposedNormalVelocity.SizeSquared() > UE_KINDA_SMALL_NUMBER)
			{
				TargetVelocity += ProposedNormalVelocity - TargetVelocity.ProjectOnToNormal(GroundNormal);
				bNormalVelocityIntent = true;
			}

			TargetPosition += ProposedMovePlaneVelocity * DeltaSeconds;
		}

		FVector ProjectedGroundVelocity = GroundDynamicsInfo.LinearVelocity;
		if (GroundDynamicsInfo.bIsGravityEnabled)
		{
			ProjectedGroundVelocity += PhysicsObjectGravityVelocity;
		}

		const FVector ProjectedRelativeVelocity = TargetVelocity - ProjectedGroundVelocity;
		const float ProjectedRelativeNormalVelocity = FloorResult.HitResult.ImpactNormal.Dot(TargetVelocity - ProjectedGroundVelocity);
		const float ProjectedRelativeVerticalVelocity = GroundNormal.Dot(TargetVelocity - ProjectedGroundVelocity);
		const float VerticalVelocityLimit = bNormalVelocityIntent ? 2.0f / DeltaSeconds : FMath::Abs(GroundNormal.Dot(DefaultSimInputs->Gravity) * DeltaSeconds);

		bool bIsLiftingOffSurface = false;
		if ((ProjectedRelativeNormalVelocity > VerticalVelocityLimit) && GroundDynamicsInfo.bIsMoving && (ProjectedRelativeVerticalVelocity > VerticalVelocityLimit))
		{
			bIsLiftingOffSurface = true;
		}

		// Determine if the character is stepping up or stepping down.
		// If stepping up make sure that the step height is less than the max step height
		// and the new surface has CanCharacterStepUpOn set to true.
		// If stepping down make sure the step height is less than the max step height.
		const float InitialHeightAboveFloor = FloorResult.FloorDist - GetTargetHeight();
		const float EndHeightAboveFloor = InitialHeightAboveFloor + ProjectedRelativeVerticalVelocity * DeltaSeconds;
		const bool bIsSteppingDown = InitialHeightAboveFloor > UE_KINDA_SMALL_NUMBER;
		const bool bIsWithinReach = EndHeightAboveFloor <= SharedSettings->MaxStepHeight;
		const bool bIsSupported = bIsWithinReach && !bIsLiftingOffSurface;
		const bool bNeedsVerticalVelocityToTarget = bIsSupported && bIsSteppingDown && (EndHeightAboveFloor > 0.0f) && !bIsLiftingOffSurface;
		if (bNeedsVerticalVelocityToTarget)
		{
			TargetVelocity -= FractionalDownwardVelocityToTarget * (EndHeightAboveFloor / DeltaSeconds) * UpDirection;
		}

		// Put the target position on the floor at the target height
		if (!GroundDynamicsInfo.bIsMoving)
		{
			TargetPosition -= UpDirection * InitialHeightAboveFloor;
		}

		// Target orientation
		// This is always applied regardless of whether the character is supported
		const FRotator TargetOrientation = UMovementUtils::ApplyAngularVelocityToRotator(StartingSyncState->GetOrientation_WorldSpace(), ProposedMove.AngularVelocityDegrees, DeltaSeconds);
		FVector TargetAngularVelocity = StartingSyncState->GetAngularVelocityDegrees_WorldSpace();

		if (ShouldCharacterRemainUpright() && bShouldApplyAngularVelocityToTarget)
		{
			if (DeltaSeconds > UE_SMALL_NUMBER)
			{
				const FQuat InitialQuat = StartingSyncState->GetOrientation_WorldSpace().Quaternion();
				FQuat TgtQuat = TargetOrientation.Quaternion();
				TgtQuat.EnforceShortestArcWith(InitialQuat);
				const FQuat QuatRotation = TgtQuat * InitialQuat.Inverse();
				const FVector AngularDisplacement = QuatRotation.ToRotationVector();
				TargetAngularVelocity = FMath::RadiansToDegrees(AngularDisplacement / DeltaSeconds);
			}
		}

		OutputSyncState.MoveDirectionIntent = ProposedMove.bHasDirIntent ? ProposedMove.DirectionIntent : FVector::ZeroVector;
		OutputSyncState.SetTransforms_WorldSpace(
			TargetPosition,
			TargetOrientation,
			TargetVelocity,
			TargetAngularVelocity);
	}
	
	OutputState.MovementEndState.RemainingMs = 0.0f;
	OutputState.MovementEndState.NextModeName = Params.StartState.SyncState.MovementMode;
}

bool UChaosWalkingMode::CanStepUpOnHitSurface(const FFloorCheckResult& FloorResult, const FVector& CharacterVelocity) const
{
	const float StepHeight = GetTargetHeight() - FloorResult.FloorDist;

	TStrongObjectPtr<const USharedChaosCharacterMovementSettings> SharedSettingsStrongPtr = SharedSettingsWeakPtr.Pin();
	const USharedChaosCharacterMovementSettings* SharedSettings = SharedSettingsStrongPtr ? SharedSettingsStrongPtr.Get() : GetDefault<USharedChaosCharacterMovementSettings>();
	check(SharedSettings);

	bool bWalkable = StepHeight <= SharedSettings->MaxStepHeight;
	constexpr float MinStepHeight = 2.0f;
	const bool SteppingUp = StepHeight > MinStepHeight;
	if (bWalkable && SteppingUp)
	{
		bWalkable = UGroundMovementUtils::CanStepUpOnHitSurface(FloorResult.HitResult);

		// Make sure the surface isn't moving away relative to the character movement when stepping up
		if (const Chaos::FPBDRigidParticleHandle* GroundParticle = UChaosGroundMovementUtils::GetRigidParticleHandleFromFloorResult_Internal(FloorResult))
		{
			const FVector Pos = GroundParticle->IsKinematic() ? FVector(GroundParticle->GetX()) : GroundParticle->GetTransformXRCom().GetLocation();
            const FVector R = FloorResult.HitResult.ImpactPoint - Pos;
            const FVector GroundV = GroundParticle->GetV() + GroundParticle->GetW().Cross(R);
            const FVector Dir = CharacterVelocity.GetSafeNormal();
            float SurfaceV = Chaos::FVec3::VectorPlaneProject(CharacterVelocity - GroundV, FloorResult.HitResult.ImpactNormal).Dot(Dir);
            bWalkable = bWalkable && (SurfaceV > UE_KINDA_SMALL_NUMBER);
		}
	}

	return bWalkable;
}

void UChaosWalkingMode::GetFloorAndCheckMovement(const FMoverDefaultSyncState& SyncState, const FProposedMove& ProposedMove, const FChaosMoverSimulationDefaultInputs& DefaultSimInputs, float DeltaSeconds, FFloorCheckResult& OutFloorResult, FWaterCheckResult& OutWaterResult, FVector& OutDeltaPos) const
{
	FVector DeltaPos = ProposedMove.LinearVelocity * DeltaSeconds;
	OutDeltaPos = DeltaPos;

	float VelocityPadding = 0.0f;
	if (bVelocityPaddingWalkingModeRollbackUseNewCode)
	{
		const FVector VelocityWithGravity = SyncState.GetVelocity_WorldSpace() + UMovementUtils::ComputeVelocityFromGravity(DefaultSimInputs.Gravity, DeltaSeconds);
		VelocityPadding = FMath::Max(DefaultSimInputs.UpDir.Dot(-VelocityWithGravity) * DeltaSeconds, 0.0f);
	}
	else
	{
		VelocityPadding = FMath::Max(DefaultSimInputs.UpDir.Dot(SyncState.GetVelocity_WorldSpace()) * DeltaSeconds, 0.0f);
	}

	TStrongObjectPtr<const USharedChaosCharacterMovementSettings> SharedSettingsStrongPtr = SharedSettingsWeakPtr.Pin();
	const USharedChaosCharacterMovementSettings* SharedSettings = SharedSettingsStrongPtr ? SharedSettingsStrongPtr.Get() : GetDefault<USharedChaosCharacterMovementSettings>();
	check(SharedSettings);

	UE::ChaosMover::Utils::FFloorSweepParams SweepParams{
		.ResponseParams = DefaultSimInputs.CollisionResponseParams,
		.QueryParams = DefaultSimInputs.CollisionQueryParams,
		.Location = SyncState.GetLocation_WorldSpace(),
		.DeltaPos = DeltaPos,
		.UpDir = DefaultSimInputs.UpDir,
		.World = DefaultSimInputs.World,
		.QueryDistance = GetTargetHeight() + SharedSettings->MaxStepHeight + VelocityPadding,
		.QueryRadius = FMath::Min(GetGroundQueryRadius(), FMath::Max(DefaultSimInputs.PawnCollisionRadius - 5.0f, 0.0f)),
		.MaxWalkSlopeCosine = GetMaxWalkSlopeCosine(),
		.TargetHeight = GetTargetHeight(),
		.CollisionChannel = DefaultSimInputs.CollisionChannel
	};

	// First, try a sweep at the end position
	UE::ChaosMover::Utils::FloorSweep_Internal(SweepParams, OutFloorResult, OutWaterResult);

	if (!OutFloorResult.bBlockingHit)
	{
		// No result at the end position. Fall back on the current floor result
		return;
	}

	bool bWalkableFloor = OutFloorResult.bWalkableFloor && CanStepUpOnHitSurface(OutFloorResult, ProposedMove.LinearVelocity);
	if (bWalkableFloor)
	{
		// Walkable floor found
		return;
	}

	// Hit something but not walkable. Try a new query to find a walkable surface
	const float StepBlockedHeight = GetTargetHeight() - DefaultSimInputs.PawnCollisionHalfHeight + DefaultSimInputs.PawnCollisionRadius;
	const float StepHeight = GetTargetHeight() - OutFloorResult.FloorDist;

	bool bIsDynamicSurface = false;
	if (const Chaos::FPBDRigidParticleHandle* GroundParticle = UChaosGroundMovementUtils::GetRigidParticleHandleFromFloorResult_Internal(OutFloorResult))
	{
		bIsDynamicSurface = GroundParticle->IsDynamic();
	}

	if ((StepHeight > StepBlockedHeight) || bIsDynamicSurface)
	{
		// Collision should prevent movement. Just try to find ground at start of movement
		SweepParams.QueryRadius = 0.25f * GetGroundQueryRadius();
		SweepParams.DeltaPos = FVector::ZeroVector;

		UE::ChaosMover::Utils::FloorSweep_Internal(SweepParams, OutFloorResult, OutWaterResult);
		OutFloorResult.bWalkableFloor = OutFloorResult.bWalkableFloor && CanStepUpOnHitSurface(OutFloorResult, ProposedMove.LinearVelocity);
		return;
	}

	if (DeltaPos.SizeSquared() < UE_SMALL_NUMBER)
	{
		// Stationary
		OutDeltaPos = FVector::ZeroVector;
		return;
	}

	// Try to limit the movement to remain on a walkable surface
	FVector HorizSurfaceDir = FVector::VectorPlaneProject(OutFloorResult.HitResult.ImpactNormal, DefaultSimInputs.UpDir);
	float HorizSurfaceDirSizeSq = HorizSurfaceDir.SizeSquared();
	bool bFoundOutwardDir = false;
	if (HorizSurfaceDirSizeSq > UE_SMALL_NUMBER)
	{
		HorizSurfaceDir *= FMath::InvSqrt(HorizSurfaceDirSizeSq);
		bFoundOutwardDir = true;
	}
	else
	{
		// Flat unwalkable surface. Try and get the horizontal direction from the normal instead
		HorizSurfaceDir = FVector::VectorPlaneProject(OutFloorResult.HitResult.Normal, DefaultSimInputs.UpDir);
		HorizSurfaceDirSizeSq = HorizSurfaceDir.SizeSquared();

		if (HorizSurfaceDirSizeSq > UE_SMALL_NUMBER)
		{
			HorizSurfaceDir *= FMath::InvSqrt(HorizSurfaceDirSizeSq);
			bFoundOutwardDir = true;
		}
	}

	if (bFoundOutwardDir)
	{
		const float DP = DeltaPos.Dot(HorizSurfaceDir);
		FVector NewDeltaPos = DeltaPos;
		if (DP > 0.0f)
		{
			// If we're moving away try a ray query at the end of the motion
			SweepParams.QueryRadius = 0.0f;
		}
		else
		{
			// Otherwise, try to find a walkable floor along the surface
			SweepParams.QueryRadius = 0.25f * GetGroundQueryRadius();
			NewDeltaPos = DeltaPos - DP * HorizSurfaceDir;
		}
		SweepParams.DeltaPos = NewDeltaPos;

		UE::ChaosMover::Utils::FloorSweep_Internal(SweepParams, OutFloorResult, OutWaterResult);
		OutFloorResult.bWalkableFloor = OutFloorResult.bWalkableFloor && CanStepUpOnHitSurface(OutFloorResult, ProposedMove.LinearVelocity);

		if (OutFloorResult.bWalkableFloor)
		{
			OutDeltaPos = NewDeltaPos;
		}
	}
	else
	{
		OutDeltaPos = FVector::ZeroVector;
	}
}

void UChaosWalkingMode::ModifyContacts(const FMoverTimeStep& TimeStep, const UE::ChaosMover::FSimulationInputData& InputData, const UE::ChaosMover::FSimulationOutputData& OutputData, Chaos::FCollisionContactModifier& Modifier) const
{
	Super::ModifyContacts(TimeStep, InputData, OutputData, Modifier);

	check(Simulation);

	// Get the updated (character) particle
	Chaos::FGeometryParticleHandle* UpdatedParticle = nullptr;
	const FChaosMoverSimulationDefaultInputs* SimInputs = Simulation->GetLocalSimInput().FindDataByType<FChaosMoverSimulationDefaultInputs>();
	if (SimInputs)
	{
		Chaos::FReadPhysicsObjectInterface_Internal ReadInterface = Chaos::FPhysicsObjectInternalInterface::GetRead();
		UpdatedParticle = ReadInterface.GetParticle(SimInputs->PhysicsObject);
	}

	if (!UpdatedParticle)
	{
		return;
	}

	TStrongObjectPtr<const USharedChaosCharacterMovementSettings> SharedSettingsStrongPtr = SharedSettingsWeakPtr.Pin();
	if (!SharedSettingsStrongPtr)
	{
		UE_LOGF(LogChaosMover, Warning, "UChaosWalkingMode unable to read from shared settings (USharedChaosCharacterMovementSettings)");
		return;
	}
	const USharedChaosCharacterMovementSettings* SharedSettings = SharedSettingsStrongPtr.Get();
	check(SharedSettings);

	// Try and find the ground particle if there is one in the latest floor result
	Chaos::FGeometryParticleHandle* GroundParticle = nullptr;
	FFloorCheckResult FloorResult;
	FloorResult.FloorDist = 1.0e10;
	if (const UMoverBlackboard* SimBlackboard = Simulation->GetBlackboard())
	{
		SimBlackboard->TryGet(CommonBlackboard::LastFloorResult, FloorResult);
		Chaos::FReadPhysicsObjectInterface_Internal Interface = Chaos::FPhysicsObjectInternalInterface::GetRead();
		GroundParticle = Interface.GetParticle(FloorResult.HitResult.PhysicsObject);
	}

	const float CharacterHeight = UpdatedParticle->GetX().Dot(SimInputs->UpDir);
	const float EndCapHeight = CharacterHeight - SimInputs->PawnCollisionHalfHeight + SimInputs->PawnCollisionRadius;
	constexpr float CosThetaMax = 0.707f;

	float MinContactHeightStepUps = CharacterHeight - 1.0e10f;
	const float StepDistance = FMath::Abs(GetTargetHeight() - FloorResult.FloorDist);
	if (StepDistance >= UE_KINDA_SMALL_NUMBER)
	{
		MinContactHeightStepUps = CharacterHeight - GetTargetHeight() + (1.0f + GroundDamping) * SharedSettings->MaxStepHeight;
	}

	for (Chaos::FContactPairModifier& PairModifier : Modifier.GetContacts(UpdatedParticle))
	{
		const int32 CharacterIdx = UpdatedParticle == PairModifier.GetParticlePair()[0] ? 0 : 1;
		Chaos::FGeometryParticleHandle* OtherParticle = PairModifier.GetOtherParticle(UpdatedParticle);
		if (!OtherParticle)
		{
			continue;
		}
		const bool bOtherParticleIsGround = OtherParticle == GroundParticle;

		for (int32 Idx = 0; Idx < PairModifier.GetNumContacts(); ++Idx)
		{
			Chaos::FVec3 Point0, Point1;
			PairModifier.GetWorldContactLocations(Idx, Point0, Point1);
			Chaos::FVec3 CharacterPoint = CharacterIdx == 0 ? Point0 : Point1;

			Chaos::FVec3 ContactNormal = PairModifier.GetWorldNormal(Idx);
			if ((ContactNormal.Z > CosThetaMax) && CharacterPoint.Z < EndCapHeight)
			{
				// Disable any nearly vertical contact with the end cap of the capsule
				// This will be handled by the character ground constraint
				PairModifier.SetContactPointDisabled(Idx);
			}
			else if (bOtherParticleIsGround && CharacterPoint.Z < MinContactHeightStepUps)
			{
				// In the case of steps ups disable all contacts below the max step height
				PairModifier.SetContactPointDisabled(Idx);
			}
		}
	}
}

