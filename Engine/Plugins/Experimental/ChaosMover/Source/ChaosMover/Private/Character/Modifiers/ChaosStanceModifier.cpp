// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosMover/Character/Modifiers/ChaosStanceModifier.h"

#include "Chaos/Capsule.h"
#include "Chaos/PhysicsObjectInternalInterface.h"
#include "ChaosMover/ChaosMoverLog.h"
#include "ChaosMover/ChaosMoverSimulation.h"
#include "DefaultMovementSet/CharacterMoverComponent.h"
#include "Components/CapsuleComponent.h"
#include "MoverTypes.h"
#include "PBDRigidsSolver.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "PhysicsEngine/PhysicsObjectExternalInterface.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosStanceModifier)

FChaosStanceModifier::FChaosStanceModifier()
	: ActiveStance(EStanceMode::Invalid)
{
	DurationMs = -1.0f;
}

bool FChaosStanceModifier::HasGameplayTag(FGameplayTag TagToFind, bool bExactMatch) const
{
	// TODO: Eventually check for other stance tags here like prone
	if (bExactMatch)
	{
		return TagToFind.MatchesTagExact(Mover_IsCrouching);
	}

	return TagToFind.MatchesTag(Mover_IsCrouching);
}

void FChaosStanceModifier::GetGameplayTags(FGameplayTagContainer& InOutTags) const
{
	InOutTags.AddTag(Mover_IsCrouching);
}

const Chaos::FCapsule* FChaosStanceModifier::GetCapsule(Chaos::FPBDRigidParticleHandle* ParticleHandle) const
{
	const Chaos::FCapsule* FoundCapsule = nullptr;

	const Chaos::FImplicitObjectRef Geometry = ParticleHandle->GetGeometry();
	if (Geometry && Geometry->IsValidGeometry())
	{
		FoundCapsule = ParticleHandle->GetGeometry()->GetObject<Chaos::FCapsule>();
		if (!FoundCapsule)
		{
			if (const Chaos::FImplicitObjectUnion* ImplicitUnion = ParticleHandle->GetGeometry()->GetObject<Chaos::FImplicitObjectUnion>())
			{
				for (const Chaos::FImplicitObjectRef ObjectRef : ImplicitUnion->GetObjects())
				{
					FoundCapsule = ObjectRef->GetObject<Chaos::FCapsule>();
					if (FoundCapsule)
					{
						break;
					}
				}
			}
		}
	}

	return FoundCapsule;
}

void FChaosStanceModifier::UpdateStance(const FMovementModifierParams_Async& AsyncParams, EStanceMode NewStance)
{
	if (!AsyncParams.IsValid())
	{
		return;
	}

	UChaosMoverSimulation* Simulation = Cast<UChaosMoverSimulation>(AsyncParams.Simulation);
	if (!Simulation)
	{
		return;
	}

	FName ModeName = CurrentModeName.IsNone() ? AsyncParams.SyncState->MovementMode : CurrentModeName;
	IChaosCharacterMovementModeInterface* CharacterMode = Cast<IChaosCharacterMovementModeInterface>(Simulation->FindMovementModeByName_Mutable(ModeName));
	if (!CharacterMode)
	{
		UE_LOGF(LogChaosMover, Warning, "ChaosStanceModifier only works with character modes");
		return;
	}

	const FChaosMoverSimulationDefaultInputs* SimInputs = Simulation->GetLocalSimInput().FindDataByType<FChaosMoverSimulationDefaultInputs>();
	if (!SimInputs)
	{
		UE_LOGF(LogChaosMover, Warning, "ChaosStanceModifier requires FCharacterDefaultInputs");
		return;
	}

	FPhysScene* Scene = SimInputs->World ? SimInputs->World->GetPhysicsScene() : nullptr;
	Chaos::FPhysicsSolver* Solver = Scene ? Scene->GetSolver() : nullptr;
	if (!Solver)
	{
		UE_LOGF(LogChaosMover, Warning, "ChaosStanceModifier requires FPhysicsSolver");
		return;
	}

	if (NewStance == ActiveStance)
	{
		// NewStance == ActiveStance can happen on the first resim frame after a rollback.
		// This happens if we rollback to a frame where the modifier was active and already
		// active on previous frames. In that case, ActiveStance was deserialized from the restored
		// sync state and already reflects the rollback frame's stance before OnStart_Async runs. 
		// We must still re-apply the capsule for physics correctness, so we fall through on that frame only.
		// On any other frame (non-resim, or subsequent resim frames) this is a genuine no-op; skip it.
		if (!AsyncParams.TimeStep->bIsResimulating || !AsyncParams.TimeStep->bIsFirstResimFrame)
		{
			UE_LOGF(LogChaosMover, Verbose, "Skipping stance change - not required");
			return;
		}
	}

	Chaos::FWritePhysicsObjectInterface_Internal WriteInterface = Chaos::FPhysicsObjectInternalInterface::GetWrite();
	Chaos::FPBDRigidParticleHandle* ParticleHandle = WriteInterface.GetRigidParticle(SimInputs->PhysicsObject);
	if (!ParticleHandle)
	{
		return;
	}

	const Chaos::FCapsule* Capsule = GetCapsule(ParticleHandle);
	if (!Capsule)
	{
		UE_LOGF(LogChaosMover, Warning, "No capsule shape found in ChaosStanceModifier");
		return;
	}

	// Invalidate the particle as we are about to change the collision geometry so need to clear contacts etc.
	Solver->GetEvolution()->InvalidateParticle(ParticleHandle);

	// Scale the capsule
	float NewHalfHeight, NewRadius, NewGroundClearance;
	if (NewStance == EStanceMode::Crouch)
	{
		NewHalfHeight = ModifiedCapsuleHalfHeight;
		NewRadius = ModifiedCapsuleRadius;
		NewGroundClearance = ModifiedCapsuleGroundClearance;

		if (MaxSpeedOverride.IsSet())
		{
			CharacterMode->OverrideMaxSpeed(MaxSpeedOverride.GetValue());
		}

		if (AccelerationOverride.IsSet())
		{
			CharacterMode->OverrideAcceleration(AccelerationOverride.GetValue());
		}
	}
	else
	{
		NewHalfHeight = DefaultCapsuleHalfHeight;
		NewRadius = DefaultCapsuleRadius;
		NewGroundClearance = DefaultCapsuleGroundClearance;

		CharacterMode->ClearMaxSpeedOverride();
		CharacterMode->ClearAccelerationOverride();
	}
	
	UpdateCapsule(ParticleHandle, NewRadius, NewHalfHeight, NewGroundClearance, CharacterMode->GetTargetHeight());

	EStanceMode OldStance = ActiveStance;
	ActiveStance = NewStance;

	// We're setting the capsule on PT but there is no automatic mechanism to
	// reflect that change on GT, so add a callback to do that
	Chaos::FImplicitObjectPtr NewCapsule = ParticleHandle->GetGeometry();
	Chaos::FPhysicsObjectHandle PhysicsObject = SimInputs->PhysicsObject;

	auto SetGeometryGTCallback = [NewCapsule, PhysicsObject](const FMoverSimulationEventData & Data, const FMoverSimEventGameThreadContext& GTContext) {
		Chaos::FPBDRigidParticle* Particle = FPhysicsObjectExternalInterface::LockRead(PhysicsObject)->GetRigidParticle(PhysicsObject);
		if (Particle && NewCapsule.IsValid())
		{
			Particle->SetGeometry(NewCapsule);
		}
	};

	// Fire simulation event
	Simulation->AddEvent(MakeShared<FStanceModifiedEventData>(*AsyncParams.TimeStep, OldStance, ActiveStance, SetGeometryGTCallback));
}

void FChaosStanceModifier::OnStart_Async(const FMovementModifierParams_Async& AsyncParams)
{
	UpdateStance(AsyncParams, EStanceMode::Crouch);
	
	// Set the current mode - this is used to check if the mode changes
	CurrentModeName = AsyncParams.SyncState->MovementMode;
}

void FChaosStanceModifier::OnEnd_Async(const FMovementModifierParams_Async& AsyncParams)
{
	UpdateStance(AsyncParams, EStanceMode::Invalid);
	
	CurrentModeName = NAME_None;
}

void FChaosStanceModifier::UpdateCapsule(Chaos::FPBDRigidParticleHandle* ParticleHandle, float NewRadius, float NewHalfHeight, float NewGroundClearance, float TargetHeight)
{
	check(ParticleHandle);

	const float NewHeight = FMath::Max(2.0f * (NewHalfHeight - NewRadius), 0.0f);
	const Chaos::FVec3 Axis = Chaos::FVec3::UpVector;
	const Chaos::FVec3 NewX1 = (NewGroundClearance + NewRadius - TargetHeight) * Axis;
	const Chaos::FVec3 NewX2 = NewX1 + Axis * NewHeight;
	const Chaos::FCapsule NewCapsule(NewX1, NewX2, NewRadius);

	ParticleHandle->SetGeometry(Chaos::FImplicitObjectPtr(new Chaos::FCapsule(NewX1, NewX2, NewRadius)));
	ParticleHandle->SetCenterOfMass(NewCapsule.GetCenterf());
}

void FChaosStanceModifier::OnPostMovement_Async(const FMovementModifierParams_Async& AsyncParams)
{
	if (bCancelOnModeChange && CurrentModeName != AsyncParams.SyncState->MovementMode)
	{
		DurationMs = 0;
	}
}

FMovementModifierBase* FChaosStanceModifier::Clone() const
{
	return new FChaosStanceModifier(*this);
}

void FChaosStanceModifier::NetSerialize(FArchive& Ar)
{
	Super::NetSerialize(Ar);

	Ar << ActiveStance;
	Ar << bCancelOnModeChange;
	Ar << ModifiedCapsuleHalfHeight;
	Ar << ModifiedCapsuleRadius;
	Ar << ModifiedCapsuleGroundClearance;
	Ar << DefaultCapsuleHalfHeight;
	Ar << DefaultCapsuleRadius;
	Ar << DefaultCapsuleGroundClearance;
	Ar << AccelerationOverride;
	Ar << MaxSpeedOverride;
	Ar << CurrentModeName;
}

UScriptStruct* FChaosStanceModifier::GetScriptStruct() const
{
	return FChaosStanceModifier::StaticStruct();
}

FString FChaosStanceModifier::ToSimpleString() const
{
	return FString::Printf(TEXT("Chaos Stance Modifier"));
}

void FChaosStanceModifier::AddReferencedObjects(class FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(Collector);
}
