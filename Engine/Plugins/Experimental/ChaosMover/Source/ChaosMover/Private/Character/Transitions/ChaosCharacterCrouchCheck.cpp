// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosMover/Character/Transitions/ChaosCharacterCrouchCheck.h"

#include "Chaos/Capsule.h"
#include "Chaos/PhysicsObjectInternalInterface.h"
#include "ChaosMover/ChaosMoverLog.h"
#include "ChaosMover/ChaosMoverSimulation.h"
#include "ChaosMover/ChaosMoverSimulationTypes.h"
#include "ChaosMover/Character/ChaosCharacterInputs.h"
#include "ChaosMover/Character/Modes/ChaosCharacterMovementMode.h"
#include "ChaosMover/Character/Modifiers/ChaosStanceModifier.h"
#include "ChaosMover/Utilities/ChaosMoverQueryUtils.h"
#include "Components/CapsuleComponent.h"
#include "MoverComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosCharacterCrouchCheck)

UChaosCharacterCrouchCheck::UChaosCharacterCrouchCheck(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSupportsAsync = true;
	bAllowModeReentry = true;
	bFirstSubStepOnly = true;
}

void UChaosCharacterCrouchCheck::OnRegistered()
{
	Super::OnRegistered();

	if (const UPrimitiveComponent* PrimComp = Cast<UPrimitiveComponent>(GetMoverComponent()->GetUpdatedComponent()))
	{
		PrimComp->CalcBoundingCylinder(OriginalCapsuleRadius, OriginalCapsuleHalfHeight);
	}
}

void UChaosCharacterCrouchCheck::OnUnregistered()
{
	Super::OnUnregistered();
}

bool UChaosCharacterCrouchCheck::CanCrouch(const FSimulationTickParams& Params) const
{
	return true;
}

bool UChaosCharacterCrouchCheck::CanUncrouch(const FSimulationTickParams& Params) const
{
	if (const FChaosMoverSimulationDefaultInputs* SimInputs = Simulation->GetLocalSimInput().FindDataByType<FChaosMoverSimulationDefaultInputs>())
	{
		if (const FMoverDefaultSyncState* StartingSyncState = Params.StartState.SyncState.SyncStateCollection.FindDataByType<FMoverDefaultSyncState>())
		{
			UE::ChaosMover::Utils::FCapsuleOverlapTestParams OverlapTestParams{
				.ResponseParams = SimInputs->CollisionResponseParams,
				.QueryParams = SimInputs->CollisionQueryParams,
				.Location = StartingSyncState->GetLocation_WorldSpace(),
				.UpDir = SimInputs->UpDir,
				.World = SimInputs->World,
				.CapsuleHalfHeight = OriginalCapsuleHalfHeight,
				.CapsuleRadius = OriginalCapsuleRadius,
				.CollisionChannel = SimInputs->CollisionChannel
			};

			return !UE::ChaosMover::Utils::CapsuleOverlapTest_Internal(OverlapTestParams);
		}
	}

	return false;
}

void UChaosCharacterCrouchCheck::Crouch(const FSimulationTickParams& TickParams)
{
	UE_LOGF(LogChaosMover, Verbose, "Crouch Activated");

	float OriginalCapsuleGroundClearance = 10.0f;
	if (const IChaosCharacterMovementModeInterface* CharacterMode = Cast<IChaosCharacterMovementModeInterface>(Simulation->GetCurrentMovementMode()))
	{
		OriginalCapsuleGroundClearance = CharacterMode->GetTargetHeight() - OriginalCapsuleHalfHeight;
	}

	TSharedPtr<FChaosStanceModifier> StanceModifier = MakeShared<FChaosStanceModifier>();
	StanceModifier->ModifiedCapsuleHalfHeight = CapsuleHalfHeight.IsSet() ? CapsuleHalfHeight.GetValue() : OriginalCapsuleHalfHeight;
	StanceModifier->ModifiedCapsuleRadius = CapsuleRadius.IsSet() ? CapsuleRadius.GetValue() : OriginalCapsuleRadius;
	StanceModifier->ModifiedCapsuleGroundClearance = CapsuleGroundClearance.IsSet() ? CapsuleGroundClearance.GetValue() : OriginalCapsuleGroundClearance;
	StanceModifier->DefaultCapsuleHalfHeight = OriginalCapsuleHalfHeight;
	StanceModifier->DefaultCapsuleRadius = OriginalCapsuleRadius;
	StanceModifier->DefaultCapsuleGroundClearance = OriginalCapsuleGroundClearance;
	StanceModifier->AccelerationOverride = Acceleration;
	StanceModifier->MaxSpeedOverride = MaxSpeed;
	StanceModifier->bCancelOnModeChange = bCancelOnModeChange;

	Simulation->QueueMovementModifier(MoveTemp(StanceModifier));
}

void UChaosCharacterCrouchCheck::Uncrouch(const FSimulationTickParams& TickParams)
{
	UE_LOGF(LogChaosMover, Verbose, "Uncrouch Activated");

	check(Simulation);

	if (const FChaosStanceModifier* Modifier = Simulation->FindMovementModifierByType<FChaosStanceModifier>())
	{
		Simulation->CancelModifierFromHandle(Modifier->GetHandle());
	}
}

FTransitionEvalResult UChaosCharacterCrouchCheck::Evaluate_Implementation(const FSimulationTickParams& TickParams) const
{
	FTransitionEvalResult EvalResult;

	if (!Simulation)
	{
		UE_LOGF(LogChaosMover, Warning, "No Simulation set on UChaosCharacterCrouchCheck. Check you have a Chaos Backend");
		return EvalResult;
	}

	if (bTriggerCrouch || bTriggerUncrouch)
	{
		UE_LOGF(LogChaosMover, Warning, "Trying to crouch/uncrouch in the same substep");
		return EvalResult;
	}

	const FChaosMoverCrouchInputs* CrouchInputs = TickParams.StartState.InputCmd.InputCollection.FindDataByType<FChaosMoverCrouchInputs>();
	if (CrouchInputs)
	{
		if (Simulation->HasMovementMode(TickParams.StartState.SyncState.MovementMode))
		{
			bool bIsCrouching = Simulation->HasGameplayTag(Mover_IsCrouching, true);

			if (bIsCrouching && !CrouchInputs->bWantsToCrouch && CanUncrouch(TickParams))
			{
				bTriggerUncrouch = true;
				if (TransitionToUncrouchedMode.IsSet())
				{
					EvalResult.NextMode = TransitionToUncrouchedMode.GetValue();
				}
				else
				{
					EvalResult.NextMode = TickParams.StartState.SyncState.MovementMode;
				}
			}
			else if (!bIsCrouching && CrouchInputs->bWantsToCrouch && CanCrouch(TickParams))
			{
				bTriggerCrouch = true;
				if (TransitionToCrouchedMode.IsSet())
				{
					EvalResult.NextMode = TransitionToCrouchedMode.GetValue();
				}
				else
				{
					EvalResult.NextMode = TickParams.StartState.SyncState.MovementMode;
				}
			}
		}
	}

	return EvalResult;
}

void UChaosCharacterCrouchCheck::Trigger_Implementation(const FSimulationTickParams& TickParams)
{
	if (!Simulation)
	{
		UE_LOGF(LogChaosMover, Warning, "No Simulation set on UChaosCharacterCrouchCheck. Check you have a Chaos Backend");
		return;
	}

	if (bTriggerCrouch)
	{
		Crouch(TickParams);
	}
	else if (bTriggerUncrouch)
	{
		Uncrouch(TickParams);
	}

	bTriggerUncrouch = bTriggerCrouch = false;
}