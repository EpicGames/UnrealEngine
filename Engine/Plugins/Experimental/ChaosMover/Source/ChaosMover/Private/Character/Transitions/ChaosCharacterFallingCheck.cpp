// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosMover/Character/Transitions/ChaosCharacterFallingCheck.h"

#include "Chaos/ParticleHandle.h"
#include "ChaosMover/ChaosMoverConsoleVariables.h"
#include "ChaosMover/ChaosMoverLog.h"
#include "ChaosMover/ChaosMoverSimulation.h"
#include "ChaosMover/ChaosMoverSimulationTypes.h"
#include "ChaosMover/Character/Settings/SharedChaosCharacterMovementSettings.h"
#include "ChaosMover/Utilities/ChaosGroundMovementUtils.h"
#include "MoveLibrary/FloorQueryUtils.h"
#include "MoverComponent.h"
#include "MoverSimulationTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosCharacterFallingCheck)

UChaosCharacterFallingCheck::UChaosCharacterFallingCheck(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSupportsAsync = true;

	TransitionToFallingMode = DefaultModeNames::Falling;
	SharedSettingsClasses.Add(USharedChaosCharacterMovementSettings::StaticClass());
}

void UChaosCharacterFallingCheck::OnRegistered()
{
	Super::OnRegistered();

	TObjectPtr<const USharedChaosCharacterMovementSettings> SharedSettingsPtr = GetMoverComponent()->FindSharedSettings<USharedChaosCharacterMovementSettings>();
	if (ensureMsgf(SharedSettingsPtr, TEXT("Failed to find instance of USharedChaosCharacterMovementSettings on %s. Movement may not function properly."), *GetPathNameSafe(this)))
	{
		SharedSettingsWeakPtr = TWeakObjectPtr<const USharedChaosCharacterMovementSettings>(SharedSettingsPtr);
	}
}

FTransitionEvalResult UChaosCharacterFallingCheck::Evaluate_Implementation(const FSimulationTickParams& Params) const
{
	FTransitionEvalResult EvalResult;

	if (TransitionToFallingMode.IsNone())
	{
		UE_LOGF(LogChaosMover, Warning, "No TransitionToFallingMode set on UChaosCharacterFallingCheck.");
		return EvalResult;
	}
	
	if (!Simulation)
	{
		UE_LOGF(LogChaosMover, Warning, "No Simulation set on UChaosCharacterFallingCheck. Check you have a Chaos Backend");
		return EvalResult;
	}

	const FMoverTickStartData& StartState = Params.StartState;

	if (const FChaosMoverSimulationDefaultInputs* SimInputs = Simulation->GetLocalSimInput().FindDataByType<FChaosMoverSimulationDefaultInputs>())
	{
		UMoverBlackboard* Blackboard = Simulation->GetBlackboard_Mutable();
		check(Blackboard);

		FFloorCheckResult FloorResult;
		UE::ChaosMover::FGroundDynamicsInfo GroundDynamicsInfo;
		if (!(Blackboard->TryGet(CommonBlackboard::LastFloorResult, FloorResult) && Blackboard->TryGet(UE::ChaosMover::Blackboard::GroundDynamicsInfo, GroundDynamicsInfo)))
		{
			return EvalResult;
		}

		const FMoverDefaultSyncState* StartingSyncState = StartState.SyncState.SyncStateCollection.FindDataByType<FMoverDefaultSyncState>();
		if (!StartingSyncState)
		{
			return EvalResult;
		}

		TStrongObjectPtr<const USharedChaosCharacterMovementSettings> SharedSettingsStrongPtr = SharedSettingsWeakPtr.Pin();
		if (!SharedSettingsStrongPtr)
		{
			UE_LOGF(LogChaosMover, Warning, "UChaosCharacterFallingCheck unable to read from shared settings (USharedChaosCharacterMovementSettings)");
			return EvalResult;
		}
		const USharedChaosCharacterMovementSettings* SharedSettings = SharedSettingsStrongPtr.Get();
		check(SharedSettings);

		if (TStrongObjectPtr<const UBaseMovementMode> BaseMode = Simulation->FindMovementModeByName(Params.StartState.SyncState.MovementMode).Pin())
		{
			if (const IChaosCharacterMovementModeInterface* Mode = Cast<IChaosCharacterMovementModeInterface>(BaseMode.Get()))
			{
				const float DeltaSeconds = Params.TimeStep.StepMs * 0.001f;
				const FVector ProjectedVelocity = StartingSyncState->GetVelocity_WorldSpace() + SimInputs->Gravity * DeltaSeconds;
				FVector ProjectedGroundVelocity = GroundDynamicsInfo.LinearVelocity;
				if (GroundDynamicsInfo.bIsDynamic && GroundDynamicsInfo.bIsGravityEnabled)
				{
					// This might not be correct if different physics objects have different gravity but is saves having to go
					// to the component to get the gravity on the physics volume.
					ProjectedGroundVelocity += SimInputs->PhysicsObjectGravity * SimInputs->UpDir * DeltaSeconds;
				}
				const bool bIsGroundMoving = FMath::Abs(ProjectedGroundVelocity.Dot(SimInputs->UpDir)) > UE_KINDA_SMALL_NUMBER;
				const FVector ProjectedRelativeVelocity = ProjectedVelocity - ProjectedGroundVelocity;
				const float ProjectedRelativeNormalVelocity = FloorResult.HitResult.ImpactNormal.Dot(ProjectedVelocity - ProjectedGroundVelocity);
				const float ProjectedRelativeVerticalVelocity = SimInputs->UpDir.Dot(ProjectedVelocity - ProjectedGroundVelocity);
				const float VerticalVelocityLimit = UE::ChaosMover::CVars::FallingCheckRelativeSpeedLimit;
				bool bIsLiftingOffSurface = false;
				if ((ProjectedRelativeNormalVelocity > VerticalVelocityLimit) && bIsGroundMoving && (ProjectedRelativeVerticalVelocity > VerticalVelocityLimit))
				{
					bIsLiftingOffSurface = true;
				}

				const bool bIsWithinReach = FloorResult.FloorDist - Mode->GetTargetHeight() <= SharedSettings->MaxStepHeight;
				bool bIsSupported = bIsWithinReach && !bIsLiftingOffSurface;
				float TimeSinceSupported = MaxUnsupportedTimeBeforeFalling;
				Blackboard->TryGet(CommonBlackboard::TimeSinceSupported, TimeSinceSupported);

				if (bIsSupported)
				{
					Blackboard->Set(CommonBlackboard::TimeSinceSupported, 0.0f);
				}
				else if (!bIsLiftingOffSurface)
				{
					// Falling
					TimeSinceSupported += DeltaSeconds;
					Blackboard->Set(CommonBlackboard::TimeSinceSupported, TimeSinceSupported);
					bIsSupported = TimeSinceSupported < MaxUnsupportedTimeBeforeFalling;
				}
				else
				{
					// Moving up relative to ground
					Blackboard->Set(CommonBlackboard::TimeSinceSupported, MaxUnsupportedTimeBeforeFalling);
				}

				if (!bIsSupported)
				{
					EvalResult.NextMode = TransitionToFallingMode;
				}
			}
		}
	}

	return EvalResult;
}

void UChaosCharacterFallingCheck::Trigger_Implementation(const FSimulationTickParams& Params)
{
}
