// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosMover/Character/Transitions/ChaosCharacterJumpCheck.h"

#include "Chaos/ParticleHandle.h"
#include "Chaos/PhysicsObjectInternalInterface.h"
#include "Chaos/Utilities.h"
#include "ChaosMover/ChaosMoverLog.h"
#include "ChaosMover/ChaosMoverSimulation.h"
#include "ChaosMover/ChaosMoverSimulationTypes.h"
#include "ChaosMover/Character/Effects/ChaosCharacterApplyVelocityEffect.h"
#include "ChaosMover/Utilities/ChaosGroundMovementUtils.h"
#include "DefaultMovementSet/CharacterMoverSimulationTypes.h"
#include "MoveLibrary/FloorQueryUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosCharacterJumpCheck)

namespace Private
{
	static void ApplyImpulse(Chaos::FPBDRigidParticleHandle* Particle, const Chaos::FVec3& Impulse, const Chaos::FVec3& Location)
	{
		Chaos::FRigidTransform3 ComTransform = Particle->GetTransformXRCom();
		const Chaos::FVec3 Offset = Location - ComTransform.GetLocation();
		const Chaos::FMatrix33 WorldInvI = Chaos::Utilities::ComputeWorldSpaceInertia(ComTransform.GetRotation(), Particle->ConditionedInvI());
		Particle->SetW(Particle->GetW() + WorldInvI * Offset.Cross(Impulse));
		Particle->SetV(Particle->GetV() + Particle->InvM() * Impulse);
	}
}

UChaosCharacterJumpCheck::UChaosCharacterJumpCheck(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSupportsAsync = true;
	bFirstSubStepOnly = true;

	TransitionToMode = DefaultModeNames::Falling;
}

FTransitionEvalResult UChaosCharacterJumpCheck::Evaluate_Implementation(const FSimulationTickParams& Params) const
{
	FTransitionEvalResult EvalResult;

	const FCharacterDefaultInputs* CharacterInputs = Params.StartState.InputCmd.InputCollection.FindDataByType<FCharacterDefaultInputs>();
	if (CharacterInputs && CharacterInputs->bIsJumpJustPressed)
	{
		EvalResult.NextMode = TransitionToMode;
	}

	return EvalResult;
}

void UChaosCharacterJumpCheck::Trigger_Implementation(const FSimulationTickParams& Params)
{
	if (!Simulation)
	{
		UE_LOGF(LogChaosMover, Warning, "No Simulation set on UChaosCharacterJumpCheck. Check you have a Chaos Backend");
		return;
	}

	if (const FChaosMoverSimulationDefaultInputs* SimInputs = Simulation->GetLocalSimInput().FindDataByType<FChaosMoverSimulationDefaultInputs>())
	{
		FVector JumpVelocity = JumpUpwardsSpeed * SimInputs->UpDir;

		// Substract any current upwards velocity so that the jump velocity is
		// always the same in the up direction
		const FMoverDefaultSyncState* StartingSyncState = Params.StartState.SyncState.SyncStateCollection.FindDataByType<FMoverDefaultSyncState>();
		if (StartingSyncState)
		{
			JumpVelocity -= StartingSyncState->GetVelocity_WorldSpace().Dot(SimInputs->UpDir) * SimInputs->UpDir;
		}

		// Add on the local ground velocity to make the jump velocity relative to the ground
		FFloorCheckResult FloorResult;
		UE::ChaosMover::FGroundDynamicsInfo GroundDynamicsInfo;
		const UMoverBlackboard* Blackboard = Simulation->GetBlackboard();
		if (Blackboard && Blackboard->TryGet(CommonBlackboard::LastFloorResult, FloorResult) && Blackboard->TryGet(UE::ChaosMover::Blackboard::GroundDynamicsInfo, GroundDynamicsInfo))
		{
			if (Chaos::FPhysicsObjectHandle GroundObject = FloorResult.HitResult.PhysicsObject)
			{
				if (StartingSyncState)
				{
					JumpVelocity += GroundDynamicsInfo.LinearVelocity.Dot(SimInputs->UpDir) * SimInputs->UpDir;
				}
			}
		}

		TSharedPtr<FChaosCharacterApplyVelocityEffect> JumpMove = MakeShared<FChaosCharacterApplyVelocityEffect>();
		JumpMove->VelocityOrImpulseToApply = JumpVelocity;
		JumpMove->Mode = EChaosMoverVelocityEffectMode::AdditiveVelocity;

		Simulation->QueueInstantMovementEffect_Internal(JumpMove);

		float StartingHeight = 0.0f;
		if (StartingSyncState)
		{
			StartingHeight = StartingSyncState->GetLocation_WorldSpace().Z;
		}
		Simulation->AddEvent(MakeShared<FJumpedEventData>(Params.TimeStep, StartingHeight));

		// Apply an equal and opposite impulse to the ground
		if (FractionalGroundReactionImpulse > 0.0f)
		{
			if (Chaos::FPhysicsObjectHandle GroundObject = FloorResult.HitResult.PhysicsObject)
			{
				Chaos::FWritePhysicsObjectInterface_Internal Interface = Chaos::FPhysicsObjectInternalInterface::GetWrite();
				const float CharacterMass = Interface.GetMass({ SimInputs->PhysicsObject });
				FVector ImpulseToApply = -FractionalGroundReactionImpulse * CharacterMass * JumpVelocity;
				if (Chaos::FPBDRigidParticleHandle* GroundParticle = Interface.GetRigidParticle(GroundObject))
				{
					if (GroundParticle->IsDynamic())
					{
						Private::ApplyImpulse(GroundParticle, ImpulseToApply, FloorResult.HitResult.ImpactPoint);
					}
				}
			}
		}
	}
	else
	{
		UE_LOGF(LogChaosMover, Warning, "UChaosCharacterJumpCheck requires FChaosMoverSimulationDefaultInputs");
	}
}