// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosMover/Character/Transitions/ChaosCharacterLandingCheck.h"

#include "ChaosMover/ChaosMoverLog.h"
#include "ChaosMover/ChaosMoverSimulation.h"
#include "ChaosMover/ChaosMoverSimulationTypes.h"
#include "ChaosMover/Utilities/ChaosGroundMovementUtils.h"
#include "DefaultMovementSet/CharacterMoverSimulationTypes.h"
#include "MoveLibrary/FloorQueryUtils.h"
#include "MoverSimulationTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosCharacterLandingCheck)

UChaosCharacterLandingCheck::UChaosCharacterLandingCheck(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSupportsAsync = true;
	bFirstSubStepOnly = true;

	TransitionToGroundMode = DefaultModeNames::Walking;
}

FTransitionEvalResult UChaosCharacterLandingCheck::Evaluate_Implementation(const FSimulationTickParams& Params) const
{
	FTransitionEvalResult EvalResult;

	if (!Simulation)
	{
		UE_LOGF(LogChaosMover, Warning, "No Simulation set on UChaosCharacterLandingCheck. Check you have a Chaos Backend");
		return EvalResult;
	}

	const FMoverTickStartData& StartState = Params.StartState;

	if (const FChaosMoverSimulationDefaultInputs* SimInputs = Simulation->GetLocalSimInput().FindDataByType<FChaosMoverSimulationDefaultInputs>())
	{
		const UMoverBlackboard* Blackboard = Simulation->GetBlackboard();
		check(Blackboard);

		const FCharacterDefaultInputs* CharacterInputs = StartState.InputCmd.InputCollection.FindDataByType<FCharacterDefaultInputs>();
		const FMoverDefaultSyncState* StartingSyncState = StartState.SyncState.SyncStateCollection.FindDataByType<FMoverDefaultSyncState>();
		if (!CharacterInputs || !StartingSyncState)
		{
			return EvalResult;
		}

		if (TStrongObjectPtr<const UBaseMovementMode> BaseMode = Simulation->FindMovementModeByName(Params.StartState.SyncState.MovementMode).Pin())
		{
			if (const IChaosCharacterMovementModeInterface* Mode = Cast<IChaosCharacterMovementModeInterface>(BaseMode.Get()))
			{
				const float DeltaSeconds = Params.TimeStep.StepMs * 0.001f;
				const FVector LinearVelocity = StartingSyncState->GetVelocity_WorldSpace();
				const float VerticalVelocity = SimInputs->UpDir.Dot(LinearVelocity);
				const bool bJumping = CharacterInputs && CharacterInputs->bIsJumpJustPressed;
				const bool bIsMovingUp = bJumping || (VerticalVelocity > 0.0f);
				const bool bIsAllowedToLand = !Simulation->HasGameplayTag(Mover_DisableLanding, false);
				
				// Check for ground landing
				if (!TransitionToGroundMode.IsNone() && bIsAllowedToLand)
				{
					FFloorCheckResult FloorResult;
					UE::ChaosMover::FGroundDynamicsInfo GroundDynamicsInfo;
					if (Blackboard->TryGet(CommonBlackboard::LastFloorResult, FloorResult) && FloorResult.IsWalkableFloor() && Blackboard->TryGet(UE::ChaosMover::Blackboard::GroundDynamicsInfo, GroundDynamicsInfo))
					{
						const float TargetHeight = Mode->GetTargetHeight();
						const float FloorDistanceWithFloorNormal = FloorResult.HitResult.ImpactNormal.Dot(StartingSyncState->GetLocation_WorldSpace() - FloorResult.HitResult.ImpactPoint);
						const FVector LocalGroundVelocity = GroundDynamicsInfo.LinearVelocity;
						const float RelativeVerticalVelocity = FloorResult.HitResult.ImpactNormal.Dot(LinearVelocity - LocalGroundVelocity);
						const float ProjectedFloorDistance = FloorDistanceWithFloorNormal + RelativeVerticalVelocity * DeltaSeconds;
						const bool bIsFloorWithinReach = ProjectedFloorDistance < TargetHeight + FloorDistanceTolerance + UE_KINDA_SMALL_NUMBER;
						const bool bIsMovingUpRelativeToFloor = (RelativeVerticalVelocity > UE_KINDA_SMALL_NUMBER) || bJumping;

						if (bIsFloorWithinReach && !bIsMovingUpRelativeToFloor)
						{
							if (Simulation->HasMovementMode(TransitionToGroundMode))
							{
								EvalResult.NextMode = TransitionToGroundMode;
							}
							else
							{
								UE_LOGF(LogChaosMover, Warning, "Invalid ground mode name %ls in ChaosCharacterLandingCheck. Cannot make transition", *TransitionToGroundMode.ToString());
							}
						
							return EvalResult;
						}
					}
				}
			}
		}
	}

	return EvalResult;
}

void UChaosCharacterLandingCheck::Trigger_Implementation(const FSimulationTickParams& Params)
{
	if (!Simulation)
	{
		UE_LOGF(LogChaosMover, Warning, "No Simulation set on UChaosCharacterLandingCheck. Check you have a Chaos Backend");
		return;
	}

	// Add a landed event to the simulation (will be broadcast on GT during post sim)
	if (const UMoverBlackboard* Blackboard = Simulation->GetBlackboard())
	{
		FFloorCheckResult FloorResult;
		if (Blackboard->TryGet(CommonBlackboard::LastFloorResult, FloorResult))
		{
			const TSharedPtr<FLandedEventData> LandedEventData = AllocLandedEventData();
			if (LandedEventData.IsValid())
			{
				PopulateLandedEventData(Params, FloorResult, *LandedEventData);
				Simulation->AddEvent(LandedEventData);
			}
		}
	}
}

TSharedPtr<FLandedEventData> UChaosCharacterLandingCheck::AllocLandedEventData()
{
	return MakeShared<FLandedEventData>();
}

void UChaosCharacterLandingCheck::PopulateLandedEventData(const FSimulationTickParams& Params, const FFloorCheckResult& FloorResult, FLandedEventData& OutLandedEvent)
{
	OutLandedEvent.Context = FMoverEventContext(Params.TimeStep);
	OutLandedEvent.HitResult = FloorResult.HitResult;
	OutLandedEvent.NewModeName = TransitionToGroundMode;
}
