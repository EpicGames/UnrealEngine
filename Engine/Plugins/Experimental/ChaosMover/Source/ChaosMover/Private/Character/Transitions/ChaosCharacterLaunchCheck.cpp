// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosMover/Character/Transitions/ChaosCharacterLaunchCheck.h"

#include "MoverComponent.h"
#include "MoveLibrary/FloorQueryUtils.h"
#include "ChaosMover/Character/Effects/ChaosCharacterApplyVelocityEffect.h"
#include "ChaosMover/Character/ChaosCharacterInputs.h"
#include "ChaosMover/ChaosMoverSimulation.h"
#include "ChaosMover/ChaosMoverLog.h"
#include "Chaos/PhysicsObjectInternalInterface.h"
#include "Chaos/PhysicsObject.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosCharacterLaunchCheck)


UChaosCharacterLaunchCheck::UChaosCharacterLaunchCheck(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSupportsAsync = true;
	bAllowModeReentry = true;
	bFirstSubStepOnly = true;
	TransitionToMode = DefaultModeNames::Falling;
}

FTransitionEvalResult UChaosCharacterLaunchCheck::Evaluate_Implementation(const FSimulationTickParams& Params) const
{
	FTransitionEvalResult EvalResult;

	if (const FChaosMoverLaunchInputs* LaunchInputs = Params.StartState.InputCmd.InputCollection.FindDataByType<FChaosMoverLaunchInputs>())
	{
		if (TransitionToMode.IsNone())
		{
			EvalResult.NextMode = Params.StartState.SyncState.MovementMode;
		}
		else
		{
			EvalResult.NextMode = TransitionToMode;
		}
	}

	return EvalResult;
}

void UChaosCharacterLaunchCheck::Trigger_Implementation(const FSimulationTickParams& Params)
{
	if (!Simulation)
	{
		UE_LOGF(LogChaosMover, Warning, "No Simulation set on UChaosCharacterLaunchCheck. Check you have a Chaos Backend");
		return;
	}

	if (const FChaosMoverLaunchInputs* LaunchInputs = Params.StartState.InputCmd.InputCollection.FindDataByType<FChaosMoverLaunchInputs>())
	{
		TSharedPtr<FChaosCharacterApplyVelocityEffect> LaunchMove = MakeShared<FChaosCharacterApplyVelocityEffect>();
		LaunchMove->VelocityOrImpulseToApply = LaunchInputs->LaunchVelocityOrImpulse;
		LaunchMove->Mode = LaunchInputs->Mode;

		Simulation->QueueInstantMovementEffect_Internal(LaunchMove);
	}
}
