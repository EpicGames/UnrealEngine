// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosMover/Character/Transitions/ChaosCharacterWaterCheck.h"

#include "CharacterMovementComponentAsync.h"
#include "ChaosMover/ChaosMoverLog.h"
#include "ChaosMover/ChaosMoverSimulation.h"
#include "ChaosMover/ChaosMoverSimulationTypes.h"
#include "MoveLibrary/WaterMovementUtils.h"
#include "MoverSimulationTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosCharacterWaterCheck)

UChaosCharacterWaterCheck::UChaosCharacterWaterCheck(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSupportsAsync = true;
 
	WaterModeName = DefaultModeNames::Swimming;
	GroundModeName = DefaultModeNames::Walking;
	AirModeName = DefaultModeNames::Falling;
}

FTransitionEvalResult UChaosCharacterWaterCheck::Evaluate_Implementation(const FSimulationTickParams& Params) const
{
	FTransitionEvalResult EvalResult;

	if (!Simulation)
	{
		UE_LOGF(LogChaosMover, Warning, "No Simulation set on UChaosCharacterWaterCheck. Check you have a Chaos Backend");
		return EvalResult;
	}

	const FMoverTickStartData& StartState = Params.StartState;

	if (const FChaosMoverSimulationDefaultInputs* SimInputs = Simulation->GetLocalSimInput().FindDataByType<FChaosMoverSimulationDefaultInputs>())
	{
		const UMoverBlackboard* Blackboard = Simulation->GetBlackboard();
		check(Blackboard);
 
		const FCharacterDefaultInputs* CharacterInputs = StartState.InputCmd.InputCollection.FindDataByType<FCharacterDefaultInputs>();
		const FMoverDefaultSyncState* StartingSyncState = StartState.SyncState.SyncStateCollection.FindDataByType<FMoverDefaultSyncState>();
		if (!CharacterInputs || !StartingSyncState)
		{
			return EvalResult;
		}

		if (TStrongObjectPtr<const UBaseMovementMode> BaseMode = Simulation->FindMovementModeByName(Params.StartState.SyncState.MovementMode).Pin())
		{
			if (const IChaosCharacterMovementModeInterface* Mode = Cast<IChaosCharacterMovementModeInterface>(BaseMode.Get()))
			{
				const float DeltaSeconds = Params.TimeStep.StepMs * 0.001f;
				const FVector LinearVelocity = StartingSyncState->GetVelocity_WorldSpace();
				const float VerticalVelocity = SimInputs->UpDir.Dot(LinearVelocity);
				const bool bJumping = CharacterInputs && CharacterInputs->bIsJumpJustPressed;
				const bool bIsMovingUp = bJumping || (VerticalVelocity > UE_KINDA_SMALL_NUMBER);

				FFindFloorResult FloorResult;
				FWaterCheckResult WaterResult;

				Blackboard->TryGet(CommonBlackboard::LastFloorResult, FloorResult);
				Blackboard->TryGet(CommonBlackboard::LastWaterResult, WaterResult);
			
				const float ProjectedImmersionDepth = WaterResult.WaterSplineData.ImmersionDepth - VerticalVelocity * DeltaSeconds;
				const bool bStartSwimming = ProjectedImmersionDepth > WaterModeStartImmersionDepth;
				const bool bInWater = StartState.SyncState.MovementMode == WaterModeName;
				const bool bStopSwimming = bInWater && (ProjectedImmersionDepth < WaterModeStopImmersionDepth);

				// Check if we need to enter/exit water mode
				if (WaterResult.IsSwimmableVolume() && bStartSwimming)
				{
					if (!WaterModeName.IsNone())
					{
						if (Simulation->HasMovementMode(WaterModeName))
						{
							EvalResult.NextMode = WaterModeName;
						}
						else
						{
							UE_LOGF(LogChaosMover, Warning, "Invalid water mode name %ls in ChaosCharacterWaterCheck. Cannot make transition", *WaterModeName.ToString());
						}
						return EvalResult;
					}
				}
				else if (bStopSwimming)
				{
					const bool bIsWithinReach = FloorResult.FloorDist <= Mode->GetTargetHeight();
 
					if (FloorResult.IsWalkableFloor() && bIsWithinReach && !bIsMovingUp)
					{
						if (!GroundModeName.IsNone())
						{
							if (Simulation->HasMovementMode(GroundModeName))
							{
								EvalResult.NextMode = GroundModeName;
							}
							else
							{
								UE_LOGF(LogChaosMover, Warning, "Invalid ground mode name %ls in ChaosCharacterWaterCheck. Cannot make transition", *GroundModeName.ToString());
							}
							return EvalResult;
						}
					}
					else
					{
						if (!AirModeName.IsNone())
						{
							if (Simulation->HasMovementMode(AirModeName))
							{
								EvalResult.NextMode = AirModeName;
							}
							else
							{
								UE_LOGF(LogChaosMover, Warning, "Invalid air mode name %ls in ChaosCharacterWaterCheck. Cannot make transition", *AirModeName.ToString());
							}
							return EvalResult;
						}
					}
				}
			}
		}
	}

	return EvalResult;
}

void UChaosCharacterWaterCheck::Trigger_Implementation(const FSimulationTickParams& Params)
{
}