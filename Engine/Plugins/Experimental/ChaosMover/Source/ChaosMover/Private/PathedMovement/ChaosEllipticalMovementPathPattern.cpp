// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosMover/PathedMovement/ChaosEllipticalMovementPathPattern.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosEllipticalMovementPathPattern)

void UChaosEllipticalMovementPathPattern::InitializePattern(UChaosMoverSimulation* InSimulation)
{
	Super::InitializePattern(InSimulation);
}

FTransform UChaosEllipticalMovementPathPattern::CalcUnmaskedTargetTransform(float PatternProgress, const FTransform& BasisTransform) const
{
	// Calc the ellipse point
	const float Angle = PatternProgress * FMath::DegreesToRadians(UsableArcAngle);
	FVector2f TargetLocationXY(RadiusX * FMath::Cos(Angle), RadiusY * FMath::Sin(Angle));

	// Scooch the center over so 0 progress means no movement
	TargetLocationXY.X -= RadiusX;

	// Spin about the path origin as desired
	TargetLocationXY = TargetLocationXY.GetRotated(OriginAngle);

	// Rotate the ellipse from the XY plane to whatever 3D plane is desired
	const FVector TargetLocation = EllipsePlaneRotation.RotateVector(FVector(TargetLocationXY.X, TargetLocationXY.Y, 0.f));

	return FTransform(BasisTransform.GetRotation(), BasisTransform.TransformPositionNoScale(TargetLocation), BasisTransform.GetScale3D());
}
