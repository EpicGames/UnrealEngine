// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosMover/PathedMovement/ChaosPathedMovementDebugDrawComponent.h"

#include "StaticMeshSceneProxy.h"
#include "Components/MeshComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/World.h"
#include "ChaosMover/PathedMovement/ChaosPathedMovementMode.h"
#include "ChaosMover/PathedMovement/ChaosPathedMovementPatternBase.h"
#include "MoverComponent.h"
#include "SceneInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosPathedMovementDebugDrawComponent)

TAutoConsoleVariable<int32> CVarChaosPathedMovementTotalDebugDrawSteps(
	TEXT("ChaosMover.PathedMovement.DebugDraw.TotalNumSteps"),
	64,
	TEXT("How many steps/lines to draw for each debug drawn path"));
	
TAutoConsoleVariable<int32> CVarChaosPathedMovementDisplayedDebugDrawSteps(
	TEXT("ChaosMover.PathedMovement.DebugDraw.DisplayedSteps"),
	0,
	TEXT("Of the total number of steps in the path debug draw (see ChaosMover.PathedMovement.DebugDraw.TotalNumSteps), how many should actually get drawn? If <= 0, all steps are drawn."));

class FChaosPathedMovementDebugRenderSceneProxy : public FDebugRenderSceneProxy
{
public:
	explicit FChaosPathedMovementDebugRenderSceneProxy(const UPrimitiveComponent& InComponent)
		: FDebugRenderSceneProxy(&InComponent)
	{}

	virtual SIZE_T GetTypeHash() const override 
	{
		static size_t UniquePointer;
		return reinterpret_cast<size_t>(&UniquePointer);
	}
	
	virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override
	{
		UWorld* World = (View && View->Family && View->Family->Scene) ? View->Family->Scene->GetWorld() : nullptr;
		bool bIsAssetViewport = World && (World->WorldType == EWorldType::EditorPreview);
		FPrimitiveViewRelevance Result;
		//@todo DanH: Add a show flag for these, like the one for spline components: View->Family->EngineShowFlags.Splines
		Result.bDrawRelevance = IsShown(View) && (bIsAssetViewport || IsSelected());
		Result.bDynamicRelevance = true;
		// Result.bShadowRelevance = IsShadowCast(View);
		Result.bEditorPrimitiveRelevance = true; // UseEditorCompositing(View);
		return Result;
	}

	virtual uint32 GetMemoryFootprint(void) const override { return sizeof(*this) + GetAllocatedSize(); }
};

UChaosPathedMovementDebugDrawComponent::UChaosPathedMovementDebugDrawComponent()
{
	// bIsEditorOnly = true;
}

void UChaosPathedMovementDebugDrawComponent::OnRegister()
{
	// DestroyProgressPreviewMeshComp();
	UpdatePreviewMeshComp();

#if WITH_EDITOR
	FCoreUObjectDelegates::OnObjectPropertyChanged.AddUObject(this, &ThisClass::HandleObjectPropertyChanged);
#endif

	Super::OnRegister();
}

void UChaosPathedMovementDebugDrawComponent::OnUnregister()
{
	Super::OnUnregister();

#if WITH_EDITOR
	FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll(this);
#endif

	if (ProgressPreviewMeshComp)
	{
		ProgressPreviewMeshComp->UnregisterComponent();
	}
}

FBoxSphereBounds UChaosPathedMovementDebugDrawComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	return Bounds;
}

void UChaosPathedMovementDebugDrawComponent::CreateRenderState_Concurrent(FRegisterComponentContext* Context)
{
	// Calculate everything for the scene proxy here so that our bounds are ready early enough (they get checked just before the scene proxy gets added)
	FBoxSphereBounds::Builder DebugBoundsBuilder;
	DebugBoundsBuilder += GetOwner()->GetTransform().GetLocation();
	
	DebugLines.Reset();
	DebugDashedLines.Reset();
	DebugArrowLines.Reset();
	DebugSpheres.Reset();
	DebugStars.Reset();

	if (const UMoverComponent* MoverComp = GetMoverComponent())
	{
		const TArray<UActorComponent*> DebugDrawInterfaceComponents = GetOwner()->GetComponentsByInterface(UChaosPathedMovementDebugDrawInterface::StaticClass());
		if (!DebugDrawInterfaceComponents.IsEmpty())
		{
			DebugDrawInterfaceObject = DebugDrawInterfaceComponents[0];
		}
	
		const UChaosPathedMovementMode* PathedMovementMode = MoverComp->FindMode_Mutable<UChaosPathedMovementMode>(MoverComp->StartingMovementMode);
		if (PathedMovementMode && !PathedMovementMode->PathPatterns.IsEmpty() && (PathedMovementMode->bDebugDrawAggregatePath || PathedMovementMode->bAllowPatternDebugDrawing))
		{
			// Collect valid patterns and count the number of total curves to draw to reserve enough space for DebugLines
			TArray<const UChaosPathedMovementPatternBase*> ValidPatterns;
			int32 NumCurvesToDraw = PathedMovementMode->bDebugDrawAggregatePath ? 1 : 0;
			for (UChaosPathedMovementPatternBase* PathPattern : PathedMovementMode->PathPatterns)
			{
				if (PathPattern)
				{
					ValidPatterns.Emplace(PathPattern);
					if (PathPattern->DebugDrawUsingStepSamples())
					{
						++NumCurvesToDraw;
					}

					if (PathedMovementMode->bAllowPatternDebugDrawing)
					{
						PathPattern->AppendDebugDrawElements(*this, DebugBoundsBuilder);
					}
				}
			}

			// Reserve enough debug lines to not have to reallocate on addition
			const float NumSteps = CVarChaosPathedMovementTotalDebugDrawSteps.GetValueOnGameThread();
			DebugLines.Reserve(DebugLines.Num() + NumSteps * NumCurvesToDraw);

			// This is an array of path progress samples that we will use if we draw the aggregate path
			// The spacing of samples needs to adapt to path patterns or the sampling resolution my
			// skip entire patterns by stepping too coarsely. As we sample patterns, we may add the same sample multiple
			// times or out of order. This array will then be sorted and values that are too close to the previous ones
			// will be skipped
			TArray<float> PathProgressSamples;
			if (PathedMovementMode->bDebugDrawAggregatePath)
			{
				// We preserve for a minimum but may have to reallocate depending on patterns
				PathProgressSamples.Reserve(NumSteps);
			}
		
			// This is the inferred path basis
			const FTransform LocalToWorld = GetOwner()->GetTransform();

			// Establish a previous transform for each path pattern, and add a sample for the aggregate path for the start of each pattern
			TArray<FTransform> PreviousStepPatternTransforms;
			PreviousStepPatternTransforms.SetNumUninitialized(ValidPatterns.Num());
			for (int32 PatternIdx = 0; PatternIdx < ValidPatterns.Num(); ++PatternIdx)
			{
				const UChaosPathedMovementPatternBase& Pattern = *ValidPatterns[PatternIdx];
				if (PathedMovementMode->bDebugDrawAggregatePath)
				{
					// Add an aggregate path sample for the beginning of this path.
					PathProgressSamples.Add(Pattern.ConvertPatternToPathProgress(Pattern.StartAtPathProgress));
				}
				const bool bDrawSampledPattern = Pattern.DebugDrawUsingStepSamples() && PathedMovementMode->bAllowPatternDebugDrawing;
				if (bDrawSampledPattern)
				{
					const FTransform FirstPatternTransform = Pattern.CalcMaskedTargetTransform(0.0f, LocalToWorld);
					PreviousStepPatternTransforms[PatternIdx] = FirstPatternTransform;
					DebugBoundsBuilder += FirstPatternTransform.GetLocation();
				}
			}			
		
			// Now sample each pattern to draw them, and add that sample to the aggregate path sample if we draw it
			const int32 NumStepsToDraw = CVarChaosPathedMovementDisplayedDebugDrawSteps.GetValueOnGameThread() > 0 ? CVarChaosPathedMovementDisplayedDebugDrawSteps.GetValueOnGameThread() : NumSteps;
			for (int32 Step = 1; Step <= NumStepsToDraw; ++Step)
			{
				for (int32 PatternIdx = 0; PatternIdx < ValidPatterns.Num(); ++PatternIdx)
				{
					const UChaosPathedMovementPatternBase& Pattern = *ValidPatterns[PatternIdx];
					const float PatternProgress = Step / NumSteps;
					if (PathedMovementMode->bDebugDrawAggregatePath)
					{
						PathProgressSamples.Add(Pattern.ConvertPatternToPathProgress(PatternProgress));
					}
					const bool bDrawSampledPattern = Pattern.DebugDrawUsingStepSamples() && PathedMovementMode->bAllowPatternDebugDrawing;
					if (bDrawSampledPattern)
					{
						const FTransform PatternTransform = Pattern.CalcMaskedTargetTransform(PatternProgress, LocalToWorld);
						DebugLines.Emplace(PreviousStepPatternTransforms[PatternIdx].GetLocation(), PatternTransform.GetLocation(), Pattern.PatternDebugDrawColor, 1.f);

						PreviousStepPatternTransforms[PatternIdx] = PatternTransform;
						DebugBoundsBuilder += PatternTransform.GetLocation();
					}
				}
			}

			// Aggregate Path
			if (PathedMovementMode->bDebugDrawAggregatePath)
			{
				// Sort overall path progress samples, they may have been added out of order when iterating over the patterns
				PathProgressSamples.Sort();

				// Establish the first point on the aggregate path
				FVector PreviousStepAggregateLocation = PathedMovementMode->CalcTargetTransform(0.0f, LocalToWorld).GetLocation();
				DebugBoundsBuilder += PreviousStepAggregateLocation;
				float PreviousPathProgressSample = 0.0f;
				// Draw the rest of the aggregate path, at the overall progress samples deduced from the sampling of path patterns
				// and skip samples that are too close to the previous one we drew
				for (float PathProgressSample : PathProgressSamples)
				{
					if (FMath::IsNearlyEqual(PathProgressSample, PreviousPathProgressSample)) // Should we look at the difference in transform location instead?
					{
						continue;
					}
					const float OverallPathProgress = PathProgressSample;
					const FVector StepLocation = PathedMovementMode->CalcTargetTransform(OverallPathProgress, LocalToWorld).GetLocation();
					DebugLines.Emplace(PreviousStepAggregateLocation, StepLocation, PathedMovementMode->PathDebugDrawColor, 2.f);
					PreviousStepAggregateLocation = StepLocation;
					DebugBoundsBuilder += StepLocation;
					PreviousPathProgressSample = PathProgressSample;
				}
			}
		}
	}

	Bounds = DebugBoundsBuilder;
	Bounds = Bounds.ExpandBy(25.);

	Super::CreateRenderState_Concurrent(Context);
}

FDebugRenderSceneProxy* UChaosPathedMovementDebugDrawComponent::CreateDebugSceneProxy()
{
	if (DebugLines.IsEmpty() &&
		DebugDashedLines.IsEmpty() &&
		DebugArrowLines.IsEmpty() &&
		DebugSpheres.IsEmpty() &&
		DebugStars.IsEmpty())
	{
		return nullptr;
	}

	FChaosPathedMovementDebugRenderSceneProxy* DebugSceneProxy = new FChaosPathedMovementDebugRenderSceneProxy(*this);
    DebugSceneProxy->DrawType = FDebugRenderSceneProxy::SolidAndWireMeshes;
    DebugSceneProxy->ViewFlagName = TEXT("Splines");
    DebugSceneProxy->ViewFlagIndex = uint32(FEngineShowFlags::FindIndexByName(*DebugSceneProxy->ViewFlagName));
	
	DebugSceneProxy->Lines = DebugLines;
	DebugSceneProxy->DashedLines = DebugDashedLines;
	DebugSceneProxy->ArrowLines = DebugArrowLines;
	DebugSceneProxy->Spheres = DebugSpheres;
	DebugSceneProxy->Stars = DebugStars;

	// We can now get rid of our debug primitives since we transfered them to the scene proxy
	// CreateDebugSceneProxy is called by UDebugDrawComponent::CreateRenderState_Concurrent which will
	// have caused UChaosPathedMovementDebugDrawComponent::CreateRenderState_Concurrent to recreate
	// them next time there is a need to update those
	DebugLines.Reset();
	DebugDashedLines.Reset();
	DebugArrowLines.Reset();
	DebugSpheres.Reset();
	DebugStars.Reset();
	
	return DebugSceneProxy;
}

void UChaosPathedMovementDebugDrawComponent::UpdatePreviewMeshComp(bool bForce)
{
	const UMoverComponent* MoverComp = GetMoverComponent();
	if (!MoverComp)
	{
		return;
	}

	UMeshComponent* MeshRoot = GetOwnerMeshRoot();
	if (MeshRoot && GetWorld() && !GetWorld()->IsGameWorld())
	{
		bool bDoUpdate = false;
		
    	if (ProgressPreviewMeshComp)
    	{
		    if (ProgressPreviewMeshComp->GetClass() == MeshRoot->GetClass() && bForce)
		    {
		    	bDoUpdate = true;

		    	// Copy all the properties on the root mesh comp (except attachments)
		    	for (TFieldIterator<FProperty> PropIter(MeshRoot->GetClass()); PropIter; ++PropIter)
		    	{
				    if (PropIter->GetFName() == TEXT("AttachChildren") || PropIter->GetFName() == TEXT("AttachParent"))
				    {
					    continue;
				    }
		    		PropIter->CopyCompleteValue_InContainer(ProgressPreviewMeshComp, MeshRoot);
		    	}
		    }
		    else
		    {
			    DestroyProgressPreviewMeshComp();
		    }
    	}

		if (!ProgressPreviewMeshComp)
    	{
    		ProgressPreviewMeshComp = DuplicateObject<UMeshComponent>(MeshRoot, this, TEXT("ProgressPreviewMeshComp"));
    		// ProgressPreviewMeshComp->SetupAttachment(this);
			ProgressPreviewMeshComp->bAutoRegister = false;

			bDoUpdate = true;
    	}

		ensure(ProgressPreviewMeshComp->GetNumChildrenComponents() == 0);

		if (!ProgressPreviewMeshComp->IsRegistered())
		{
			ProgressPreviewMeshComp->RegisterComponent();
		}

		if (bDoUpdate)
		{
			//@todo DanH: Double-clicking the preview comp in the level editor can still select the component - how do I disable that?
			ProgressPreviewMeshComp->CreationMethod = EComponentCreationMethod::UserConstructionScript;
			ProgressPreviewMeshComp->SetSimulatePhysics(false);
			ProgressPreviewMeshComp->SetCollisionEnabled(ECollisionEnabled::NoCollision);
			ProgressPreviewMeshComp->bIsEditorOnly = true;
			ProgressPreviewMeshComp->SetHiddenInGame(true);

			const TArray<UActorComponent*> DebugDrawInterfaceComponents = GetOwner()->GetComponentsByInterface(UChaosPathedMovementDebugDrawInterface::StaticClass());
			if (!DebugDrawInterfaceComponents.IsEmpty())
			{
				DebugDrawInterfaceObject = DebugDrawInterfaceComponents[0];
			}

			const UChaosPathedMovementMode* PathedMovementMode = MoverComp->FindMode_Mutable<UChaosPathedMovementMode>(MoverComp->StartingMovementMode);			
			if (PathedMovementMode && DebugDrawInterfaceObject && IChaosPathedMovementDebugDrawInterface::Execute_ShouldDisplayProgressPreviewMesh(DebugDrawInterfaceObject))
			{
				ProgressPreviewMeshComp->SetVisibility(true);
				
				float OverallPathProgress = IChaosPathedMovementDebugDrawInterface::Execute_GetPreviewMeshOverallPathProgress(DebugDrawInterfaceObject);
				// Calculate the location without scale, then tack on the source mesh's scale
				FTransform PreviewMeshWorldTransform = PathedMovementMode->CalcTargetTransform(OverallPathProgress, FTransform(MeshRoot->GetComponentTransform().ToMatrixNoScale()));
				PreviewMeshWorldTransform.SetScale3D(MeshRoot->GetComponentScale());
				ProgressPreviewMeshComp->SetWorldTransform(PreviewMeshWorldTransform);

				UMaterialInterface* ProgressPreviesMeshMaterial = IChaosPathedMovementDebugDrawInterface::Execute_GetProgressPreviewMeshMaterial(DebugDrawInterfaceObject);
				if (ProgressPreviesMeshMaterial)
				{
					const int32 NumMaterials = ProgressPreviewMeshComp->GetNumMaterials();
					for (int MaterialIdx = 0; MaterialIdx < NumMaterials; ++MaterialIdx)
					{
						ProgressPreviewMeshComp->SetMaterial(MaterialIdx, ProgressPreviesMeshMaterial);
					}
				}
			}
			else
			{
				ProgressPreviewMeshComp->SetVisibility(false);
			}
		}
	}
	else if (ProgressPreviewMeshComp)
	{
		DestroyProgressPreviewMeshComp();
	}
}

void UChaosPathedMovementDebugDrawComponent::DestroyProgressPreviewMeshComp()
{
	if (ProgressPreviewMeshComp)
	{
		ProgressPreviewMeshComp->DestroyComponent();
		ProgressPreviewMeshComp = nullptr;
	}
}

UMeshComponent* UChaosPathedMovementDebugDrawComponent::GetOwnerMeshRoot() const
{
	if (AActor* Owner = GetOwner())
	{
		return Cast<UMeshComponent>(Owner->GetRootComponent());
	}
	return nullptr;
}

UMoverComponent* UChaosPathedMovementDebugDrawComponent::GetMoverComponent() const
{
	AActor* OwnerActor = GetOwner();
	return OwnerActor ? OwnerActor->FindComponentByClass<UMoverComponent>() : nullptr;
}

#if WITH_EDITOR
void UChaosPathedMovementDebugDrawComponent::HandleObjectPropertyChanged(UObject* Object, FPropertyChangedEvent& PropertyChangedEvent)
{
	if (UMeshComponent* MeshComponent = Cast<UMeshComponent>(Object);
		MeshComponent && MeshComponent == GetOwner()->GetRootComponent())
	{
		UpdatePreviewMeshComp();		
	}
}
#endif
