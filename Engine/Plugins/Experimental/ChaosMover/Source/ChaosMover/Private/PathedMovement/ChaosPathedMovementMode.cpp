// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosMover/PathedMovement/ChaosPathedMovementMode.h"

#include "Chaos/PBDJointConstraintData.h"
#include "Chaos/PBDJointConstraints.h"
#include "ChaosMover/PathedMovement/ChaosPathedMovementPatternBase.h"
#include "ChaosMover/ChaosMoverSimulation.h"
#include "ChaosMover/ChaosMoverLog.h"
#include "MoverComponent.h"
#include "MoverSimulationTypes.h"
#include "Physics/ImmediatePhysics/ImmediatePhysicsAdapters.h"
#include "Framework/Threading.h"
#include "Kismet/KismetMathLibrary.h"
#include "Misc/DataValidation.h"
#include "ChaosVisualDebugger/ChaosVisualDebuggerTrace.h"

#if WITH_EDITOR
#include "Engine/SimpleConstructionScript.h"
#include "Engine/SCS_Node.h"
#include "Components/PrimitiveComponent.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosPathedMovementMode)

#define LOCTEXT_NAMESPACE "UChaosPathedMovementMode"

UChaosPathedMovementMode::UChaosPathedMovementMode(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	bSupportsAsync = true;

	// Sets stiff constraint properties by default
	SetDefaultJointConstraintProperties();
}

void UChaosPathedMovementMode::OnRegistered(const FName ModeName, const FMoverSimContext& SimContext)
{
	Chaos::EnsureIsInGameThreadContext();

	Super::OnRegistered(ModeName, SimContext);

	if (const UChaosMoverSimulation* Sim = GetSimulation())
	{
		bIsUsingAsyncPhysics = Sim->IsUsingAsyncPhysics();
	}

	InitializePath();

	ImmediatePhysics::UpdateJointSettingsFromConstraintProfile(JointConstraintProperties, JointSettings);
}

FMoverTime UChaosPathedMovementMode::MakeChangeTime(const FMoverTimeStep& TimeStep) const
{
	FMoverTime T;
	T.FrameCount = bIsUsingAsyncPhysics ? TimeStep.ServerFrame : INDEX_NONE;
	T.TimeMs = TimeStep.BaseSimTimeMs;
	return T;
}

bool UChaosPathedMovementMode::ShouldUseConstraint() const
{
	return bUseJointConstraint;
}

const Chaos::FPBDJointSettings& UChaosPathedMovementMode::GetConstraintSettings() const
{
	return JointSettings;
}

void UChaosPathedMovementMode::GenerateMove_Implementation(const FMoverSimContext& SimContext, const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, FProposedMove& OutProposedMove) const
{
}

void UChaosPathedMovementMode::SimulationTick_Implementation(const FSimulationTickParams& Params, FMoverTickEndData& OutputState)
{
	QUICK_SCOPE_CYCLE_COUNTER(ChaosPathedMovementMode_SimulationTick);

	Chaos::EnsureIsInPhysicsThreadContext();

	if (!ensureMsgf(Simulation, TEXT("No Simulation set on ChaosPathedMovementMode")))
	{
		return;
	}

	// We retrieve the previous properties so we can compare with the new. If there are none we default to FChaosPathedMovementState()
	const FChaosPathedMovementState* FoundPreviousMoveState = Params.StartState.SyncState.SyncStateCollection.FindDataByType<FChaosPathedMovementState>();
	const FChaosPathedMovementState PreviousMoveState = FoundPreviousMoveState ? *FoundPreviousMoveState : FChaosPathedMovementState();	
	const FChaosMutablePathedMovementProperties& PreviousProperties = PreviousMoveState.PropertiesInEffect;

	const FChaosPathedMovementInputs* PathedMovementInputs = Params.StartState.InputCmd.InputCollection.FindDataByType<FChaosPathedMovementInputs>();
	const FChaosMutablePathedMovementProperties& InputProperties = PathedMovementInputs ? PathedMovementInputs->Props : PreviousProperties;

	// We initialize the output move state from the previous move state
	FChaosPathedMovementState& OutputPathedMovemenState = OutputState.SyncState.SyncStateCollection.FindOrAddMutableDataByType<FChaosPathedMovementState>();
	// We initialize the output movement state from the previous movement state
	OutputPathedMovemenState = PreviousMoveState;

	// Compute the current path progress before we modify movement properties or update LastChangeTime
	float CurrentPathProgress = PreviousMoveState.LastChangePathProgress;
	if (PreviousProperties.bWantsToPlay && !PreviousMoveState.bHasFinished)
	{
		const float PlaybackSpeed = (OneWayPlaybackDuration >= 0.0f) ? (1.0f / OneWayPlaybackDuration) : 0.0f;
		const float PreviousDirection = PreviousMoveState.bIsPathProgressionIncreasing ? 1.0f : -1.0f;
		const double FrameDt = Params.TimeStep.StepMs * 0.001;
		const double Elapsed = PreviousMoveState.LastChangeTime.ElapsedSecondsTo(Params.TimeStep.ToStartTime(), FrameDt);
		CurrentPathProgress += (float)(Elapsed * PreviousDirection * PlaybackSpeed);
	}
	
	bool bReceivedNewStartRequest = false;

	// If we have inputs,
	if (PathedMovementInputs
		// ... and we have reached the frame at which they should enter into effect
		&& (Params.TimeStep.ServerFrame >= PathedMovementInputs->LastChangeFrame)
		// ... and they are different than the ones in effect currently
		//         or the input's MovementStartFrame is later than the output's movement start frame
		&& ((PreviousProperties != PathedMovementInputs->Props)
			|| (PathedMovementInputs->MovementStartFrame > OutputPathedMovemenState.LatestMovementStartFrame)))
	{
		// ... then the input properties enter into effect
		OutputPathedMovemenState.PropertiesInEffect = PathedMovementInputs->Props;

		// Since we changed movement properties, we record the path progress and frame at which it changed
		// This is all that should be needed to be able to compute path progress for future frames, until movement properties change again
		// From now on all frame differences should be using these values and not the ones in PreviousMoveState
		OutputPathedMovemenState.LastChangePathProgress = CurrentPathProgress;
		OutputPathedMovemenState.LastChangeTime = MakeChangeTime(Params.TimeStep);

		bReceivedNewStartRequest = OutputPathedMovemenState.PropertiesInEffect.bWantsToPlay
			&& (!PreviousProperties.bWantsToPlay
			    || PathedMovementInputs->MovementStartFrame > OutputPathedMovemenState.LatestMovementStartFrame);
	}

	// PropertiesInEffect is const here, to indicate it is a bad idea to modify those, since they are used to detect external changes in mutable properties,
	// which would become confusing if modified internally.
	const FChaosMutablePathedMovementProperties& CurrentProperties = OutputPathedMovemenState.PropertiesInEffect;

	// These booleans will determine whether events get fired at the end of this function
	bool bHasJustFinished = false;
	bool bHasJustBounced = false;
	bool bJustRestarted = false;
	bool bWasStoppedJustRequested = PreviousProperties.bWantsToPlay && !CurrentProperties.bWantsToPlay;

	// When bWantsReversePlayback changes and we are still playing, we change the direction of motion
	bool bIsInReverseJustChanged = PreviousProperties.bWantsReversePlayback != CurrentProperties.bWantsReversePlayback;
	if (!OutputPathedMovemenState.bHasFinished && bIsInReverseJustChanged)
	{
		OutputPathedMovemenState.bIsPathProgressionIncreasing = !OutputPathedMovemenState.bIsPathProgressionIncreasing;
	}

	// Reset bHasFinished if we are restarting
	if (bReceivedNewStartRequest && OutputPathedMovemenState.bHasFinished)
	{
		OutputPathedMovemenState.bHasFinished = false;
		// We initialize the direction of motion to respect reverse playback on start
		OutputPathedMovemenState.bIsPathProgressionIncreasing = !CurrentProperties.bWantsReversePlayback;
		if (CurrentProperties.bWantsOneWayPlayback)
		{
			if (CurrentPathProgress == (OutputPathedMovemenState.bIsPathProgressionIncreasing ? 1.0f : 0.0f))
			{
				CurrentPathProgress = 1.0f - CurrentPathProgress;
			}
		}
	}

	bool bLoopingJustChanged = PreviousProperties.bWantsLoopingPlayback != CurrentProperties.bWantsLoopingPlayback;
	if (bLoopingJustChanged && CurrentProperties.bWantsToPlay && OutputPathedMovemenState.bHasFinished)
	{
		OutputPathedMovemenState.bHasFinished = false;
		bJustRestarted = true;
	}

	float NewPathProgress = CurrentPathProgress;
	if (CurrentProperties.bWantsToPlay && (!PreviousMoveState.bHasFinished || bReceivedNewStartRequest))
	{
		const float PlaybackSpeed = (OneWayPlaybackDuration >= 0.0f) ? (1.0f / FMath::Max(OneWayPlaybackDuration, SMALL_NUMBER)) : 0.0f;
		const float Direction = OutputPathedMovemenState.bIsPathProgressionIncreasing ? 1.0f : -1.0f;
		const double FrameDt = Params.TimeStep.StepMs * 0.001;
		const double Elapsed = OutputPathedMovemenState.LastChangeTime.ElapsedSecondsTo(Params.TimeStep.ToNextTime(), FrameDt);
		NewPathProgress = OutputPathedMovemenState.LastChangePathProgress + (float)(Elapsed * Direction * PlaybackSpeed);

		if (CurrentProperties.bWantsOneWayPlayback)
		{
			// One way forward playback
			if (OutputPathedMovemenState.bIsPathProgressionIncreasing)
			{
				if (NewPathProgress >= 1.0f)
				{
					if (CurrentProperties.bWantsLoopingPlayback)
					{
						NewPathProgress = FMath::Fmod(NewPathProgress, 1.0f);
						bJustRestarted = true;
						OutputPathedMovemenState.LastChangePathProgress = NewPathProgress;
						OutputPathedMovemenState.LastChangeTime = MakeChangeTime(Params.TimeStep);
					}
					else
					{
						NewPathProgress = 1.0f;
						bHasJustFinished = true;
						OutputPathedMovemenState.bHasFinished = true;
						OutputPathedMovemenState.LastChangePathProgress = NewPathProgress;
						OutputPathedMovemenState.LastChangeTime = MakeChangeTime(Params.TimeStep);
					}
				}
			}
			else // One way reverse playback
			{
				if (NewPathProgress <= 0.0f)
				{
					if (CurrentProperties.bWantsLoopingPlayback)
					{
						NewPathProgress = 1.0f - FMath::Fmod(-NewPathProgress, 1.0f);
						bJustRestarted = true;
						OutputPathedMovemenState.LastChangePathProgress = NewPathProgress;
						OutputPathedMovemenState.LastChangeTime = MakeChangeTime(Params.TimeStep);
					}
					else
					{
						NewPathProgress = 0.0f;
						bHasJustFinished = true;
						OutputPathedMovemenState.bHasFinished = true;
						OutputPathedMovemenState.LastChangePathProgress = NewPathProgress;
						OutputPathedMovemenState.LastChangeTime = MakeChangeTime(Params.TimeStep);
					}
				}
			}
		}
		else
		{
			// Round-trip playback
			if (OutputPathedMovemenState.bIsPathProgressionIncreasing && NewPathProgress >= 1.0f)
			{
				if (!CurrentProperties.bWantsReversePlayback || CurrentProperties.bWantsLoopingPlayback)
				{
					OutputPathedMovemenState.bIsPathProgressionIncreasing = false;
					NewPathProgress = 1.0f - FMath::Fmod(NewPathProgress, 1.0f);
					bHasJustBounced = true;
					OutputPathedMovemenState.LastChangePathProgress = NewPathProgress;
					OutputPathedMovemenState.LastChangeTime = MakeChangeTime(Params.TimeStep);
				}
				else
				{
					NewPathProgress = 1.0f;
					bHasJustFinished = true;
					OutputPathedMovemenState.bHasFinished = true;
					OutputPathedMovemenState.LastChangePathProgress = NewPathProgress;
					OutputPathedMovemenState.LastChangeTime = MakeChangeTime(Params.TimeStep);
				}
			}
			else if (!OutputPathedMovemenState.bIsPathProgressionIncreasing && NewPathProgress <= 0.0f)
			{
				if (CurrentProperties.bWantsReversePlayback || CurrentProperties.bWantsLoopingPlayback)
				{
					NewPathProgress = FMath::Fmod(-NewPathProgress, 1.0f);
					bHasJustBounced = true;
					OutputPathedMovemenState.bIsPathProgressionIncreasing = true;
					OutputPathedMovemenState.LastChangePathProgress = NewPathProgress;
					OutputPathedMovemenState.LastChangeTime = MakeChangeTime(Params.TimeStep);
				}
				else
				{
					NewPathProgress = 0.0f;
					bHasJustFinished = true;
					OutputPathedMovemenState.bHasFinished = true;
					OutputPathedMovemenState.LastChangePathProgress = NewPathProgress;
					OutputPathedMovemenState.LastChangeTime = MakeChangeTime(Params.TimeStep);
				}
			}
		}
	}

	if (bIsInReverseJustChanged)
	{
		OnInReverseChanging(Params.TimeStep, CurrentProperties.bWantsReversePlayback);
	}
	if (bLoopingJustChanged)
	{
		OnLoopingChanging(Params.TimeStep, CurrentProperties.bWantsLoopingPlayback);
	}
	bool bOneWayJustChanged = CurrentProperties.bWantsOneWayPlayback != PreviousProperties.bWantsOneWayPlayback;
	if (bOneWayJustChanged)
	{
		OnOneWayChanging(Params.TimeStep, CurrentProperties.bWantsOneWayPlayback);
	}

	if (bReceivedNewStartRequest)
	{
		OnPlaybackStarting(Params.TimeStep);
		OutputPathedMovemenState.LatestMovementStartFrame = Params.TimeStep.ServerFrame;
	}

	if ((bWasStoppedJustRequested && !OutputPathedMovemenState.bHasFinished) || bHasJustFinished)
	{
		OnPlaybackStopping(Params.TimeStep, bHasJustFinished);
	}

	if (bHasJustBounced)
	{
		OnPlaybackBounced(Params.TimeStep);
	}

	FMoverDefaultSyncState& OutputDefaultSyncState = OutputState.SyncState.SyncStateCollection.FindOrAddMutableDataByType<FMoverDefaultSyncState>();

	const FTransform TargetWorldTransform = CalcTargetTransform(NewPathProgress, Simulation->GetMovementBasisTransform());
	
	FVector WorldTranslation(TargetWorldTransform.GetTranslation());
	FRotator WorldRotation(TargetWorldTransform.GetRotation());

	OutputDefaultSyncState.SetTransforms_WorldSpace(WorldTranslation, WorldRotation,
											  FVector::ZeroVector, // Fix this eventually to return the velocity tangential to the path, including due to a change in path basis since last update
											  FVector::ZeroVector,
											  nullptr ); // no movement base

#if WITH_CHAOS_VISUAL_DEBUGGER
	if (FChaosVisualDebuggerTrace::IsTracing())
	{
		DebugData.ServerFrame = Params.TimeStep.ServerFrame;
		DebugData.PreSimMoveState = PreviousMoveState;
		DebugData.PostSimMoveState = OutputPathedMovemenState;
		const FMoverDefaultSyncState* PreviousDefaultSyncState = Params.StartState.SyncState.SyncStateCollection.FindDataByType<FMoverDefaultSyncState>();
		DebugData.PreviousWorldTranslation = PreviousDefaultSyncState ? PreviousDefaultSyncState->GetTransform_WorldSpace().GetLocation() : FVector::ZeroVector;
		DebugData.PreviousWorldRotation = PreviousDefaultSyncState ? FRotator(PreviousDefaultSyncState->GetTransform_WorldSpace().GetRotation()) : FRotator::ZeroRotator;
		DebugData.NewWorldTranslation = WorldTranslation;
		DebugData.NewWorldRotation = WorldRotation;
		// Record state for debugging
		UE::ChaosMover::GetDebugSimData(Simulation).FindOrAddMutableDataByType<FChaosPathedMovementModeDebugData>() = DebugData;
		// Advance the debug data. We do this immediately to avoid stomping anything happening between now and next time we trace
		DebugData.Advance();
	}
#endif
}

bool UChaosPathedMovementMode::UsesMovementBasisTransform() const
{
	return true;
}

void UChaosPathedMovementMode::OnPlaybackStarting(const FMoverTimeStep& SimTime)
{
	// Add a pathed movement started event to the simulation (will be broadcast on GT during post sim)
	if (Simulation)
	{
		Simulation->AddEvent(MakeShared<FPathedMovementStartedEventData>(SimTime));

#if WITH_CHAOS_VISUAL_DEBUGGER
	if (FChaosVisualDebuggerTrace::IsTracing())
	{
		DebugData.bPlaybackStartedThisFrame = true;
	}
#endif // WITH_CHAOS_VISUAL_DEBUGGER
	}
}

void UChaosPathedMovementMode::OnPlaybackStopping(const FMoverTimeStep& SimTime, bool bReachedEndOfPlayback)
{
	// Add a pathed movement stopped event to the simulation (will be broadcast on GT during post sim)
	if (Simulation)
	{
		Simulation->AddEvent(MakeShared<FPathedMovementStoppedEventData>(SimTime, bReachedEndOfPlayback));

#if WITH_CHAOS_VISUAL_DEBUGGER
	if (FChaosVisualDebuggerTrace::IsTracing())
	{
		DebugData.bPlaybackStoppedThisFrame = true;
		DebugData.bPlaybackReachedEndThisFrame = bReachedEndOfPlayback;
	}
#endif // WITH_CHAOS_VISUAL_DEBUGGER
	}
}

void UChaosPathedMovementMode::OnPlaybackBounced(const FMoverTimeStep& SimTime)
{
	// Add a pathed movement bounced event to the simulation (will be broadcast on GT during post sim)
	if (Simulation)
	{
		Simulation->AddEvent(MakeShared<FPathedMovementBouncedEventData>(SimTime));

#if WITH_CHAOS_VISUAL_DEBUGGER
		if (FChaosVisualDebuggerTrace::IsTracing())
		{
			DebugData.bPlaybackBouncedThisFrame = true;
		}
#endif // WITH_CHAOS_VISUAL_DEBUGGER
	}
}

void UChaosPathedMovementMode::OnInReverseChanging(const FMoverTimeStep& SimTime, bool bNewInReverse)
{
	// Add a pathed movement stopped event to the simulation (will be broadcast on GT during post sim)
	if (Simulation)
	{
		Simulation->AddEvent(MakeShared<FPathedMovementInReverseChangedEventData>(SimTime, bNewInReverse));

#if WITH_CHAOS_VISUAL_DEBUGGER
		if (FChaosVisualDebuggerTrace::IsTracing())
		{
			DebugData.bPlaybackReverseChangedThisFrame = true;
		}
#endif // WITH_CHAOS_VISUAL_DEBUGGER
	}
}

void UChaosPathedMovementMode::OnLoopingChanging(const FMoverTimeStep& SimTime, bool bNewLooping)
{
	// Add a pathed movement stopped event to the simulation (will be broadcast on GT during post sim)
	if (Simulation)
	{
		Simulation->AddEvent(MakeShared<FPathedMovementLoopingChangedEventData>(SimTime, bNewLooping));

#if WITH_CHAOS_VISUAL_DEBUGGER
		if (FChaosVisualDebuggerTrace::IsTracing())
		{
			DebugData.bPlaybackLoopingChangedThisFrame = true;
		}
#endif // WITH_CHAOS_VISUAL_DEBUGGER
	}
}

void UChaosPathedMovementMode::OnOneWayChanging(const FMoverTimeStep& SimTime, bool bNewOneWay)
{
	// Add a pathed movement stopped event to the simulation (will be broadcast on GT during post sim)
	if (Simulation)
	{
		Simulation->AddEvent(MakeShared<FPathedMovementOneWayChangedEventData>(SimTime, bNewOneWay));

#if WITH_CHAOS_VISUAL_DEBUGGER
		if (FChaosVisualDebuggerTrace::IsTracing())
		{
			DebugData.bPlaybackOneWayChangedThisFrame = true;
		}
#endif // WITH_CHAOS_VISUAL_DEBUGGER
	}
}

void UChaosPathedMovementMode::InitializePath()
{
	Chaos::EnsureIsInGameThreadContext();

	for (UChaosPathedMovementPatternBase* Pattern : PathPatterns)
	{
		if (Pattern)
		{
			Pattern->InitializePattern(Simulation);
		}
	}
}

UChaosPathedMovementPatternBase* UChaosPathedMovementMode::BP_FindPattern(TSubclassOf<UChaosPathedMovementPatternBase> PatternType) const
{
	for (UChaosPathedMovementPatternBase* Pattern : PathPatterns)
	{
		if (Pattern && Pattern->IsA(PatternType))
		{
			return Pattern;
		}
	}
	
	return nullptr;
}

void UChaosPathedMovementMode::SetOneWayTripDuration_BeginPlayOnly(float NewDuration)
{
	Chaos::EnsureIsInGameThreadContext();

	if (const UMoverComponent* MoverComponent = GetMoverComponent())
	{
		if (ensure(!MoverComponent->HasBegunPlay()))
		{
			OneWayPlaybackDuration = FMath::Max(NewDuration, 0.f);
		}
	}
}

FTransform UChaosPathedMovementMode::CalcTargetTransform(float OverallPathProgress, const FTransform& BasisTransform) const
{
	// The scale of basis transform does not affect the path itself
	// Rather, the scale returned by CalcTargetTransform affects the scale of whatever is following the path
	FTransform BasisTransformNoScale = BasisTransform;
	BasisTransformNoScale.SetScale3D(FVector(1,1,1));

	const float EasedPathProgress = FAlphaBlend::AlphaToBlendOption(OverallPathProgress, Easing, CustomEasingCurve);

	FTransform TargetRelativeTransform = FTransform::Identity;
	FVector AccumulatedRelativeScale = FVector(1,1,1);

	for (UChaosPathedMovementPatternBase* PathPattern : PathPatterns)
	{
		if (PathPattern)
		{
			const FTransform PatternWorldTransform = PathPattern->CalcTargetTransform(EasedPathProgress, BasisTransformNoScale);
			const FTransform PatternRelativeTransform = FTransform(
				BasisTransformNoScale.InverseTransformRotation(PatternWorldTransform.GetRotation()),
				BasisTransformNoScale.InverseTransformPositionNoScale(PatternWorldTransform.GetLocation()));
			TargetRelativeTransform.Accumulate(PatternRelativeTransform);
			AccumulatedRelativeScale *= PatternWorldTransform.GetScale3D();
		}
	}

	return FTransform(
		BasisTransformNoScale.TransformRotation(TargetRelativeTransform.GetRotation()),
		BasisTransformNoScale.TransformPositionNoScale(TargetRelativeTransform.GetLocation()),
		BasisTransform.GetScale3D() * AccumulatedRelativeScale);
}

void UChaosPathedMovementMode::SetDefaultJointConstraintProperties()
{
	JointConstraintProperties.bLinearBreakable = false;
	JointConstraintProperties.bAngularBreakable = false;
	JointConstraintProperties.bDisableCollision = true;

	// Free the linear joints and set the max allowed distance from the target
	static constexpr float DefaultMaxLinearDistance = 100.f;
	JointConstraintProperties.LinearLimit.XMotion = LCM_Free;
	JointConstraintProperties.LinearLimit.YMotion = LCM_Free;
	JointConstraintProperties.LinearLimit.ZMotion = LCM_Free;
	JointConstraintProperties.LinearLimit.Limit = DefaultMaxLinearDistance;

	static constexpr float DefaultDriveMaxForce = 5000.f;
	
	// Linear drive config
	static constexpr float LinearDriveStiffness = 750.f;
	static const float LinearDriveCriticalDamping = 2 * FMath::Sqrt(LinearDriveStiffness);
	
	FConstraintDrive DefaultLinearDriveConstraint;
	DefaultLinearDriveConstraint.bEnablePositionDrive = true;
	DefaultLinearDriveConstraint.bEnableVelocityDrive = true;
	DefaultLinearDriveConstraint.Stiffness = LinearDriveStiffness;
	DefaultLinearDriveConstraint.Damping = LinearDriveCriticalDamping;
	DefaultLinearDriveConstraint.MaxForce = DefaultDriveMaxForce;

	JointConstraintProperties.LinearDrive.XDrive = DefaultLinearDriveConstraint;
	JointConstraintProperties.LinearDrive.YDrive = DefaultLinearDriveConstraint;
	JointConstraintProperties.LinearDrive.ZDrive = DefaultLinearDriveConstraint;

	// Free the angular joints and set degree limits relative to the target
	static constexpr float DefaultAngularLimitDegrees = 15.f;
	JointConstraintProperties.ConeLimit.Swing1Motion = ACM_Free;
	JointConstraintProperties.ConeLimit.Swing1LimitDegrees = DefaultAngularLimitDegrees;
	JointConstraintProperties.ConeLimit.Swing2Motion = ACM_Free;
	JointConstraintProperties.ConeLimit.Swing2LimitDegrees = DefaultAngularLimitDegrees;
	JointConstraintProperties.TwistLimit.TwistMotion = ACM_Free;
	JointConstraintProperties.TwistLimit.TwistLimitDegrees = DefaultAngularLimitDegrees;
	
	// Angular drive config
	static constexpr float AngularDriveStiffness = 1500.f;
	static const float AngularDriveCriticalDamping = 2 * FMath::Sqrt(AngularDriveStiffness);

	JointConstraintProperties.AngularDrive.AngularDriveMode = EAngularDriveMode::SLERP;
	JointConstraintProperties.AngularDrive.SlerpDrive.bEnablePositionDrive = true;
	JointConstraintProperties.AngularDrive.SlerpDrive.bEnableVelocityDrive = true;
	JointConstraintProperties.AngularDrive.SlerpDrive.Stiffness = AngularDriveStiffness;
	JointConstraintProperties.AngularDrive.SlerpDrive.Damping = AngularDriveCriticalDamping;
	JointConstraintProperties.AngularDrive.SlerpDrive.MaxForce = DefaultDriveMaxForce;
}

#if WITH_EDITOR
template <typename TComponentType>
TComponentType* GetOwnerActorsRootComponent(const UActorComponent* ActorComponent)
{
	if (!ActorComponent)
	{
		return nullptr;
	}

	const AActor* OuterActor = ActorComponent ? ActorComponent->GetTypedOuter<AActor>() : nullptr;
	if (OuterActor)
	{
		return Cast<TComponentType>(OuterActor->GetRootComponent());
	}
	else if (UBlueprintGeneratedClass* OuterBlueprintGeneratedClass = ActorComponent->GetTypedOuter<UBlueprintGeneratedClass>())
	{
		TArray<const UBlueprintGeneratedClass*> BlueprintClasses;
		UBlueprintGeneratedClass::GetGeneratedClassesHierarchy(OuterBlueprintGeneratedClass, BlueprintClasses);
		for (const UBlueprintGeneratedClass* BPGClass : BlueprintClasses)
		{
			if (BPGClass && BPGClass->SimpleConstructionScript)
			{
				const TArray<USCS_Node*>& ActorBlueprintNodes = BPGClass->SimpleConstructionScript->GetAllNodes();
				for (USCS_Node* Node : ActorBlueprintNodes)
				{
					if (Node->IsRootNode() && Node->ComponentClass && Node->ComponentClass->IsChildOf(USceneComponent::StaticClass()))
					{
						if (TComponentType* CastRootComponent = Cast<TComponentType>(Node->GetActualComponentTemplate(OuterBlueprintGeneratedClass)))
						{
							return CastRootComponent;
						}
					}
				}
			}
		}
	}

	return nullptr;
}

EDataValidationResult UChaosPathedMovementMode::IsDataValid(FDataValidationContext& Context) const
{
	EDataValidationResult Result = Super::IsDataValid(Context);

	// Check for correct physics setup of the actor this mode will be moving:
	//    - its root needs to be a primitive component
	//        * (this check should be done in UMoverComponent::IsDataValid also, 
	//           or we should relax this constraint and allow to pick a non root primitive
	//           component as the updated component)
	//    - it needs to be simulating physics
	//    - if kinematically moved, it needs to have bUpdateKinematicFromSimulation set to true
	//
	// Find the CDO of the MoverComponent's Actor.
	// There might not be one if this path mode is on a free blueprinted mover component asset
	// that's not on an actor. In that case there's nothing to check.
	const UMoverComponent* MoverComponent = GetMoverComponent();
	if (!MoverComponent)
	{
		return Result;
	}

	// Find the root scene component
	if (const USceneComponent* RootSceneComponent = GetOwnerActorsRootComponent<USceneComponent>(MoverComponent))
	{
		// Check that the root scene component is a primitive component
		const UPrimitiveComponent* RootPrimitiveComponent = Cast<UPrimitiveComponent>(RootSceneComponent);

		if (!RootPrimitiveComponent)
		{
			Context.AddError(FText::Format(LOCTEXT("RootNotPrimitiveComponent", "The root component {0} of an actor using a {1} must be a primitive component"), FText::FromString(GetNameSafe(RootSceneComponent)), FText::FromString(GetClass()->GetName())));
		}
		else if (!RootPrimitiveComponent->BodyInstance.bSimulatePhysics)
		{
			Context.AddWarning(FText::Format(LOCTEXT("SuspiciousSimulatingPhysics", "The root component {0} of an actor using a {1} is not simulating physics, its mover component may not work correctly"), FText::FromString(GetNameSafe(RootSceneComponent)), FText::FromString(GetClass()->GetName())));
		}
		else if (!bUseJointConstraint && !RootPrimitiveComponent->BodyInstance.bUpdateKinematicFromSimulation)
		{
			Context.AddError(FText::Format(LOCTEXT("InvalidUpdateKinematicFromSimulation", "The root component {0} of an actor using a {1} will be moved kinematically, it needs to have Update Kinematic From Simulation checked in its Physics properties"), FText::FromString(GetNameSafe(RootSceneComponent)), FText::FromString(GetClass()->GetName())));
		}
	}

	return Result;
}
#endif

// ---------------------------------------------------------------------------
// IChaosPostSimulationTickInterface
// ---------------------------------------------------------------------------

void UChaosPathedMovementMode::PostSimulationTick_Async(FChaosMoverPostSimContext& Context)
{
	QUICK_SCOPE_CYCLE_COUNTER(ChaosPathedMovementMode_PostSimulationTick);

	if (!Simulation)
	{
		return;
	}

	Chaos::FPBDJointConstraintHandle* ActuationConstraintHandle = Simulation->GetActuationConstraintHandle();
	if (!ActuationConstraintHandle)
	{
		return;
	}

	if (ShouldUseConstraint())
	{
		Simulation->EnableActuationConstraint();

		if (!Simulation->IsControlledParticleDynamic())
		{
			Simulation->SetControlledParticleDynamic();
		}

		ActuationConstraintHandle->SetSettings(GetConstraintSettings());
	}
	else
	{
		if (!Simulation->IsControlledParticleKinematic())
		{
			Simulation->SetControlledParticleKinematic();
		}
	}

	const FMoverDefaultSyncState* PostSimDefaultSyncState =
		Context.OutputData.SyncState.SyncStateCollection.FindDataByType<FMoverDefaultSyncState>();
	check(PostSimDefaultSyncState);
	Simulation->SetActuationTargetTransform(PostSimDefaultSyncState->GetTransform_WorldSpace());
}

#undef LOCTEXT_NAMESPACE
