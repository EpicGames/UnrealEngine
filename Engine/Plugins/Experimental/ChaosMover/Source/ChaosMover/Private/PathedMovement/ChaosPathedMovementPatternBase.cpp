// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosMover/PathedMovement/ChaosPathedMovementPatternBase.h"

#include "AlphaBlend.h"
#include "HAL/IConsoleManager.h"
#include "ChaosMover/PathedMovement/ChaosPathedMovementMode.h"
#include "ChaosMover/ChaosMoverSimulation.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosPathedMovementPatternBase)

void UChaosPathedMovementPatternBase::InitializePattern(UChaosMoverSimulation* InSimulation)
{
	Simulation = InSimulation;
}

template <typename T>
T ApplyAxisMask(const T& Unmasked, EChaosPatternAxisMaskFlags Flags, float MaskedValue = 0.f)
{
	T MaskedResult = Unmasked;
	if (EnumHasAnyFlags(Flags, EChaosPatternAxisMaskFlags::X))
	{
		MaskedResult.X = MaskedValue;
	}
	if (EnumHasAnyFlags(Flags, EChaosPatternAxisMaskFlags::Y))
	{
		MaskedResult.Y = MaskedValue;
	}
	if (EnumHasAnyFlags(Flags, EChaosPatternAxisMaskFlags::Z))
	{
		MaskedResult.Z = MaskedValue;
	}
	return MaskedResult;
}

template <>
FRotator ApplyAxisMask(const FRotator& Unmasked, EChaosPatternAxisMaskFlags Flags, float MaskedValue)
{
	FRotator MaskedResult = Unmasked;
	if (EnumHasAnyFlags(Flags, EChaosPatternAxisMaskFlags::X))
	{
		MaskedResult.Roll = MaskedValue;
	}
	if (EnumHasAnyFlags(Flags, EChaosPatternAxisMaskFlags::Y))
	{
		MaskedResult.Pitch = MaskedValue;
	}
	if (EnumHasAnyFlags(Flags, EChaosPatternAxisMaskFlags::Z))
	{
		MaskedResult.Yaw = MaskedValue;
	}
	return MaskedResult;
}

float UChaosPathedMovementPatternBase::ConvertPathToPatternProgress(float OverallPathProgress) const
{
	// If you update this function, please also update ConvertPatternToPathProgress to support debug draws
	if (NumLoopsPerPath <= 0 || StartAtPathProgress >= EndAtPathProgress || OverallPathProgress < StartAtPathProgress)
	{
		return -1.0f;
	}

	// How far into the current loop of this specific pattern are we?
	const float PathProgressSinceStart = FMath::Min(EndAtPathProgress, OverallPathProgress) - StartAtPathProgress;
	const float PathProgressPerPatternLoop = (EndAtPathProgress - StartAtPathProgress) / NumLoopsPerPath;
	float CurLoopPathProgress = FMath::Fmod(PathProgressSinceStart, PathProgressPerPatternLoop);
	if (FMath::IsNearlyZero(CurLoopPathProgress) && PathProgressSinceStart > UE_SMALL_NUMBER)
	{
		// Treat path progress that matches the per-loop span exactly as 100%, not 0%
		CurLoopPathProgress = PathProgressPerPatternLoop;
	}

	// Normalized progress within this loop, in [0, 1]
	const float t = CurLoopPathProgress / PathProgressPerPatternLoop;

	if (bBounceAtEnds)
	{
		// Bounce at 0 and 1 instead of wrapping - safe for non-looping paths
		if (!IsOneWay)
		{
			// Piecewise triangle spanning the full [0, 1] range. No seam crossing.
			// Ping (first half):  rises  LoopOffset -> 1.0
			// Middle transition:  falls  1.0 -> 0.0  (passes through LoopOffset on the way)
			// Pong (second half): rises  0.0 -> LoopOffset
			// LoopOffset=0: standard 0->1->0. LoopOffset=0.5: 0.5->1.0->0.0->0.5.
			const float Leg1End = (1.f - LoopOffset) * 0.5f;
			const float Leg2End = (2.f - LoopOffset) * 0.5f;
			if (t <= Leg1End)
			{
				return LoopOffset + 2.f * t;
			}
			else if (t <= Leg2End)
			{
				return 2.f - LoopOffset - 2.f * t;
			}
			else
			{
				return 2.f * t - 2.f + LoopOffset;
			}
		}
		else  // IsOneWay
		{
			// Triangle wave bouncing between LoopOffset and 1.0. Never goes below LoopOffset, never crosses seam.
			// LoopOffset=0: 0->1->0. LoopOffset=0.5: 0.5->1.0->0.5.
			if (t <= 0.5f)
			{
				return LoopOffset + 2.f * t * (1.f - LoopOffset);
			}
			else
			{
				return LoopOffset + 2.f * (1.f - t) * (1.f - LoopOffset);
			}
		}
	}

	// Seam-crossing (wraps at 0/1): if each loop is there and back, flip progress after completing half the span
	if (!IsOneWay)
	{
		// ThereAndBack means we actually progress twice as fast as a OneShot
		CurLoopPathProgress *= 2.f;
		const float ReversePathProgress = CurLoopPathProgress - PathProgressPerPatternLoop;
		if (ReversePathProgress > 0.f)
		{
			CurLoopPathProgress = PathProgressPerPatternLoop - ReversePathProgress;
		}
	}

	// Now convert from overall path progress to pattern progress
	const float PatternProgress = CurLoopPathProgress / PathProgressPerPatternLoop;
	const float OffsettedPatternProgress = PatternProgress + LoopOffset;
	float FinalPatternProgress = FMath::Fmod(OffsettedPatternProgress, 1.0f);
	if (FMath::IsNearlyZero(FinalPatternProgress) && OffsettedPatternProgress > UE_SMALL_NUMBER)
	{
		// Treat progress that matches one loop as 100%, not 0% ("modulo-inclusive Fmod") otherwise this Fmod could undo the fix above
		FinalPatternProgress = 1.0f;
	}

	return FinalPatternProgress;
}

float UChaosPathedMovementPatternBase::ConvertPatternToPathProgress(float FinalPatternProgress) const
{
	if (NumLoopsPerPath <= 0 || StartAtPathProgress >= EndAtPathProgress)
	{
		return -1.0f;
	}

	const float PathProgressPerPatternLoop = (EndAtPathProgress - StartAtPathProgress) / NumLoopsPerPath;

	if (bBounceAtEnds)
	{
		// Compute normalized loop progress t in [0, 1] for the first occurrence of FinalPatternProgress.
		float t;
		if (!IsOneWay)
		{
			// Inverse of leg 1: P = LoopOffset + 2t  =>  t = (P - LoopOffset) / 2
			t = FMath::Clamp((FMath::Clamp(FinalPatternProgress, LoopOffset, 1.0f) - LoopOffset) * 0.5f, 0.f, (1.f - LoopOffset) * 0.5f);
		}
		else  // IsOneWay - inverse of P = LoopOffset + 2t*(1-LoopOffset) for t in [0, 0.5]
		{
			const float Denom = 2.f * (1.f - LoopOffset);
			t = FMath::IsNearlyZero(Denom) ? 0.f
				: FMath::Clamp((FMath::Clamp(FinalPatternProgress, LoopOffset, 1.0f) - LoopOffset) / Denom, 0.f, 0.5f);
		}
		return FMath::Clamp(StartAtPathProgress + t * PathProgressPerPatternLoop, 0.0f, 1.0f);
	}

	// Seam-crossing: reverse the LoopOffset wrap, then reverse the IsOneWay flip
	const float PatternProgressOffsetRemoved = FMath::Clamp(FinalPatternProgress, 0.0f, 1.0f) - LoopOffset;
	float PatternProgress = FMath::Fmod(PatternProgressOffsetRemoved, 1.0f);
	if (PatternProgress < 0.f) { PatternProgress += 1.f; }
	if (FMath::IsNearlyZero(PatternProgress) && PatternProgressOffsetRemoved > UE_SMALL_NUMBER)
	{
		// Treat PatternProgress = 1.0f as 1.0f
		PatternProgress = 1.0f;
	}

	float CurLoopPathProgress = PatternProgress * PathProgressPerPatternLoop;
	if (!IsOneWay)
	{
		CurLoopPathProgress /= 2.f;
	}

	return FMath::Clamp(StartAtPathProgress + CurLoopPathProgress, 0.0f, 1.0f);
}

FTransform UChaosPathedMovementPatternBase::CalcTargetTransform(float OverallPathProgress, const FTransform& BasisTransform) const
{
	float PatternProgress = ConvertPathToPatternProgress(OverallPathProgress);
	if (PatternProgress >= 0.0f)
	{
		PatternProgress = FAlphaBlend::AlphaToBlendOption(PatternProgress, Easing, CustomEasingCurve);
	}
	return CalcMaskedTargetTransform(PatternProgress, BasisTransform);
}

FTransform UChaosPathedMovementPatternBase::CalcMaskedTargetTransform(float PatternProgress, const FTransform& BasisTransform) const
{
	if (PatternProgress < 0.0f)
	{
		return BasisTransform;
	}

	const FTransform UnmaskedTarget = CalcUnmaskedTargetTransform(PatternProgress, BasisTransform);

	if (bOrientComponentToPath)
	{
		//@todo DanH: Rotate so that the forward vector matches the translation vector direction
	}

	const FVector TargetLocation = ApplyAxisMask(UnmaskedTarget.GetLocation(), (EChaosPatternAxisMaskFlags)TranslationMasks);
	const FRotator TargetRotation = ApplyAxisMask(UnmaskedTarget.GetRotation().Rotator(), (EChaosPatternAxisMaskFlags)RotationMasks);
	const FVector TargetScale = ApplyAxisMask(UnmaskedTarget.GetScale3D(), (EChaosPatternAxisMaskFlags)ScaleMasks, 1.f);
	return FTransform(TargetRotation, TargetLocation, TargetScale);
}

UChaosPathedMovementMode& UChaosPathedMovementPatternBase::GetMovementMode() const
{
	UChaosPathedMovementMode* OuterMode = GetOuterUChaosPathedMovementMode();
	check(OuterMode);
	return *OuterMode;
}
