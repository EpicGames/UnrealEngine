// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosMover/PathedMovement/ChaosPathedMovementTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosPathedMovementTypes)

void FChaosMutablePathedMovementProperties::NetSerialize(FArchive& Ar)
{
	Ar.SerializeBits(&bWantsToPlay, 1);
	Ar.SerializeBits(&bWantsReversePlayback, 1);
	Ar.SerializeBits(&bWantsLoopingPlayback, 1);
	Ar.SerializeBits(&bWantsOneWayPlayback, 1);
}

void FChaosMutablePathedMovementProperties::ToString(FAnsiStringBuilderBase& Out) const
{
	Out.Appendf("Wants to Play Path: %hs | ", *LexToString(bWantsToPlay));
	Out.Appendf("Wants Reverse Playback: %hs | ", *LexToString(bWantsReversePlayback));
	Out.Appendf("Wants Looping Playback: %hs | ", *LexToString(bWantsLoopingPlayback));
	Out.Appendf("Wants One Way Playback: %hs\n", *LexToString(bWantsOneWayPlayback));
}

bool FChaosMutablePathedMovementProperties::operator==(const FChaosMutablePathedMovementProperties& Other) const
{
	return bWantsToPlay == Other.bWantsToPlay
		&& bWantsReversePlayback == Other.bWantsReversePlayback
		&& bWantsLoopingPlayback == Other.bWantsLoopingPlayback
		&& bWantsOneWayPlayback == Other.bWantsOneWayPlayback
		;
}

bool FChaosPathedMovementInputs::NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess)
{
	Super::NetSerialize(Ar, Map, bOutSuccess);

	Ar << MovementStartFrame;
	Ar << LastChangeFrame;
	Props.NetSerialize(Ar);
	
	bOutSuccess = true;
	return true;
}

void FChaosPathedMovementInputs::ToString(FAnsiStringBuilderBase& Out) const
{
	Props.ToString(Out);
	Out.Appendf("Movement Start Frame: %d\n", MovementStartFrame);
	Out.Appendf("Last Change Frame: %d\n", LastChangeFrame);
}

bool FChaosPathedMovementInputs::ShouldReconcile(const FMoverDataStructBase& AuthorityState) const
{
	const FChaosPathedMovementInputs& TypedAuthority = static_cast<const FChaosPathedMovementInputs&>(AuthorityState);
	return Props != TypedAuthority.Props;
}

void FChaosPathedMovementInputs::Interpolate(const FMoverDataStructBase& From, const FMoverDataStructBase& To, float Pct)
{
	if (Pct < .5f)
	{
		*this = static_cast<const FChaosPathedMovementInputs&>(From);
	}
	else
	{
		*this = static_cast<const FChaosPathedMovementInputs&>(To);
	}
}

void FChaosPathedMovementInputs::Merge(const FMoverDataStructBase& From)
{
	const FChaosPathedMovementInputs& TypedFrom = static_cast<const FChaosPathedMovementInputs&>(From);
	//@todo DanH: What, if anything, do I actually want to merge on the mutable props?
}

bool FChaosPathedMovementState::NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess)
{
	Super::NetSerialize(Ar, Map, bOutSuccess);

	static FChaosPathedMovementState Defaults;

	bool HasLastChangePathProgress;
	bool HasLatestMovementStartFrame;
	bool HasLastChangeTime;

	if (Ar.IsSaving())
	{
		HasLastChangePathProgress = LastChangePathProgress != Defaults.LastChangePathProgress;
		HasLatestMovementStartFrame = LatestMovementStartFrame != Defaults.LatestMovementStartFrame;
		// In variable-dt (non-async) mode, MakeChangeTime sets FrameCount = INDEX_NONE, which matches the default,
		// so HasLastChangeTime is always false and nothing is sent. This is intentional: variable-dt implies
		// standalone-only (no networking), so there is nothing to serialize. See note on FChaosPathedMovementInputs.
		HasLastChangeTime = LastChangeTime.FrameCount != Defaults.LastChangeTime.FrameCount;
	}
	else
	{
		// Set to defaults
		*this = Defaults;
	}

	Ar.SerializeBits(&HasLastChangePathProgress, 1);
	Ar.SerializeBits(&HasLatestMovementStartFrame, 1);
	Ar.SerializeBits(&HasLastChangeTime, 1);
	Ar.SerializeBits(&bIsPathProgressionIncreasing, 1);
	Ar.SerializeBits(&bHasFinished, 1);
	PropertiesInEffect.NetSerialize(Ar);

	if (HasLastChangePathProgress)
	{
		Ar << LastChangePathProgress;
	}
	if (HasLatestMovementStartFrame)
	{
		Ar << LatestMovementStartFrame;
	}
	if (HasLastChangeTime)
	{
		Ar << LastChangeTime.FrameCount;
	}

	return true;
}

void FChaosPathedMovementState::ToString(FAnsiStringBuilderBase& Out) const
{
	FMoverDataStructBase::ToString(Out);

	Out.Appendf("Last Change Path Progress: %.5f | ", LastChangePathProgress);
	Out.Appendf("Last Change Time: %s | ", *LastChangeTime.ToString());
	Out.Appendf("Latest Movement Start Frame: %d | ", LatestMovementStartFrame);
	Out.Appendf("Is Path Progress Increasing: %s | ", bIsPathProgressionIncreasing ? TEXT("True") : TEXT("False"));
	Out.Appendf("Has Finished: %s | ", bHasFinished ? TEXT("True") : TEXT("False"));
	PropertiesInEffect.ToString(Out);
}

bool FChaosPathedMovementState::ShouldReconcile(const FMoverDataStructBase& AuthorityState) const
{
	const FChaosPathedMovementState& TypedAuthority = static_cast<const FChaosPathedMovementState&>(AuthorityState);

	static constexpr float PathProgressTolerance = 1e-4f;

	if (PropertiesInEffect != TypedAuthority.PropertiesInEffect
		|| LatestMovementStartFrame != TypedAuthority.LatestMovementStartFrame
		|| LastChangeTime.FrameCount != TypedAuthority.LastChangeTime.FrameCount
		|| bIsPathProgressionIncreasing != TypedAuthority.bIsPathProgressionIncreasing
		|| bHasFinished != TypedAuthority.bHasFinished
		|| !FMath::IsNearlyEqual(LastChangePathProgress, TypedAuthority.LastChangePathProgress, PathProgressTolerance))
	{
		return true;
	}

	return false;
}

void FChaosPathedMovementState::Interpolate(const FMoverDataStructBase& From, const FMoverDataStructBase& To, float Pct)
{
	const FChaosPathedMovementState& TypedFrom = static_cast<const FChaosPathedMovementState&>(From);
	const FChaosPathedMovementState& TypedTo = static_cast<const FChaosPathedMovementState&>(To);

	PropertiesInEffect = (Pct < .5f) ? TypedFrom.PropertiesInEffect : TypedTo.PropertiesInEffect;
	LatestMovementStartFrame = (Pct < .5f) ? TypedFrom.LatestMovementStartFrame : TypedTo.LatestMovementStartFrame;
	LastChangeTime = (Pct < .5f) ? TypedFrom.LastChangeTime : TypedTo.LastChangeTime;
	LastChangePathProgress = FMath::Lerp(TypedFrom.LastChangePathProgress, TypedTo.LastChangePathProgress, Pct);
	bIsPathProgressionIncreasing = (Pct < .5f) ? TypedFrom.bIsPathProgressionIncreasing : TypedTo.bIsPathProgressionIncreasing;
	bHasFinished = (Pct < .5f) ? TypedFrom.bHasFinished : TypedTo.bHasFinished;
}

FMoverDataStructBase* FChaosPathedMovementModeDebugData::Clone() const
{
	FChaosPathedMovementModeDebugData* CopyPtr = new FChaosPathedMovementModeDebugData(*this);
	return CopyPtr;
}

UScriptStruct* FChaosPathedMovementModeDebugData::GetScriptStruct() const
{
	return StaticStruct();
}
