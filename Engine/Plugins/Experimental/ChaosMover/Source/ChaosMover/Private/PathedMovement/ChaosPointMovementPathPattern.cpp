// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosMover/PathedMovement/ChaosPointMovementPathPattern.h"

#include "DebugRenderSceneProxy.h"
#include "Kismet/KismetMathLibrary.h"
#include "ChaosMover/ChaosMoverSimulation.h"
#include "Framework/Threading.h"
#include "ChaosMover/PathedMovement/ChaosPathedMovementMode.h"
#include "ChaosMover/PathedMovement/ChaosPathedMovementDebugDrawComponent.h"
#include "MoverComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosPointMovementPathPattern)

void UChaosPointMovementPathPattern::InitializePattern(UChaosMoverSimulation* InSimulation)
{
	Super::InitializePattern(InSimulation);

	RefreshAssignedPointProgress(GetPathBasisTransform());
}

#if WITH_EDITOR
void UChaosPointMovementPathPattern::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	RefreshAssignedPointProgress(GetPathBasisTransform());
}
#endif

FTransform UChaosPointMovementPathPattern::GetPathBasisTransform() const
{
	FTransform BasisTransform = FTransform::Identity;
	if (GetSimulation())
	{
		BasisTransform = GetSimulation()->GetMovementBasisTransform();
	}
	else if (UChaosPathedMovementMode* ChaosPathedMovementMode = Cast<UChaosPathedMovementMode>(GetOuter()))
	{
		if (UActorComponent* ActorComponent = ChaosPathedMovementMode->GetMoverComponent())
		{
			if (ActorComponent->GetOwner())
			{
				BasisTransform = ActorComponent->GetOwner()->GetTransform();
			}
		}
	}
	return BasisTransform;
}

void UChaosPointMovementPathPattern::AppendDebugDrawElements(UChaosPathedMovementDebugDrawComponent& DebugDrawComp, FBoxSphereBounds::Builder& InOutDebugBoundsBuilder)
{
	if (!PathPoints.IsEmpty())
	{
		const FTransform& PathBasisTransform = DebugDrawComp.GetOwner()->GetTransform();

		FVector PreviousPathPointLocation = PathPoints[0].GetWorldTransform(PathBasisTransform).GetLocation();
		DebugDrawComp.DebugSpheres.Emplace(4.f, PreviousPathPointLocation, PatternDebugDrawColor);
		InOutDebugBoundsBuilder += PreviousPathPointLocation;
		
		for (int PointIndex = 1; PointIndex < PathPoints.Num(); ++PointIndex)
		{
			const FChaosPointMovementPathPoint& PathPoint = PathPoints[PointIndex];
			FVector PathPointLocation = PathPoint.GetWorldTransform(PathBasisTransform).GetLocation();
			DebugDrawComp.DebugSpheres.Emplace(4.f, PathPointLocation, PatternDebugDrawColor);

			DebugDrawComp.DebugLines.Emplace(PreviousPathPointLocation, PathPointLocation, PatternDebugDrawColor, 1.f);
			PreviousPathPointLocation = PathPointLocation;

			InOutDebugBoundsBuilder += PathPointLocation;
		}
	}
}

FTransform FChaosPointMovementPathPoint::GetWorldTransform(const FTransform& BasisTransform) const
{
	return FTransform(
		(EffectiveRotationBasis == EChaosPointMovementLocationBasis::World) ? FRotator(BasedRotation) : FRotator(BasisTransform.TransformRotation(FQuat(BasedRotation))),
		(EffectiveBasis == EChaosPointMovementLocationBasis::World) ? BasedLocation : BasisTransform.TransformPositionNoScale(BasedLocation),
		BasisTransform.GetScale3D()
	);
}

FTransform UChaosPointMovementPathPattern::CalcUnmaskedTargetTransform(float PatternProgress, const FTransform& BasisTransform) const
{

	if (!PathPoints.IsEmpty())
	{
		if (PatternProgress < SMALL_NUMBER)
		{
			return PathPoints[0].GetWorldTransform(BasisTransform);
		}
		else
		{
			// The destination point is the first point at or equal to the target progress
			int32 DestPointIdx = 0;
			for (; DestPointIdx < PathPoints.Num() && PathPoints[DestPointIdx].Progress < PatternProgress; ++DestPointIdx) {}

			if (PathPoints.IsValidIndex(DestPointIdx))
			{
				const FChaosPointMovementPathPoint& NextPoint = PathPoints[DestPointIdx];
				FChaosPointMovementPathPoint PrevPoint;
				if (DestPointIdx > 0)
				{
					PrevPoint = PathPoints[DestPointIdx - 1];
				}
				else
				{
					PrevPoint.BasedLocation = BasisTransform.GetLocation();
					PrevPoint.BasedRotation = BasisTransform.GetRotation();
					PrevPoint.EffectiveBasis = EChaosPointMovementLocationBasis::World;
					PrevPoint.EffectiveRotationBasis = EChaosPointMovementLocationBasis::World;
				}

				const float ProgressSinceLastPoint = PatternProgress - PrevPoint.Progress;
				const float Alpha = (PatternProgress == PrevPoint.Progress) ? 0.0f : ProgressSinceLastPoint / (NextPoint.Progress - PrevPoint.Progress);
				const FTransform PreviousWorldTransform = PrevPoint.GetWorldTransform(BasisTransform);
				const FTransform NextWorldTransform = NextPoint.GetWorldTransform(BasisTransform);
				const FVector TargetWorldLocation = UKismetMathLibrary::VLerp(PreviousWorldTransform.GetLocation(), NextWorldTransform.GetLocation(), Alpha);
				const FQuat TargetWorldRotation = UKismetMathLibrary::Quat_Slerp(PreviousWorldTransform.GetRotation(), NextWorldTransform.GetRotation(), Alpha);

				return FTransform(TargetWorldRotation, TargetWorldLocation, BasisTransform.GetScale3D());
			}
			else
			{
				return PathPoints[PathPoints.Num() - 1].GetWorldTransform(BasisTransform);
			}
		}
	}

	return BasisTransform;
}

void UChaosPointMovementPathPattern::RefreshAssignedPointProgress(const FTransform& PathBasisTransform) const
{
	Chaos::EnsureIsInGameThreadContext();

	if (!bHasAssignedPointProgress || !CachedPathBasisTransform.Equals(PathBasisTransform, SMALL_NUMBER))
	{
		if (PathPoints.Num() > 0)
		{
			const FChaosPointMovementPathPoint& FirstPoint = PathPoints[0];
			// For point 0, being relative to the previous point means being relative to the path basis
			EChaosPointMovementLocationBasis LocationBasis = (FirstPoint.Basis != EChaosPointMovementLocationBasis::PreviousPoint) ? PathPoints[0].Basis : EChaosPointMovementLocationBasis::PathOrigin;
			EChaosPointMovementLocationBasis RotationBasis = (FirstPoint.RotationBasis != EChaosPointMovementLocationBasis::PreviousPoint) ? PathPoints[0].RotationBasis : EChaosPointMovementLocationBasis::PathOrigin;
			FirstPoint.EffectiveBasis = LocationBasis;
			FirstPoint.EffectiveRotationBasis = RotationBasis;
			FirstPoint.BasedLocation = FirstPoint.Location;
			FirstPoint.BasedRotation = FQuat(FirstPoint.Rotation);
			FirstPoint.PathDistanceFromStart = 0.0f;

			FTransform PreviousWorldPointTransform = FirstPoint.GetWorldTransform(PathBasisTransform);

			TotalPathDistance = 0.f;
			int NumCollapsedIntervals = 0;

			// Run through and establish point locations, rotations and distance
			for (int PointIndex = 1; PointIndex < PathPoints.Num(); ++PointIndex)
			{
				const FChaosPointMovementPathPoint& PathPoint = PathPoints[PointIndex];
				switch (PathPoint.Basis)
				{
				case EChaosPointMovementLocationBasis::PreviousPoint:
					// This point's transform is relative to the previous point's transform, whether that transform was relative to world or the path basis
					// since this point will retain the previous point's effective basis
					PathPoint.BasedLocation = PreviousWorldPointTransform.TransformPositionNoScale(PathPoint.Location);
					if (LocationBasis == EChaosPointMovementLocationBasis::PathOrigin)
					{
						// If the effective location basis is path origin (remember that it cannot be PreviousPoint),
						// we need to transform this world position back into a position relative to the path basis
						PathPoint.BasedLocation = PathBasisTransform.InverseTransformPositionNoScale(PathPoint.BasedLocation);
					}
					
					PathPoint.EffectiveBasis = LocationBasis;
					// We avoid updating LocationBasis and we will not update it until we find a Basis that is not PreviousPoint
					break;
				default:
					PathPoint.EffectiveBasis = PathPoint.Basis;
					PathPoint.BasedLocation = PathPoint.Location;
					// We update LocationBasis, since it is not PreviousPoint
					LocationBasis = PathPoint.Basis;
					break;
				}

				switch (PathPoint.RotationBasis)
				{
				case EChaosPointMovementLocationBasis::PreviousPoint:
					PathPoint.BasedRotation = PreviousWorldPointTransform.GetRotation() * FQuat(PathPoint.Rotation);
					if (RotationBasis == EChaosPointMovementLocationBasis::PathOrigin)
					{
						// If the effective rotation basis is path origin (remember that it cannot be PreviousPoint),
						// we need to transform this world rotation back into a rotation relative to the path basis
						PathPoint.BasedRotation = PathBasisTransform.InverseTransformRotation(PathPoint.BasedRotation);
					}
					PathPoint.EffectiveRotationBasis = RotationBasis;
					// We avoid updating RotationBasis and we will not update it until we find a RotationBasis that is not PreviousPoint
					break;
				default:
					PathPoint.EffectiveRotationBasis = PathPoint.RotationBasis;
					PathPoint.BasedRotation = FQuat(PathPoint.Rotation);
					// We update RotationBasis, since it is not PreviousPoint
					RotationBasis = PathPoint.RotationBasis;
					break;
				}

				const FTransform CurrentWorldPointTransform = PathPoint.GetWorldTransform(PathBasisTransform);


				const float DistanceFromPrevious = (CurrentWorldPointTransform.GetLocation() - PreviousWorldPointTransform.GetLocation()).Size();
				if (DistanceFromPrevious < SMALL_NUMBER)
				{
					NumCollapsedIntervals++;
				}
				TotalPathDistance += DistanceFromPrevious;

				PathPoint.PathDistanceFromStart = TotalPathDistance;

				PreviousWorldPointTransform = CurrentWorldPointTransform;
			}

			// Go through again and assign progress based on distance or number of points traveled if collapsed
			// Since C intervals of the N total intervals are collapsed, we assign C/N amount of total progress on collapsed intervals and 1-C/N on non collapsed intervals
			// Progress on each collapsed interval is 1/N ( = C/N / C)
			// Progress on non collapsed interval is the ratio of how long that interval is (d_i) and the total length along the path (D), normalized by the total progress on non collapsed intervals
			// If P(i) is the progress at the end of interval index i (i in [1,N]),
			// then the progress at the end of a collapsed interval is:
			//     P(i) = P(i-1) + 1/N
			// and the progress at the end of a non collapsed interval is:
			//     P(i) = P(i-1) + d(i)/D * (1-C/N)
			// The sum is given by:
			// Sum(P(i)) = C * 1/N + Sum(d(i)) / D * (1-C/N)
			// Since D = Sum(d(i)):
			// Sum(P(i)) = C * 1/N + D/D*(1-C/N) = C/N + 1*C/N = 1
			PathPoints[0].Progress = 0.0f;
			int NumIntervals = PathPoints.Num() - 1;
			if (NumIntervals > 0)
			{
				float PreviousProgress = 0.0f;
				float PreviousPathDistance = 0.0f;
				float TotalProgressOnCollapsedIntervals = FMath::Clamp(float(NumCollapsedIntervals) / float(NumIntervals), 0.0f, 1.0f);
				float TotalProgressOnSeparatedIntervals = FMath::Clamp(1.0f - TotalProgressOnCollapsedIntervals, 0.0f, 1.0f);
				float ProgressOnOneCollapsedInterval = (TotalProgressOnCollapsedIntervals > SMALL_NUMBER) ? (1.0f / float(NumIntervals)) : 0.0f;
				for (int PointIndex = 1; PointIndex < PathPoints.Num(); ++PointIndex)
				{
					const FChaosPointMovementPathPoint& PathPoint = PathPoints[PointIndex];
					float DistanceOnPathSincePrevious = PathPoint.PathDistanceFromStart - PreviousPathDistance;
					if (DistanceOnPathSincePrevious < SMALL_NUMBER)
					{
						// This is the end point of a collapsed interval, we increase progress by ProgressOnOneCollapsedInterval
						PathPoint.Progress = PreviousProgress + ProgressOnOneCollapsedInterval;
					}
					else
					{
						// This is the end point of a non collapsed interval, we increase progress by TotalProgressOnSeparatedIntervals * d(i)/D
						PathPoint.Progress = PreviousProgress + UKismetMathLibrary::SafeDivide(DistanceOnPathSincePrevious, TotalPathDistance) * TotalProgressOnSeparatedIntervals;
					}

					PathPoint.Progress = FMath::Clamp(PathPoint.Progress, 0.0f, 1.0f);
					PreviousProgress = PathPoint.Progress;
					PreviousPathDistance = PathPoint.PathDistanceFromStart;
				}
				// The last point needs to be at exactly 1.0 progress or some assumptions don't hold
				PathPoints[PathPoints.Num()-1].Progress = 1.0f;
			}

			bHasAssignedPointProgress = true;
		}
	}

	CachedPathBasisTransform = PathBasisTransform;
}
