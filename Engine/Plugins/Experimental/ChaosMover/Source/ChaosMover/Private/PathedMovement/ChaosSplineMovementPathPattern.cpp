// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosMover/PathedMovement/ChaosSplineMovementPathPattern.h"
#include "MoverComponent.h"
#include "ChaosMover/PathedMovement/ChaosPathedMovementMode.h"
#include "Framework/Threading.h"

#include "Components/SplineComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosSplineMovementPathPattern)

void UChaosSplineMovementPathPattern::InitializePattern(UChaosMoverSimulation* InSimulation)
{
	Super::InitializePattern(InSimulation);

	Chaos::EnsureIsInGameThreadContext();

	if (!SplineComp)
	{
		UChaosPathedMovementMode& MovementMode = GetMovementMode();
		if (UMoverComponent* MoverComponent = MovementMode.GetMoverComponent())
		{
			if (AActor* OwningActor = MoverComponent->GetOwner(); ensure(OwningActor))
			{
				SplineComp = Cast<USplineComponent>(SplineComponentRef.GetComponent(OwningActor));
				if (!SplineComp)
				{
					SplineComp = OwningActor->FindComponentByClass<USplineComponent>();
				}
			}
		}
	}
}

FTransform UChaosSplineMovementPathPattern::CalcUnmaskedTargetTransform(float PatternProgress, const FTransform& BasisTransform) const
{
	if (SplineComp)
	{
		const float SplineLength = SplineComp->GetSplineLength();

		const bool bHasValidBounds = HasValidBounds();
		const float SplineStartDist = bHasValidBounds ? SplineLength * LowerBound : 0.f;
		const float SplineEndDist = bHasValidBounds ? SplineLength * UpperBound : SplineLength;

		const float DistanceAlongSpline = FMath::Lerp(SplineStartDist, SplineEndDist, PatternProgress);
		FTransform TargetToSpline = SplineComp->GetTransformAtDistanceAlongSpline(DistanceAlongSpline, ESplineCoordinateSpace::Local, bApplySplineScaling);

		if (!bOrientComponentToPath)
		{
			TargetToSpline.SetRotation(FQuat::Identity);
		}
		
		// Intentionally not accounting for any offset between the path origin and the spline's transform - zero progress should always equate to the path origin
		return FTransform(
			BasisTransform.TransformRotation(TargetToSpline.GetRotation()),
			BasisTransform.TransformPositionNoScale(TargetToSpline.GetLocation()),
			TargetToSpline.GetScale3D()* BasisTransform.GetScale3D());
	}
	return BasisTransform;
}
