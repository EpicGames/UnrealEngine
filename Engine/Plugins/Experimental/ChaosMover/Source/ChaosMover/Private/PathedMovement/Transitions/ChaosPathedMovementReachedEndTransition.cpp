// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosMover/PathedMovement/Transitions/ChaosPathedMovementReachedEndTransition.h"

#include "ChaosMover/PathedMovement/ChaosPathedMovementTypes.h"
#include "ChaosMover/ChaosMoverSimulation.h"
#include "MoverSimulationTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosPathedMovementReachedEndTransition)

UChaosPathedMovementReachedEndTransition::UChaosPathedMovementReachedEndTransition(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSupportsAsync = true;

	TransitionToMode = DefaultModeNames::Falling;
}

FTransitionEvalResult UChaosPathedMovementReachedEndTransition::Evaluate_Implementation(const FSimulationTickParams& Params) const
{
	FTransitionEvalResult EvalResult;

	if (TransitionToMode.IsNone())
	{
		return EvalResult;
	}
	
	check(Simulation);

	bool IsInPathedMovementMode = (nullptr != Cast<IChaosPathedMovementModeInterface>(Simulation->GetCurrentMovementMode()));

	if (IsInPathedMovementMode)
	{
		const FChaosPathedMovementState* FoundPreSimMoveState = Params.StartState.SyncState.SyncStateCollection.FindDataByType<FChaosPathedMovementState>();
		if (FoundPreSimMoveState && FoundPreSimMoveState->bHasFinished)
		{
			EvalResult.NextMode = TransitionToMode;
		}
	}

	return EvalResult;
}

void UChaosPathedMovementReachedEndTransition::Trigger_Implementation(const FSimulationTickParams& Params)
{
}