// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosMover/Utilities/ChaosGroundMovementUtils.h"

#include "Chaos/PhysicsObjectInternalInterface.h"
#include "MoveLibrary/FloorQueryUtils.h"
#include "MoveLibrary/MovementUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosGroundMovementUtils)

FVector UChaosGroundMovementUtils::ComputeLocalGroundVelocity_Internal(const FVector& Position, const FFloorCheckResult& FloorResult)
{
	Chaos::FVec3 GroundVelocity = Chaos::FVec3::ZeroVector;
    if (const Chaos::FPBDRigidParticleHandle* Rigid = GetRigidParticleHandleFromFloorResult_Internal(FloorResult))
    {
    	const FVector GroundPosition = Rigid->IsKinematic() ? FVector(Rigid->GetX()) : Rigid->GetTransformXRCom().GetLocation();
    	FVector Offset = Chaos::FVec3(Position) - GroundPosition;
    	Offset -= Offset.ProjectOnToNormal(FloorResult.HitResult.ImpactNormal);
    	GroundVelocity = Rigid->GetV() + Rigid->GetW().Cross(Offset);
    }
    return FVector(GroundVelocity);
}

Chaos::FPBDRigidParticleHandle* UChaosGroundMovementUtils::GetRigidParticleHandleFromFloorResult_Internal(const FFloorCheckResult& FloorResult)
{
	if (Chaos::FPhysicsObjectHandle PhysicsObject = FloorResult.HitResult.PhysicsObject)
	{
		Chaos::FReadPhysicsObjectInterface_Internal Interface = Chaos::FPhysicsObjectInternalInterface::GetRead();
		return Interface.GetRigidParticle(PhysicsObject);
	}

	return nullptr;
}

FProposedMove UChaosGroundMovementUtils::ComputeRequestedMove(const FRequestedMoveParams& Params)
{
	FProposedMove OutProposedMove;

	const FPlane MovementPlane(FVector::ZeroVector, Params.GroundNormal);
	FVector NewVelocity = UMovementUtils::ConstrainToPlane(Params.PriorVelocity, MovementPlane, true);

	// Copied from UCharacterMovementComponent::ApplyRequestedMove
	const float RequestedSpeedSquared = Params.RequestedVelocity.SizeSquared();
	float RequestedSpeed = 0.0f;
	FVector RequestedMoveDir = FVector::ForwardVector;
	const bool bZeroRequestedSpeed = RequestedSpeedSquared < UE_SMALL_NUMBER;
	if (!bZeroRequestedSpeed)
	{
		// Compute requested speed from path following
		RequestedSpeed = Params.RequestedVelocity.Size();
		RequestedMoveDir = UMovementUtils::ConstrainToPlane(Params.RequestedVelocity / RequestedSpeed, MovementPlane, true);
		RequestedSpeed = (Params.bRequestedMoveWithMaxSpeed ? Params.MaxSpeed : FMath::Min(Params.MaxSpeed, RequestedSpeed));

		OutProposedMove.bHasDirIntent = true;
		OutProposedMove.DirectionIntent = RequestedMoveDir;

		// Compute actual requested velocity
		const FVector MoveVelocity = RequestedMoveDir * RequestedSpeed;

		// Compute new velocity
		if (Params.bShouldComputeAcceleration && (Params.DeltaSeconds > UE_SMALL_NUMBER))
		{
			// How much do we need to accelerate to get to the new velocity?
			FVector NewAcceleration = ((MoveVelocity - NewVelocity) / Params.DeltaSeconds);
			NewAcceleration = NewAcceleration.GetClampedToMaxSize(Params.Acceleration);
			NewVelocity = Params.PriorVelocity + NewAcceleration * Params.DeltaSeconds;
		}
		else
		{
			// Just set velocity directly.
			// If decelerating we do so instantly, so we don't slide through the destination if we can't brake fast enough.
			NewVelocity = MoveVelocity;
		}
	}

	const bool bIsExceedingMaxSpeed = !Params.bRequestedMoveWithMaxSpeed && UMovementUtils::IsExceedingMaxSpeed(NewVelocity, Params.MaxSpeed);
	if (bZeroRequestedSpeed || bIsExceedingMaxSpeed)
	{
		// Dampen velocity magnitude based on deceleration.
		const FVector OldVelocity = NewVelocity;
		const float VelSize = FMath::Max(NewVelocity.Size() - FMath::Abs(Params.Deceleration) * Params.DeltaSeconds, 0.f);
		NewVelocity = NewVelocity.GetSafeNormal() * VelSize;

		// Don't allow braking to lower us below max speed if we started above it.
		if (bIsExceedingMaxSpeed && NewVelocity.SizeSquared() < FMath::Square(Params.MaxSpeed))
		{
			NewVelocity = OldVelocity.GetSafeNormal() * Params.MaxSpeed;
		}
	}

	OutProposedMove.LinearVelocity = NewVelocity;

	OutProposedMove.LinearVelocity = UMovementUtils::ConstrainToPlane(NewVelocity, MovementPlane, true);

	// Linearly rotate in place
	if (OutProposedMove.bHasDirIntent)
	{
		FRotator OrientationIntent = OutProposedMove.DirectionIntent.Rotation();
		OutProposedMove.AngularVelocityDegrees = UMovementUtils::ComputeAngularVelocityDegrees(Params.PriorOrientation, OrientationIntent, Params.DeltaSeconds, Params.TurningRate);
	}

	return OutProposedMove;
}
