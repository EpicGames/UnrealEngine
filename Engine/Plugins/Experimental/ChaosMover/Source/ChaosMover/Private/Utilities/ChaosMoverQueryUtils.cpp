// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosMover/Utilities/ChaosMoverQueryUtils.h"

#include "Chaos/PhysicsObject.h"
#include "Chaos/PhysicsObjectInternalInterface.h"
#include "ChaosMover/ChaosMoverConsoleVariables.h"
#include "ChaosMover/ChaosMoverDefines.h"
#include "Framework/Threading.h"
#include "MoveLibrary/FloorQueryUtils.h"
#include "MoveLibrary/WaterMovementUtils.h"
#include "Physics/GenericPhysicsInterface.h"
#include "WaterBodyActor.h"

#if CHAOSMOVER_DEBUG_DRAW
#include "Chaos/DebugDrawQueue.h"
#endif

namespace UE::ChaosMover::Utils
{
	namespace Internal
	{
		bool GetWaterResultFromHitResults_Internal(const TArray<FHitResult>& Hits, const FVector& Location, const float TargetHeight, FWaterCheckResult& OutWaterResult)
		{
			// Find the closet hit that is a water body
			// Note: Relies on ordering of hit results
			for (int32 Idx = 0; Idx < Hits.Num(); ++Idx)
			{
				const FHitResult& Hit = Hits[Idx];

				if (Hit.Component.IsValid())
				{
					if (AActor* Actor = Hit.Component->GetOwner())
					{
						if (Actor->IsA(AWaterBody::StaticClass()))
						{
							AWaterBody* WaterBody = Cast<AWaterBody>(Actor);

							OutWaterResult.HitResult = Hit;
							OutWaterResult.bSwimmableVolume = true;

							OutWaterResult.WaterSplineData.SplineInputKey = WaterBody->GetWaterBodyComponent()->FindInputKeyClosestToWorldLocation(Location);

							OutWaterResult.WaterSplineData.WaterBody = WaterBody;

							TValueOrError<FWaterBodyQueryResult, EWaterBodyQueryError> QueryResult = WaterBody->GetWaterBodyComponent()->TryQueryWaterInfoClosestToWorldLocation(
								Location,
								EWaterBodyQueryFlags::ComputeLocation | EWaterBodyQueryFlags::ComputeNormal | EWaterBodyQueryFlags::ComputeImmersionDepth,
								OutWaterResult.WaterSplineData.SplineInputKey
							);

							// If the query has no value, treat it as if this was never in the hit list:
							if (!QueryResult.HasValue())
							{
								continue;
							}

							const FWaterBodyQueryResult& Query = QueryResult.GetValue();

							OutWaterResult.WaterSplineData.ImmersionDepth = Query.GetImmersionDepth();

							OutWaterResult.WaterSplineData.WaterPlaneLocation = Query.GetWaterPlaneLocation();
							OutWaterResult.WaterSplineData.WaterPlaneNormal = Query.GetWaterPlaneNormal();

							const float CapsuleBottom = Location.Z;
							const float CapsuleTop = Location.Z + (TargetHeight * 2);
							OutWaterResult.WaterSplineData.WaterSurfaceLocation = Query.GetWaterSurfaceLocation();

							OutWaterResult.WaterSplineData.WaterSurfaceOffset = OutWaterResult.WaterSplineData.WaterSurfaceLocation - Location;

							OutWaterResult.WaterSplineData.ImmersionPercent = FMath::Clamp((OutWaterResult.WaterSplineData.WaterSurfaceLocation.Z - CapsuleBottom) / (CapsuleTop - CapsuleBottom), 0.f, 1.f);

							OutWaterResult.WaterSplineData.WaterSurfaceNormal = Query.GetWaterSurfaceNormal();

	#if CHAOSMOVER_DEBUG_DRAW
							Chaos::FDebugDrawQueue::GetInstance().DrawDebugLine(Location, Location - FVector::UpVector * OutWaterResult.WaterSplineData.ImmersionDepth, FColor::Blue, false, -1.f, 10, 1.0f);
							Chaos::FDebugDrawQueue::GetInstance().DrawDebugPoint(OutWaterResult.HitResult.Location, FColor::Blue, false, -1.f, 10, 1.0f);
	#endif
							return true;
						}
					}
				}
			}

			return false;
		}

	} // namespace Internal

	void FloorSweep_Internal(const FFloorSweepParams& Params, FFloorCheckResult& OutFloorResult, FWaterCheckResult& OutWaterResult)
	{
		QUICK_SCOPE_CYCLE_COUNTER(ChaosMoverUtils_FloorSweep);

		Chaos::EnsureIsInPhysicsThreadContext();

		TArray<FHitResult> Hits;

		const float DeltaPosVertLength = Params.DeltaPos.Dot(Params.UpDir);
		const FVector DeltaPosHoriz = Params.DeltaPos - DeltaPosVertLength * Params.UpDir;

		// Make sure the query is long enough to include the vertical movement
		const float AdjustedQueryDistance = FMath::Max(UE_KINDA_SMALL_NUMBER - DeltaPosVertLength + Params.TargetHeight, Params.QueryDistance);

		// The bottom of the query shape should be at the integrated location (ignoring vertical movement)
		FVector Start = Params.Location + DeltaPosHoriz + (Params.QueryRadius + UE_KINDA_SMALL_NUMBER) * Params.UpDir;
		FVector End = Start - AdjustedQueryDistance * Params.UpDir;
		FHitResult OutHit;
		if (Chaos::Private::FGenericPhysicsInterface_Internal::SpherecastMulti(Params.World, Params.QueryRadius, Hits, Start, End, Params.CollisionChannel, Params.QueryParams, Params.ResponseParams))
		{
			OutHit = Hits.Last();
		}

#if CHAOSMOVER_DEBUG_DRAW
		// Draw full length of query
		if (CVars::bDrawGroundQueries)
		{
			const FVector Center = 0.5f * (Start + End);
			const float Dist = (Start - End).Size();
			Chaos::FDebugDrawQueue::GetInstance().DrawDebugCapsule(Center, 0.5f * Dist + Params.QueryRadius, Params.QueryRadius, FQuat::Identity, FColor::Silver, false, -1.f, 10, 1.0f);
		}
#endif
		
		if (OutHit.bBlockingHit)
		{
			bool bWalkable = UFloorQueryUtils::IsHitSurfaceWalkable(OutHit, Params.UpDir, Params.MaxWalkSlopeCosine);

#if CHAOSMOVER_DEBUG_DRAW
			if (CVars::bDrawGroundQueries)
			{
				const FVector Center = Start - 0.5f * OutHit.Distance * Params.UpDir;
				const FColor Color = bWalkable ? FColor::Green : FColor::Red;
				Chaos::FDebugDrawQueue::GetInstance().DrawDebugCapsule(Center, 0.5f * OutHit.Distance + Params.QueryRadius, Params.QueryRadius, FQuat::Identity, Color, false, -1.f, 10, 1.0f);
			}
#endif
			OutFloorResult.bBlockingHit = true;
			OutFloorResult.bWalkableFloor = bWalkable;
			OutFloorResult.FloorDist = Params.UpDir.Dot(Params.Location - OutHit.ImpactPoint);
			OutFloorResult.HitResult = OutHit;

			// Update the ground distance based on the slope. If you are on a slope the query might hit on an edge
			// rather than directly under the capsule. Also move back to the start location
			const float DP = OutHit.ImpactNormal.Dot(Params.UpDir);
			if (DP > UE_SMALL_NUMBER)
			{
				OutFloorResult.FloorDist += 2.0f * Params.QueryRadius * (1.0f - DP) / DP
					+ FVector::VectorPlaneProject(Params.DeltaPos, OutHit.ImpactNormal).Dot(Params.UpDir);
			}

			// Make sure that the object stored in the hit result is the root object
			if (const Chaos::FPhysicsObjectHandle HitObject = OutHit.PhysicsObject)
			{
				Chaos::FReadPhysicsObjectInterface_Internal Interface = Chaos::FPhysicsObjectInternalInterface::GetRead();
				OutFloorResult.HitResult.PhysicsObject = Interface.GetRootObject(HitObject);
			}
		}
		else
		{
			OutFloorResult.Clear();
			OutFloorResult.FloorDist = 1.0e10f;
		}

		Internal::GetWaterResultFromHitResults_Internal(Hits, Params.Location, Params.TargetHeight, OutWaterResult);
	}

	bool CapsuleOverlapTest_Internal(const FCapsuleOverlapTestParams& Params)
	{
		QUICK_SCOPE_CYCLE_COUNTER(ChaosMoverUtils_CapsuleOverlapTest);

		Chaos::EnsureIsInPhysicsThreadContext();

		TArray<FHitResult> Hits;

		FVector Start = Params.Location - (Params.CapsuleHalfHeight -  Params.CapsuleRadius) * Params.UpDir;
		FVector End = Params.Location + (Params.CapsuleHalfHeight - Params.CapsuleRadius) * Params.UpDir;
		FHitResult OutHit;
		if (Chaos::Private::FGenericPhysicsInterface_Internal::SpherecastMulti(Params.World, Params.CapsuleRadius, Hits, Start, End, Params.CollisionChannel, Params.QueryParams, Params.ResponseParams))
		{
			OutHit = Hits.Last();
		}

		bool bBlockingHitFound = false;
		if (OutHit.bBlockingHit)
		{
			if (!Params.bBlockOnDynamic)
			{
				Chaos::FReadPhysicsObjectInterface_Internal Interface = Chaos::FPhysicsObjectInternalInterface::GetRead();
				bBlockingHitFound = !Interface.AreAllDynamic({ OutHit.PhysicsObject });
			}
			else
			{
				bBlockingHitFound = true;
			}
		}

#if CHAOSMOVER_DEBUG_DRAW
		if (CVars::bDrawOverlapQueries)
		{
			const FVector Center = Params.Location;
			const FColor Color = bBlockingHitFound ? FColor::Purple : FColor::Turquoise;
			Chaos::FDebugDrawQueue::GetInstance().DrawDebugCapsule(Center, Params.CapsuleHalfHeight, Params.CapsuleRadius, FQuat::Identity, Color, false, -1.f, 10, 1.0f);
		}
#endif

		return bBlockingHitFound;
	}

} // namespace UE::ChaosMover::Utils