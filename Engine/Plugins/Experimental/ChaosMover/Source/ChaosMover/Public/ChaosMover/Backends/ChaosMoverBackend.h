// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Backends/MoverBackendLiaison.h"
#include "Chaos/Character/CharacterGroundConstraint.h"
#include "Chaos/Declares.h"
#include "ChaosMover/ChaosMoverSimulation.h"
#include "ChaosMover/ChaosMoverSimulationTypes.h"
#include "ChaosMover/PathedMovement/ChaosPathedMovementTypes.h"
#include "Components/ActorComponent.h"

#include "ChaosMoverBackend.generated.h"

class UMoverComponent;
class UNetworkPhysicsComponent;

namespace Chaos
{
	class FCharacterGroundConstraint;
	class FCharacterGroundConstraintProxy;

	class FJointConstraint;
	class FJointConstraintPhysicsProxy;
}

UCLASS(MinimalAPI, Within = MoverComponent)
class UChaosMoverBackendComponent : public UActorComponent, public IMoverBackendLiaisonInterface
{
	GENERATED_BODY()

public:
	CHAOSMOVER_API UChaosMoverBackendComponent();

	CHAOSMOVER_API virtual void InitializeComponent() override;
	CHAOSMOVER_API virtual void UninitializeComponent() override;
	CHAOSMOVER_API virtual void BeginPlay() override;
	CHAOSMOVER_API virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;

	// IMoverBackendLiaisonInterface
	CHAOSMOVER_API virtual bool IsAsync() const override;
	CHAOSMOVER_API virtual bool IsFixedDt() const override;
	CHAOSMOVER_API virtual bool ShouldResim() const override;
	CHAOSMOVER_API virtual float GetEventSchedulingMinDelaySeconds() const override;
	CHAOSMOVER_API virtual FMoverTime GetScheduledNetworkTime(const FMoverTime& Time) const override;
	CHAOSMOVER_API virtual double GetCurrentSimTimeMs();
	CHAOSMOVER_API virtual int32 GetCurrentSimFrame();

	CHAOSMOVER_API virtual void ProduceInputData(int32 PhysicsStep, int32 NumSteps, const FMoverTimeStep& TimeStep, UE::ChaosMover::FSimulationInputData& InputData);
	CHAOSMOVER_API virtual void ConsumeOutputData(const FMoverTimeStep& TimeStep, const UE::ChaosMover::FSimulationOutputData& OutputData);
	CHAOSMOVER_API virtual void FinalizeFrame(double ResultsTimeInMs);
	CHAOSMOVER_API virtual void OnSimulationRollback(const FMoverSyncState& NewSyncState, const FMoverTimeStep& NewBaseTimeStep, const FMoverTimeStep& InPreRollbackTimeStep) override;

	CHAOSMOVER_API UChaosMoverSimulation* GetSimulation();
	CHAOSMOVER_API const UChaosMoverSimulation* GetSimulation() const;

	const FMoverTimeStep& GetPreRollbackTimeStep() const { return PreRollbackTimeStep; }

protected:
	CHAOSMOVER_API virtual void InitSimulation();
	CHAOSMOVER_API virtual void DeinitSimulation();
	CHAOSMOVER_API virtual void CreatePhysics();
	CHAOSMOVER_API virtual void DestroyPhysics();

	CHAOSMOVER_API void CreateCharacterGroundConstraint();
	CHAOSMOVER_API void DestroyCharacterGroundConstraint();

	CHAOSMOVER_API void CreateActuationConstraint();
	CHAOSMOVER_API void DestroyActuationConstraint();

	CHAOSMOVER_API virtual void GenerateInput(const FMoverTimeStep& TimeStep, UE::ChaosMover::FSimulationInputData& InputData);
	CHAOSMOVER_API virtual void GenerateServerInput(const FMoverTimeStep& TimeStep, UE::ChaosMover::FSimulationInputData& InputData);

	CHAOSMOVER_API UMoverComponent& GetMoverComponent() const;
	CHAOSMOVER_API Chaos::FPhysicsSolver* GetPhysicsSolver() const;
	CHAOSMOVER_API Chaos::FPhysicsObject* GetPhysicsObject() const;
	CHAOSMOVER_API Chaos::FPBDRigidParticle* GetControlledParticle() const;

	UFUNCTION()
	CHAOSMOVER_API virtual void HandleUpdatedComponentPhysicsStateChanged(UPrimitiveComponent* ChangedComponent, EComponentPhysicsStateChange StateChange);

	UFUNCTION()
	CHAOSMOVER_API void HandleOwningPawnControllerChanged_Server(APawn* OwnerPawn, AController* OldController, AController* NewController);

	TSubclassOf<UChaosMoverSimulation> SimulationClass;

	UPROPERTY(Transient)
	TObjectPtr<UNetworkPhysicsComponent> NetworkPhysicsComponent;

	UPROPERTY(Transient)
	TObjectPtr<UNullMovementMode> NullMovementMode;

	UPROPERTY(Transient)
	TObjectPtr<UImmediateMovementModeTransition> ImmediateModeTransition;

	UPROPERTY(Transient)
	TObjectPtr<UChaosMoverSimulation> Simulation;

	UE::Mover::FSimulationOutputRecord SimOutputRecord;

	// Last forward-simulated (non-resim) timestep simulated before the latest rollback; passed as PreRollbackTimeStep in OnSimulationRollback.
	FMoverTimeStep PreRollbackTimeStep;

	// Character ground constraint, for moving on ground like characters
	TUniquePtr<Chaos::FCharacterGroundConstraint> CharacterGroundConstraint;

	// General purpose joint constraint, for moving the controlled component physically
	FPhysicsConstraintHandle ActuationConstraintHandle;
	FConstraintInstance ActuationConstraintInstance;
	FPhysicsUserData ActuationConstraintPhysicsUserData;

	FTransform TransformOnInit;

	bool bIsUsingAsyncPhysics = false;
	bool bWantsDestroySim = false;
	bool bWantsCreateSim = true;

	// Transfers queued instant movement effects from the simulation to the simulation input (via networked input or local sim input).
	void InjectInstantMovementEffectsIntoSim(UE::ChaosMover::FSimulationInputData& OutInputData);

	// Transfers the given instant movement effects to the (networked) simulation input.
	// This is used in the input producing case.
	void InjectInstantMovementEffectsIntoInput(UE::ChaosMover::FSimulationInputData& OutInputData, const TArray<FScheduledInstantMovementEffect>& Effects);

	// Transfers the given instant movement effects to the simulation directly, bypassing the networked input.
	// This is used in the non input producing case.
	void InjectInstantMovementEffectsIntoLocalSimInput(const TArray<FScheduledInstantMovementEffect>& Effects);

	// Routes the given instant movement effects through the UNetworkPhysicsComponent sim action system using
	// Proposed auth style. Called by InjectInstantMovementEffectsIntoSim when
	// ChaosMover.Networking.NetworkInstantMovementEffectsWithSimActions is true.
	// All roles independently enqueue the same action (having detected the condition from GetLastInputCmd() in PostSim).
	void InjectInstantMovementEffectsAsActions(const TArray<FScheduledInstantMovementEffect>& Effects);

	// Transfers queued layered moves from the simulation to the simulation input (via networked input or local sim input).
	void InjectLayeredMovesIntoSim(UE::ChaosMover::FSimulationInputData& OutInputData);

	// Transfers the given layered moves to the (networked) simulation input.
	// This is used in the input producing case.
	void InjectLayeredMovesIntoInput(UE::ChaosMover::FSimulationInputData& OutInputData, const TArray<FScheduledLayeredMove>& Moves);

	// Transfers the given layered moves to the simulation directly, bypassing the networked input.
	// This is used in the non input producing case.
	void InjectLayeredMovesIntoLocalSimInput(const TArray<FScheduledLayeredMove>& Moves);

	// Routes the given layered moves through the UNetworkPhysicsComponent sim action system using Proposed auth style.
	// Called by InjectLayeredMovesIntoSim when ChaosMover.Networking.NetworkMovesWithSimActions is true.
	// All roles independently enqueue the same action (having detected the condition from GetLastInputCmd() in PostSim).
	void InjectLayeredMovesAsActions(const TArray<FScheduledLayeredMove>& Moves);

	// GT-callable: enqueue a FLayeredMoveInstance into the simulation-level queue for injection on the next tick.
	CHAOSMOVER_API void QueueLayeredMoveInstance(TSharedPtr<FLayeredMoveInstance> Move);

	// Transfers queued FLayeredMoveInstance moves from the simulation to the simulation input (via networked input or local sim input).
	void InjectLayeredMoveInstancesIntoSim(UE::ChaosMover::FSimulationInputData& OutInputData);

	// Transfers the given scheduled FLayeredMoveInstance moves to the (networked) simulation input.
	void InjectLayeredMoveInstancesIntoInput(UE::ChaosMover::FSimulationInputData& OutInputData, const TArray<FChaosScheduledLayeredMoveInstance>& ScheduledInstances);

	// Transfers the given scheduled FLayeredMoveInstance moves to the simulation directly, bypassing networked input.
	void InjectLayeredMoveInstancesIntoLocalSimInput(const TArray<FChaosScheduledLayeredMoveInstance>& ScheduledInstances);

	// Routes the given scheduled FLayeredMoveInstance activations through the UNetworkPhysicsComponent sim action system using
	// Proposed auth style. Called by InjectLayeredMoveInstancesIntoSim when
	// ChaosMover.Networking.NetworkMovesWithSimActions is true.
	// All roles independently enqueue the same action (having detected the condition from GetLastInputCmd() in PostSim).
	void InjectLayeredMoveInstancesAsActions(const TArray<FChaosScheduledLayeredMoveInstance>& ScheduledInstances);

private:

	void ConvertToSerializable(const TArray<FScheduledInstantMovementEffect>& ScheduledInstantMovementEffects, FChaosNetInstantMovementEffectsQueue& OutNetInstantMovementEffectsQueue);
	void ConvertToSerializable(const TArray<FScheduledLayeredMove>& ScheduledLayeredMoves, FChaosNetLayeredMovesQueue& OutNetLayeredMovesQueue);
	void ConvertToSerializable(const TArray<FChaosScheduledLayeredMoveInstance>& ScheduledMoveInstances, FChaosNetLayeredMoveInstancesQueue& OutNetQueue);

	// Removes past-due layered move entries from LocalSimInput so they are not re-marshalled to the
	// PT on every tick. Called once per ProduceInputData before the inject calls.
	void CullPastDueLocalSimInputMoves();

	// Returns the next unique ID used to prevent multiple application of the same sim command
	uint8 GetNextUniqueSimCommandID();

	// Unique ID counter for sim commands.
	// Unique IDs need to be generated because input can be repeated and we need to prevent multiple application of a given command only queued once.
	uint8 NextUniqueSimCommandID = 0xFF;
};
