// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ChaosMover/ChaosMovementMode.h"
#include "ChaosMover/ChaosMoverSourceBase.h"
#include "ChaosMover/ChaosMoveExecutorBase.h"
#include "ChaosCompositeMovementMode.generated.h"

#define UE_API CHAOSMOVER_API

/**
 * UChaosCompositeMovementMode: a ChaosMover movement mode that composes a
 * UChaosMoverSourceBase (what velocity to produce) with a UChaosMoveExecutorBase
 * (how to apply it to the physics simulation). This separates move generation from
 * physics application so that both can be configured and swapped independently.
 *
 * All simulation methods are async-safe — no AActor or UActorComponent access.
 *
 * When MoveSource::IsFinished() returns true, the mode automatically transitions to
 * NextModeName (if set). The Transitions array can be used for additional conditions.
 *
 * If the executor implements IChaosCharacterMovementModeInterface and/or
 * IChaosCharacterConstraintMovementModeInterface, CollectSimulationInterfaces
 * automatically exposes those to the simulation -- no subclassing required.
 */
UCLASS(MinimalAPI, Blueprintable, BlueprintType, EditInlineNew, DefaultToInstanced)
class UChaosCompositeMovementMode : public UChaosMovementMode
{
	GENERATED_BODY()

public:
	UE_API UChaosCompositeMovementMode(const FObjectInitializer& ObjectInitializer);

	/** The source that produces a FProposedMove each tick. */
	UPROPERTY(EditAnywhere, Instanced, Category = Mover)
	TObjectPtr<UChaosMoverSourceBase> MoveSource;

	/** Applies the proposed move to the Chaos physics simulation state each tick. */
	UPROPERTY(EditAnywhere, Instanced, Category = Mover)
	TObjectPtr<UChaosMoveExecutorBase> MoveExecutor;

	/** Mode to transition to when MoveSource::IsFinished() returns true. NAME_None = no auto-transition. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	FName NextModeName;

	UE_API virtual void OnRegistered(const FName ModeName, const FMoverSimContext& SimContext) override;
	UE_API virtual void OnUnregistered(const FMoverSimContext& SimContext) override;

	UE_API virtual void Activate(const FMoverEventContext& Context, FName PrevModeName, const FMoverSimContext& SimContext, const FMoverTickStartData& StartState, FMoverSyncState* OutSyncState, FMoverAuxStateContext* OutAuxState) override;
	UE_API virtual void Deactivate(const FMoverEventContext& Context, FName InNextModeName, const FMoverSimContext& SimContext) override;

	UE_API virtual void GenerateMove_Implementation(const FMoverSimContext& SimContext, const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, FProposedMove& OutProposedMove) const override;
	UE_API virtual void SimulationTick_Implementation(const FSimulationTickParams& Params, FMoverTickEndData& OutputState) override;

	UE_API virtual void CollectSimulationInterfaces(FChaosMoverSimulationInterfaceCache& OutCache) override;

private:
	// Set when MoveSource->OnEnd_Async has been called, cleared on Activate.
	// Prevents OnEnd_Async from being called more than once per activation when
	// IsFinished() returns true but no mode transition is configured.
	bool bMoveSourceEnded = false;
};

#undef UE_API
