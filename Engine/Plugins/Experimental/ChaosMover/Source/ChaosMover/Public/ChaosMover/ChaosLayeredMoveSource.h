// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ChaosMover/ChaosMoverSourceBase.h"
#include "LayeredMove.h"
#include "StructUtils/InstancedStruct.h"
#include "ChaosLayeredMoveSource.generated.h"

#define UE_API CHAOSMOVER_API

/**
 * UChaosLayeredMoveSource: a UChaosMoverSourceBase that drives move generation
 * from a single FLayeredMoveBase instance stored as an instanced struct property.
 *
 * All four source lifecycle methods (OnStart_Async, GenerateMove_Async, IsFinished,
 * OnEnd_Async) are fully delegated to the underlying FLayeredMoveBase async API,
 * so subclasses of FLayeredMoveBase can be configured and swapped in the editor
 * without subclassing this object.
 */
UCLASS(MinimalAPI, Blueprintable, BlueprintType, EditInlineNew, DefaultToInstanced)
class UChaosLayeredMoveSource : public UChaosMoverSourceBase
{
	GENERATED_BODY()

public:
	/** The layered move that drives velocity generation. */
	UPROPERTY(EditAnywhere, Category = Mover)
	TInstancedStruct<FLayeredMoveBase> LayeredMove;

	UE_API virtual void OnStart_Async(UMoverBlackboard* SimBlackboard, const FMoverTime& SimTime) override;
	UE_API virtual void GenerateMove_Async(const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, UMoverBlackboard* SimBlackboard, FProposedMove& OutProposedMove) override;
	UE_API virtual bool IsFinished(double CurrentSimTimeMs) const override;
	UE_API virtual void OnEnd_Async(UMoverBlackboard* SimBlackboard, double CurrentSimTimeMs) override;
	UE_API virtual void AppendMontageOutputEntry(TArray<FMoverSimDrivenMontageEntry>& OutEntries, const FMoverTimeStep& TimeStep) const override;
};

#undef UE_API
