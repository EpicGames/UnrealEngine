// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ChaosMoveExecutorBase.generated.h"

#define UE_API CHAOSMOVER_API

class UChaosMoverSimulation;
struct FSimulationTickParams;
struct FMoverTickEndData;

/**
 * Abstract base class for move executors in ChaosMover.
 *
 * A move executor is responsible for applying a FProposedMove to the physics simulation
 * state — computing target delta positions, updating the sync state, and handling any
 * surface interaction. It decouples the "how to apply movement" concern from the mode
 * hierarchy, allowing physics behaviors (walking, free-move, etc.) to be composed
 * independently of move generation.
 *
 * All methods must be safe to call from the physics simulation thread — no access to
 * AActor or UActorComponent is permitted.
 *
 * Concrete subclasses that support the Chaos character ground constraint should
 * additionally implement IChaosCharacterMovementModeInterface and/or
 * IChaosCharacterConstraintMovementModeInterface. UChaosCompositeMovementMode
 * automatically exposes those interfaces to the simulation via
 * CollectSimulationInterfaces.
 */
UCLASS(Abstract, MinimalAPI, EditInlineNew, DefaultToInstanced)
class UChaosMoveExecutorBase : public UObject
{
	GENERATED_BODY()

public:
	/**
	 * Called by the owning mode during its OnRegistered, on the game thread.
	 * Subclasses may override to perform initialization that requires game-thread
	 * access (e.g. reading character mesh offset, finding shared settings).
	 */
	virtual void OnModeRegistered(const FName ModeName) {}

	/**
	 * Called by the owning mode during its OnUnregistered, on the game thread.
	 * Subclasses may override to release any references acquired in OnModeRegistered.
	 */
	virtual void OnModeUnregistered() {}

	/**
	 * Applies Params.ProposedMove to the simulation output state. Thread-safe —
	 * no AActor or UActorComponent access.
	 */
	virtual void ExecuteMove_Async(
		const FSimulationTickParams& Params,
		FMoverTickEndData& OutputState) {}

	/**
	 * Propagates the simulation reference so that the executor can perform physics
	 * queries (e.g. floor sweeps) when needed. Called by the owning mode during
	 * OnRegistered and cleared during OnUnregistered.
	 */
	virtual void SetSimulation(UChaosMoverSimulation* InSimulation)
	{
		Simulation = InSimulation;
	}

protected:
	TObjectPtr<UChaosMoverSimulation> Simulation;
};

#undef UE_API
