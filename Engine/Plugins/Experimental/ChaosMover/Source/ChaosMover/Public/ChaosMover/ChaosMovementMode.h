// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosMover/ChaosMoverSimulationTypes.h"
#include "MovementMode.h"

#include "ChaosMovementMode.generated.h"

class UChaosMoverSimulation;
namespace Chaos
{
	class FCollisionContactModifier;
}

UENUM()
enum class EChaosMoverIgnoredCollisionMode : uint8
{
	EnableCollisionsWithIgnored,
	DisableCollisionsWithIgnored,
};

/**
 * Base class for all Chaos movement modes
 */
UCLASS(MinimalAPI, Abstract, Blueprintable, BlueprintType, EditInlineNew, DefaultToInstanced)
class UChaosMovementMode : public UBaseMovementMode
{
	GENERATED_BODY()

public:
	CHAOSMOVER_API UChaosMovementMode(const FObjectInitializer& ObjectInitializer);

	UFUNCTION(BlueprintPure, Category = ChaosMover)
	const UChaosMoverSimulation* GetSimulation() const
	{
		return Simulation;
	}

	CHAOSMOVER_API void SetSimulation(UChaosMoverSimulation* InSimulation);

	// Whether this movement mode is relative to a basis transform
	virtual bool UsesMovementBasisTransform() const
	{
		return false;
	}

	// Whether the mode allows teleportation to the target transform
	// This is for mode specific teleportation tests, in addition to those already done prior to this check
	virtual bool CanTeleport(const FTransform& TargetTransform, const FMoverSyncState& CurrentSyncState) const
	{
		return true;
	}

	virtual void ModifyContacts(const FMoverTimeStep& TimeStep, const UE::ChaosMover::FSimulationInputData& InputData, const UE::ChaosMover::FSimulationOutputData& OutputData, Chaos::FCollisionContactModifier& Modifier) const
	{
	}

	// Populate OutCache with simulation interface pointers (IChaosPreSimulationTickInterface,
	// IChaosPostSimulationTickInterface) this mode can service. The base implementation
	// casts 'this'; composite overrides also try the executor.
	CHAOSMOVER_API virtual void CollectSimulationInterfaces(FChaosMoverSimulationInterfaceCache& OutCache);

	UPROPERTY(EditAnywhere, Category = "Collision Settings")
	EChaosMoverIgnoredCollisionMode IgnoredCollisionMode = EChaosMoverIgnoredCollisionMode::DisableCollisionsWithIgnored;

#if WITH_EDITOR
	CHAOSMOVER_API virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;
#endif // WITH_EDITOR

protected:
	UChaosMoverSimulation* Simulation;
};

#undef UE_API
