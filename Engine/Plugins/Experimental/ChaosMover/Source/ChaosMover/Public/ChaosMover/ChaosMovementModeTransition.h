// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovementModeTransition.h"

#include "ChaosMovementModeTransition.generated.h"

class UChaosMoverSimulation;

/**
 * Base class for all Chaos movement mode transitions
 */
UCLASS(MinimalAPI, Abstract, Blueprintable, BlueprintType, EditInlineNew, DefaultToInstanced)
class UChaosMovementModeTransition : public UBaseMovementModeTransition
{
	GENERATED_BODY()

public:
	CHAOSMOVER_API UChaosMovementModeTransition(const FObjectInitializer& ObjectInitializer);

	UFUNCTION(BlueprintPure, Category = ChaosMover)
	const UChaosMoverSimulation* GetSimulation() const
	{
		return Simulation;
	}

	UFUNCTION(BlueprintPure, Category = ChaosMover)
	UChaosMoverSimulation* GetSimulation_Mutable()
	{
		return Simulation;
	}

	void SetSimulation(UChaosMoverSimulation* InSimulation)
	{
		Simulation = InSimulation;
	}

#if WITH_EDITOR
	CHAOSMOVER_API virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;
#endif // WITH_EDITOR

protected:
	UChaosMoverSimulation* Simulation;
};