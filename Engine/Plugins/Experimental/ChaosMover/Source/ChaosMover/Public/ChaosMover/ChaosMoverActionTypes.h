// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InstantMovementEffect.h"
#include "LayeredMove.h"
#include "LayeredMoveBase.h"
#include "Physics/NetworkPhysicsComponent.h"
#include "StructUtils/InstancedStruct.h"

#include "ChaosMoverActionTypes.generated.h"

/**
 * Scheduled action that delivers any FLayeredMoveBase subtype to the ChaosMover simulation
 * via UNetworkPhysicsComponent::EnqueueScheduledAction_External.
 */
USTRUCT()
struct FChaosMoverLayeredMoveAction : public FNetworkPhysicsActionPayload
{
	GENERATED_BODY()

	virtual const EActionAuthorStyle GetAuthorStyle() const override { return AuthorStyle; }

	// Controls who authors this action and whether client prediction is active.
	//   Predicted / PredictedAutonomousOnly: all eligible roles independently detect the condition
	//     (e.g. from GetLastInputCmd() in PostSim) and enqueue the same action. Server-authoritative.
	//   Authority: server-only; replicates down to clients without client-side prediction.
	//   Proposed / ProposedAutonomousOnly: the client proposes its locally-detected move parameters;
	//     the server compares them to its own via FLayeredMoveBase::IsNearlyEqualTo. If accepted, the
	//     client's data wins (no resim). If rejected, the server's data wins (client resims). With the
	//     default exact-field comparison, this is equivalent to Predicted (server wins on any divergence).
	//     Override IsNearlyEqualTo on the FLayeredMoveBase subtype to accept proposals within a tolerance --
	//     a resim-minimizing strategy where the server forgives small client deviations.
	EActionAuthorStyle AuthorStyle = EActionAuthorStyle::ProposedAutonomousOnly;

	// The move to deliver. Supports any FLayeredMoveBase subtype.
	UPROPERTY()
	TInstancedStruct<FLayeredMoveBase> Move;

	virtual bool IsNearlyEqual(const FNetworkPhysicsActionPayload& Other) const override
	{
		const FChaosMoverLayeredMoveAction& OtherTyped = static_cast<const FChaosMoverLayeredMoveAction&>(Other);
		if (!Move.IsValid() || !OtherTyped.Move.IsValid()) { return false; }
		if (Move.GetScriptStruct() != OtherTyped.Move.GetScriptStruct()) { return false; }
		return Move.Get<FLayeredMoveBase>().IsNearlyEqualTo(OtherTyped.Move.Get<FLayeredMoveBase>());
	}
};

/**
 * Scheduled action that activates a FLayeredMoveInstance (ULayeredMoveLogic + FLayeredMoveInstancedData)
 * on the ChaosMover simulation via UNetworkPhysicsComponent::EnqueueScheduledAction_External.
 */
USTRUCT()
struct FChaosMoverLayeredMoveInstanceAction : public FNetworkPhysicsActionPayload
{
	GENERATED_BODY()

	virtual const EActionAuthorStyle GetAuthorStyle() const override { return AuthorStyle; }

	// Controls who authors this action and whether client prediction is active.
	//   Predicted / PredictedAutonomousOnly: all eligible roles independently detect the condition
	//     (e.g. from GetLastInputCmd() in PostSim) and enqueue the same action. Server-authoritative.
	//   Authority: server-only; replicates down to clients without client-side prediction.
	//   Proposed / ProposedAutonomousOnly: the client proposes its locally-detected move parameters;
	//     the server compares them to its own via FLayeredMoveInstancedData::IsNearlyEqualTo. If
	//     accepted, the client's data wins (no resim). If rejected, the server's data wins (client
	//     resims). With the default exact-field comparison, this is equivalent to Predicted (server
	//     wins on any divergence). Override IsNearlyEqualTo on the FLayeredMoveInstancedData subtype
	//     to accept proposals within a tolerance -- a resim-minimizing strategy where the server
	//     forgives small client deviations.
	EActionAuthorStyle AuthorStyle = EActionAuthorStyle::ProposedAutonomousOnly;

	// Logic class serialized for re-linking on the simulation thread.
	UPROPERTY()
	TSubclassOf<ULayeredMoveLogic> MoveLogicClass;

	// Per-activation data. Supports any FLayeredMoveInstancedData subtype.
	UPROPERTY()
	TInstancedStruct<FLayeredMoveInstancedData> InstancedData;

	virtual bool IsNearlyEqual(const FNetworkPhysicsActionPayload& Other) const override
	{
		const FChaosMoverLayeredMoveInstanceAction& OtherTyped = static_cast<const FChaosMoverLayeredMoveInstanceAction&>(Other);
		if (MoveLogicClass != OtherTyped.MoveLogicClass) { return false; }
		if (!InstancedData.IsValid() || !OtherTyped.InstancedData.IsValid()) { return false; }
		if (InstancedData.GetScriptStruct() != OtherTyped.InstancedData.GetScriptStruct()) { return false; }
		return InstancedData.Get<FLayeredMoveInstancedData>().IsNearlyEqualTo(OtherTyped.InstancedData.Get<FLayeredMoveInstancedData>());
	}
};

/**
 * Scheduled action that delivers any FInstantMovementEffect to the ChaosMover simulation
 * via UNetworkPhysicsComponent::EnqueueScheduledAction_External.
 *
 * Callers populate Effect with any FInstantMovementEffect subtype, then call:
 *
 *   FChaosMoverInstantMovementEffectAction Action;
 *   Action.Effect.InitializeAs<FMyEffect>(...);
 *   NetworkPhysicsComponent->EnqueueScheduledAction_External(Action, SourceObject, DelaySeconds, bReliable);
 *
 * The effect is applied on the simulation thread at the scheduled frame, using the same
 * QueueInstantMovementEffect path as locally-triggered effects.
 */
USTRUCT()
struct FChaosMoverInstantMovementEffectAction : public FNetworkPhysicsActionPayload
{
	GENERATED_BODY()

	virtual const EActionAuthorStyle GetAuthorStyle() const override
	{
		return AuthorStyle;
	}

	// Controls who authors this action and whether client prediction is active.
	//   Predicted / PredictedAutonomousOnly: all eligible roles independently detect the condition
	//     (e.g. from GetLastInputCmd() in PostSim) and enqueue the same action. Server-authoritative.
	//   Authority: server-only; replicates down to clients without client-side prediction.
	//   Proposed / ProposedAutonomousOnly: the client proposes its locally-detected effect parameters;
	//     the server compares them to its own via FInstantMovementEffect::IsNearlyEqualTo. If accepted,
	//     the client's data wins (no resim). If rejected, the server's data wins (client resims). With
	//     the default exact-field comparison, this is equivalent to Predicted (server wins on any
	//     divergence). Override IsNearlyEqualTo on the FInstantMovementEffect subtype to accept
	//     proposals within a tolerance -- a resim-minimizing strategy where the server forgives small
	//     client deviations.
	EActionAuthorStyle AuthorStyle = EActionAuthorStyle::ProposedAutonomousOnly;

	// The effect to deliver. Supports any FInstantMovementEffect subtype.
	UPROPERTY()
	TInstancedStruct<FInstantMovementEffect> Effect;

	virtual bool IsNearlyEqual(const FNetworkPhysicsActionPayload& Other) const override
	{
		const FChaosMoverInstantMovementEffectAction& OtherTyped =
			static_cast<const FChaosMoverInstantMovementEffectAction&>(Other);
		if (!Effect.IsValid() || !OtherTyped.Effect.IsValid()) { return false; }
		if (Effect.GetScriptStruct() != OtherTyped.Effect.GetScriptStruct()) { return false; }
		return Effect.Get<FInstantMovementEffect>().IsNearlyEqualTo(OtherTyped.Effect.Get<FInstantMovementEffect>());
	}
};
