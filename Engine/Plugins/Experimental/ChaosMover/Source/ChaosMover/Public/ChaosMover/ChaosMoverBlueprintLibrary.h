// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "LayeredMoveBase.h"

#include "ChaosMoverBlueprintLibrary.generated.h"

class UMoverComponent;

UCLASS()
class CHAOSMOVER_API UChaosMoverBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	// Returns true if layered moves and instanced move activations queued via QueueLayeredMove/QueueLayeredMoveInstance
	// are networked via the sim action system (Proposed style: all roles independently detect from
	// GetLastInputCmd() in PostSim; server-authoritative, cheat-resistant).
	// Returns false if they are embedded in the input cmd (PreSim detection; client-authoritative, co-op path).
	// IMPORTANT: when true, BP must detect conditions from GetLastInputCmd() in PostSim, NOT from the PreSim input.
	UFUNCTION(BlueprintPure, Category = "ChaosMover|Networking")
	static bool IsNetworkingMovesWithSimActions();

	// Returns true if instant movement effects queued via QueueInstantMovementEffect are networked via the
	// sim action system (Proposed style: all roles independently detect from GetLastInputCmd() in PostSim;
	// server-authoritative, cheat-resistant).
	// Returns false if they are embedded in the input cmd (PreSim detection; client-authoritative, co-op path).
	// IMPORTANT: when true, BP must detect conditions from GetLastInputCmd() in PostSim, NOT from the PreSim input.
	UFUNCTION(BlueprintPure, Category = "ChaosMover|Networking")
	static bool IsNetworkingInstantMovementEffectsWithSimActions();

	/**
	 * Server-only. Enqueues a layered move as an authoritative sim action, replicating it to clients
	 * without client-side prediction. Use for server-initiated moves (e.g. forced launches, knockbacks).
	 * Requires a ChaosMover backend (UNetworkPhysicsComponent on the owner); asserts in non-shipping
	 * builds if that requirement is not met. No-ops silently if called without authority.
	 * The move struct must derive from FLayeredMoveBase.
	 */
	UFUNCTION(BlueprintCallable, CustomThunk, Category = "ChaosMover|Networking",
		meta = (CustomStructureParam = "MoveAsRawData", AllowAbstract = "false",
		        DisplayName = "Queue Layered Move (Authority)", DefaultToSelf = "MoverComponent"))
	static void K2_QueueLayeredMove_Authority(UMoverComponent* MoverComponent, UPARAM(DisplayName = "Layered Move") const int32& MoveAsRawData);
	DECLARE_FUNCTION(execK2_QueueLayeredMove_Authority);

	/**
	 * Server-only. Enqueues an instanced layered move activation as an authoritative sim action,
	 * replicating it to clients without client-side prediction. Use for server-initiated activations.
	 * Requires a ChaosMover backend (UNetworkPhysicsComponent on the owner); asserts in non-shipping
	 * builds if that requirement is not met. No-ops silently if called without authority.
	 */
	UFUNCTION(BlueprintCallable, Category = "ChaosMover|Networking",
		meta = (DefaultToSelf = "MoverComponent"))
	static void QueueLayeredMoveInstance_Authority(UMoverComponent* MoverComponent,
		TSubclassOf<ULayeredMoveLogic> MoveLogicClass,
		UPARAM(ref) FLayeredMoveInstancedData& InstancedData);

	/**
	 * Server-only. Enqueues an instant movement effect as an authoritative sim action, replicating it
	 * to clients without client-side prediction. Use for server-initiated effects (e.g. forced teleports).
	 * Requires a ChaosMover backend (UNetworkPhysicsComponent on the owner); asserts in non-shipping
	 * builds if that requirement is not met. No-ops silently if called without authority.
	 * The effect struct must derive from FInstantMovementEffect.
	 */
	UFUNCTION(BlueprintCallable, CustomThunk, Category = "ChaosMover|Networking",
		meta = (CustomStructureParam = "EffectAsRawData", AllowAbstract = "false",
		        DisplayName = "Queue Instant Movement Effect (Authority)", DefaultToSelf = "MoverComponent"))
	static void K2_QueueInstantMovementEffect_Authority(UMoverComponent* MoverComponent, UPARAM(DisplayName = "Instant Movement Effect") const int32& EffectAsRawData);
	DECLARE_FUNCTION(execK2_QueueInstantMovementEffect_Authority);

	/**
	 * Server-only. Schedules a layered move as an authoritative sim action, far enough in the future
	 * for all endpoints to apply it on the same physics frame (no correction). Prefer this over
	 * QueueLayeredMove_Authority when frame synchronization matters (e.g. forced launches, knockbacks
	 * that must not trigger client corrections). The delay is sourced from
	 * UNetworkPhysicsSettingsComponent::EventSchedulingMinDelaySeconds, mirroring UMoverComponent's
	 * own schedule API. Requires a ChaosMover backend (UNetworkPhysicsComponent on the owner); asserts
	 * in non-shipping builds if that requirement is not met. No-ops silently if called without authority.
	 * The move struct must derive from FLayeredMoveBase.
	 */
	UFUNCTION(BlueprintCallable, CustomThunk, Category = "ChaosMover|Networking",
		meta = (CustomStructureParam = "MoveAsRawData", AllowAbstract = "false",
		        DisplayName = "Schedule Layered Move (Authority)", DefaultToSelf = "MoverComponent"))
	static void K2_ScheduleLayeredMove_Authority(UMoverComponent* MoverComponent, UPARAM(DisplayName = "Layered Move") const int32& MoveAsRawData);
	DECLARE_FUNCTION(execK2_ScheduleLayeredMove_Authority);

	/**
	 * Server-only. Schedules an instanced layered move activation as an authoritative sim action,
	 * far enough in the future for all endpoints to apply it on the same physics frame (no correction).
	 * Prefer this over QueueLayeredMoveInstance_Authority when frame synchronization matters.
	 * Requires a ChaosMover backend (UNetworkPhysicsComponent on the owner); asserts in non-shipping
	 * builds if that requirement is not met. No-ops silently if called without authority.
	 */
	UFUNCTION(BlueprintCallable, Category = "ChaosMover|Networking",
		meta = (DefaultToSelf = "MoverComponent"))
	static void ScheduleLayeredMoveInstance_Authority(UMoverComponent* MoverComponent,
		TSubclassOf<ULayeredMoveLogic> MoveLogicClass,
		UPARAM(ref) FLayeredMoveInstancedData& InstancedData);

	/**
	 * Server-only. Schedules an instant movement effect as an authoritative sim action, far enough
	 * in the future for all endpoints to apply it on the same physics frame (no correction). Prefer
	 * this over QueueInstantMovementEffect_Authority when frame synchronization matters (e.g. forced
	 * teleports that must not trigger client corrections). Requires a ChaosMover backend
	 * (UNetworkPhysicsComponent on the owner); asserts in non-shipping builds if that requirement is
	 * not met. No-ops silently if called without authority.
	 * The effect struct must derive from FInstantMovementEffect.
	 */
	UFUNCTION(BlueprintCallable, CustomThunk, Category = "ChaosMover|Networking",
		meta = (CustomStructureParam = "EffectAsRawData", AllowAbstract = "false",
		        DisplayName = "Schedule Instant Movement Effect (Authority)", DefaultToSelf = "MoverComponent"))
	static void K2_ScheduleInstantMovementEffect_Authority(UMoverComponent* MoverComponent, UPARAM(DisplayName = "Instant Movement Effect") const int32& EffectAsRawData);
	DECLARE_FUNCTION(execK2_ScheduleInstantMovementEffect_Authority);
};
