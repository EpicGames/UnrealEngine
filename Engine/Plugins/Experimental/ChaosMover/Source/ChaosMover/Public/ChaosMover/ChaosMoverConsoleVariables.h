// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace UE::ChaosMover
{
	namespace CVars
	{
		CHAOSMOVER_API extern bool bForceSingleThreadedGT;
		CHAOSMOVER_API extern bool bForceSingleThreadedPT;
		CHAOSMOVER_API extern bool bDrawGroundQueries;
		CHAOSMOVER_API extern bool bDrawOverlapQueries;
		CHAOSMOVER_API extern bool bSkipGenerateMoveIfOverridden;
		CHAOSMOVER_API extern bool bDisableResimDuplicateEventChecking;
		CHAOSMOVER_API extern int32 InstantMovementEffectIDHistorySize;
		CHAOSMOVER_API extern int32 LayeredMoveIDHistorySize;
		CHAOSMOVER_API extern int32 FrameDifferenceLeniencyForEventTimeComparison;
		CHAOSMOVER_API extern float FallingCheckRelativeSpeedLimit;

		// When true, instant movement effects queued via QueueInstantMovementEffect are networked using the sim
		// action system ("Proposed" style: each role independently detects from GetLastInputCmd() in PostSim and
		// enqueues the same action; server-authoritative, cheat-resistant).
		// When false, they are embedded in the input cmd (PreSim detection, client-authoritative).
		// IMPORTANT: when true, BP must read GetLastInputCmd() in PostSim -- NOT the PreSim input.
		// In sim actions mode the server must independently queue the same action for verification.
		// Unlike input-generating roles, the server has no access to generated input during PreSim
		// and can only read it from the simulation output via GetLastInputCmd() in PostSim,
		// introducing 1 frame of lag relative to the PreSim path.
		// To allow the client to deviate from the server within tolerance (avoiding resims for small differences),
		// override IsNearlyEqualTo on the FInstantMovementEffect subtype (see InstantMovementEffect.h).
		CHAOSMOVER_API extern bool bNetworkInstantMovementEffectsWithSimActions;

		// When true, layered moves and instanced move activations queued via QueueLayeredMove/QueueLayeredMoveInstance
		// are networked using the sim action system ("Proposed" style: each role independently detects from
		// GetLastInputCmd() in PostSim and enqueues the same action; server-authoritative, cheat-resistant).
		// When false, they are embedded in the input cmd (PreSim detection, client-authoritative).
		// IMPORTANT: when true, BP must read GetLastInputCmd() in PostSim -- NOT the PreSim input.
		// In sim actions mode the server must independently queue the same action for verification.
		// Unlike input-generating roles, the server has no access to generated input during PreSim
		// and can only read it from the simulation output via GetLastInputCmd() in PostSim,
		// introducing 1 frame of lag relative to the PreSim path.
		// To allow the client to deviate from the server within tolerance (avoiding resims for small differences),
		// override IsNearlyEqualTo on the FLayeredMoveBase or FLayeredMoveInstancedData subtype (see LayeredMove.h / LayeredMoveBase.h).
		CHAOSMOVER_API extern bool bNetworkMovesWithSimActions;

		// Temporary
		CHAOSMOVER_API extern bool bEnablePreSimGroundCheck;
		CHAOSMOVER_API extern bool bEnableServerLaunchOverride;
	}
};
