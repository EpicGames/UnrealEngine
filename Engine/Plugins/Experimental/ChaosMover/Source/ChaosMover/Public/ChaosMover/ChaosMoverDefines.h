// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#ifndef CHAOSMOVER_DEBUG_DRAW
#define CHAOSMOVER_DEBUG_DRAW (!(UE_BUILD_SHIPPING || UE_BUILD_TEST) || WITH_EDITOR)
#endif
