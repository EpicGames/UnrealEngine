// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/Declares.h"
#include "ChaosMover/ChaosMoverSimulationTypes.h"
#include "ChaosMover/PathedMovement/ChaosPathedMovementTypes.h"
#include "ChaosMover/ChaosMoverStateMachine.h"
#include "MoveLibrary/FloorQueryUtils.h"
#include "MoveLibrary/MovementMixer.h"
#include "MoveLibrary/WaterMovementUtils.h"
#include "MoverTypes.h"
#include "MoverSimulation.h"
#include "Physics/NetworkPhysicsComponent.h"
#include "ChaosMover/ChaosMoverActionTypes.h"
#include "PhysicsEngine/ConstraintInstance.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "ChaosMoverSimulation.generated.h"

namespace Chaos
{
	class FCharacterGroundConstraintHandle;
	class FPBDJointConstraintHandle;
	class FCollisionContactModifier;
}

struct FChaosMoverPredictionStateBackup
{
	TOptional<FFloorCheckResult> BlackboardLastFloorResult;
	TOptional<FWaterCheckResult> BlackboardLastWaterResult;
	TOptional<UE::ChaosMover::FGroundDynamicsInfo> BlackboardGroundDynamicsInfo;
};

UCLASS(MinimalAPI, BlueprintType)
class UChaosMoverSimulation : public UMoverSimulation, public INetworkPhysicsActionHandler_Internal
{
	GENERATED_BODY()

public:
#if !NO_LOGGING
	// Mover Gameplay Tag Logging
	FString DebugOwnerName;
	FString DebugOwnerRole;
#endif

	CHAOSMOVER_API UChaosMoverSimulation();

	// Returns the local simulation input MoverDataCollection, to read local non networked data passed to the simulation by the gameplay thread
	UFUNCTION(BlueprintPure, Category = Mover)
	CHAOSMOVER_API const FMoverDataCollection& GetLocalSimInput() const;

	// Returns the local simulation input MoverDataCollection, to pass local non networked data to the simulation
	// Only available from the gameplay thread
	UFUNCTION(BlueprintPure, Category = Mover)
	CHAOSMOVER_API FMoverDataCollection& GetLocalSimInput_Mutable();

	CHAOSMOVER_API TWeakObjectPtr<const UBaseMovementMode> GetCurrentMovementMode() const;
	CHAOSMOVER_API TWeakObjectPtr<UBaseMovementMode> GetCurrentMovementMode_Mutable();
	CHAOSMOVER_API TWeakObjectPtr<const UBaseMovementMode> FindMovementModeByName(const FName& Name) const;
	CHAOSMOVER_API TWeakObjectPtr<UBaseMovementMode> FindMovementModeByName_Mutable(const FName& Name);
	CHAOSMOVER_API bool HasMovementMode(const FName& Name) const;
	CHAOSMOVER_API void QueueNextMovementMode(FName ModeName);

	UFUNCTION(BlueprintCallable, CustomThunk, Category = Mover, meta = (CustomStructureParam = "EffectAsRawData", AllowAbstract = "false", DisplayName = "Queue Instant Movement Effect"))
	void K2_QueueInstantMovementEffect(UPARAM(DisplayName = "Instant Movement Effect") const int32& EffectAsRawData);
	DECLARE_FUNCTION(execK2_QueueInstantMovementEffect);

	// Queue an Instant Movement Effect to take place at the end of this frame or start of the next subtick - whichever happens first
	CHAOSMOVER_API void QueueInstantMovementEffect_Internal(TSharedPtr<FInstantMovementEffect> InstantMovementEffect, bool bShouldRollBack = true);
	// Queue a scheduled instant movement effect
	CHAOSMOVER_API void QueueInstantMovementEffect_Internal(const FScheduledInstantMovementEffect& ScheduledInstantMovementEffect, bool bShouldRollBack = true);

	UFUNCTION(BlueprintCallable, CustomThunk, Category = Mover, meta = (CustomStructureParam = "LayeredMoveAsRawData", AllowAbstract = "false", DisplayName = "Queue Layered Move"))
	void K2_QueueLayeredMove(UPARAM(DisplayName = "Layered Move") const int32& LayeredMoveAsRawData);
	DECLARE_FUNCTION(execK2_QueueLayeredMove);

	// Queue a Layered Move to take place at the end of this frame or start of the next subtick - whichever happens first
	CHAOSMOVER_API void QueueLayeredMove_Internal(TSharedPtr<FLayeredMoveBase> LayeredMove, bool bShouldRollBack = true);
	// Queue a scheduled layered move
	CHAOSMOVER_API void QueueLayeredMove_Internal(const FScheduledLayeredMove& ScheduledLayeredMove, bool bShouldRollBack = true);

	// Queue a FLayeredMoveInstance activation. Looks up the registered logic by class and constructs the instance.
	CHAOSMOVER_API void QueueLayeredMoveInstance_Internal(TSubclassOf<ULayeredMoveLogic> MoveLogicClass, TSharedRef<FLayeredMoveInstancedData> Data, bool bShouldRollBack = true);

	// Updates the cached snapshot of registered ULayeredMoveLogic objects. Call from the game thread
	// before each simulation tick to keep the cache current with any dynamic re-registration.
	CHAOSMOVER_API void SetRegisteredMoves(const TArray<TObjectPtr<ULayeredMoveLogic>>& Moves);

	UFUNCTION(BlueprintCallable, CustomThunk, Category = Mover, meta = (CustomStructureParam = "MoveAsRawData", AllowAbstract = "false", DisplayName = "Queue Movement Modifier"))
	CHAOSMOVER_API FMovementModifierHandle K2_QueueMovementModifier(UPARAM(DisplayName = "Movement Modifier") const int32& MoveAsRawData);
	DECLARE_FUNCTION(execK2_QueueMovementModifier);

	// Queue a Movement Modifier to start during the next simulation frame.
	CHAOSMOVER_API FMovementModifierHandle QueueMovementModifier(TSharedPtr<FMovementModifierBase> Modifier);

	// Find movement modifier by type (returns the first modifier it finds). Returns nullptr if the modifier couldn't be found
	CHAOSMOVER_API const FMovementModifierBase* FindMovementModifierByType(const UScriptStruct* DataStructType) const;

	/** Find a movement modifier of a specific type in this components movement modifiers. If not found, null will be returned. */
	template <typename ModifierT = FMovementModifierBase UE_REQUIRES(std::is_base_of_v<FMovementModifierBase, ModifierT>)>
	const ModifierT* FindMovementModifierByType() const { return static_cast<const ModifierT*>(FindMovementModifierByType(ModifierT::StaticStruct())); }


	UFUNCTION(BlueprintCallable, Category = Mover)
	CHAOSMOVER_API void CancelModifierFromHandle(FMovementModifierHandle ModifierHandle);

	UFUNCTION(BlueprintPure, Category = Mover, meta = (Keywords = "HasTag"))
	CHAOSMOVER_API bool HasGameplayTag(FGameplayTag TagToFind, bool bExactMatch) const;

	ENetRole GetLocalRole() const { return OwnerLocalRole; }
	bool IsUsingAsyncPhysics() const { return bIsUsingAsyncPhysics; }

	CHAOSMOVER_API bool ShouldResim() const;

	struct FInitParams
	{
		TMap<FName, TWeakObjectPtr<UBaseMovementMode>> ModesToRegister;
		TArray<TWeakObjectPtr<UBaseMovementModeTransition>> TransitionsToRegister;
		FMoverSyncState InitialSyncState;
		FName StartingMovementMode = NAME_None;
		TWeakObjectPtr<UNullMovementMode> NullMovementMode = nullptr;
		TWeakObjectPtr<UImmediateMovementModeTransition> ImmediateModeTransition = nullptr;
		TWeakObjectPtr<UMovementMixer> MovementMixer = nullptr;
		FTransform TransformOnInit = FTransform::Identity;
		Chaos::FCharacterGroundConstraintHandle* CharacterGroundConstraintHandle = nullptr;
		Chaos::FPBDJointConstraintHandle* ActuationConstraintHandle = nullptr;
		Chaos::FKinematicGeometryParticleHandle* ActuationConstraintEndPointParticleHandle = nullptr;
		Chaos::FConstPhysicsObjectHandle PhysicsObject = nullptr;
		Chaos::FPhysicsSolver* Solver = nullptr;
		UWorld* World = nullptr;
		bool bIsUsingAsyncPhysics = true;
	};

	CHAOSMOVER_API void Init(const FInitParams& InitParams);
	CHAOSMOVER_API void SetOwner(const AActor* Owner);
	CHAOSMOVER_API void Deinit();

	CHAOSMOVER_API void ProcessInputs(int32 PhysicsStep, const FMoverTimeStep& TimeStep, const UE::ChaosMover::FSimulationInputData& InputData);
	CHAOSMOVER_API void SimulationTick(const FMoverSimContext& SimContext, const FMoverTimeStep& TimeStep, const UE::ChaosMover::FSimulationInputData& InputData, UE::ChaosMover::FSimulationOutputData& OutputData);
	CHAOSMOVER_API void ModifyContacts(const FMoverTimeStep& TimeStep, const UE::ChaosMover::FSimulationInputData& InputData, const UE::ChaosMover::FSimulationOutputData& OutputData, Chaos::FCollisionContactModifier& Modifier);

	CHAOSMOVER_API void AddEvent(TSharedPtr<FMoverSimulationEventData> Event);
	CHAOSMOVER_API const TArray<TSharedPtr<FMoverSimulationEventData>>& GetEvents() const;

	// INetworkPhysicsActionHandler_Internal: receives scheduled actions enqueued via UNetworkPhysicsComponent::EnqueueScheduledAction_External.
	// Dispatches to typed handlers based on the action payload type.
	CHAOSMOVER_API virtual void ApplyAction_Internal(const TInstancedStruct<FNetworkPhysicsActionPayload>& ActionInstance) override;

	CHAOSMOVER_API void ApplyNetInputData(const FMoverInputCmdContext& InNetInputCmd);
	CHAOSMOVER_API void BuildNetInputData(FMoverInputCmdContext& OutNetInputCmd) const;
	CHAOSMOVER_API void ApplyNetStateData(const FMoverSyncState& InNetSyncState);
	CHAOSMOVER_API void BuildNetStateData(FMoverSyncState& OutNetSyncState) const;

	// Movement modes can be relative to a basis transform, which can change at runtime
	// Returns the movement basis transform, for movement relative to it
	CHAOSMOVER_API virtual const FTransform& GetMovementBasisTransform() const;
	// Sets the movement basis transform, for movement relative to it
	CHAOSMOVER_API virtual void SetMovementBasisTransform(const FTransform& InMovementBasisTransform);

	//~ Debugging Util functions
	// Collection for holding extra debug data, that will be sent to the Chaos Visual Debugger for debugging
	UFUNCTION(BlueprintPure, Category = Mover)
	CHAOSMOVER_API FMoverDataCollection& GetDebugSimData();
	//~ End of Debugging Util functions

protected:
	CHAOSMOVER_API virtual void OnInit();
	CHAOSMOVER_API virtual void OnDeinit();
	CHAOSMOVER_API virtual void OnPreSimulationTick(const FMoverTimeStep& TimeStep, const UE::ChaosMover::FSimulationInputData& InputData);
	CHAOSMOVER_API virtual void OnSimulationTick(const FMoverSimContext& SimContext, const FMoverTimeStep& TimeStep, const UE::ChaosMover::FSimulationInputData& InputData, UE::ChaosMover::FSimulationOutputData& OutputData);
	CHAOSMOVER_API virtual void OnPostSimulationTick(const FMoverTimeStep& TimeStep, UE::ChaosMover::FSimulationOutputData& OutputData);
	CHAOSMOVER_API virtual void OnModifyContacts(const FMoverTimeStep& TimeStep, const UE::ChaosMover::FSimulationInputData& InputData, const UE::ChaosMover::FSimulationOutputData& OutputData, Chaos::FCollisionContactModifier& Modifier);
	CHAOSMOVER_API virtual void OnSimulationRollback(const FMoverTimeStep& NewTimeStep, const FMoverSyncState& PrevSyncState);

	CHAOSMOVER_API virtual void ProcessSimulationEvent(const FMoverSimulationEventData& ModeChangedData);
	CHAOSMOVER_API virtual void OnMovementModeChanged(const FMovementModeChangedEventData& ModeChangedData);

	// Trajectory Prediction
	CHAOSMOVER_API void BackupState(FChaosMoverPredictionStateBackup& Out) const;
	CHAOSMOVER_API void RestoreState(const FChaosMoverPredictionStateBackup& In);
	CHAOSMOVER_API void PredictTrajectory(const FMoverTimeStep& TimeStep, const UE::ChaosMover::FSimulationInputData& InputData, const FChaosMoverTrajectoryPredictionInputs* PredInputs, const FMoverSyncState& StartSyncState, UE::ChaosMover::FSimulationOutputData& OutputData) const;

	// Teleportation (internal functions)
public:
	// Attempt to teleport. This will first check CanTeleport and broadcast simulation events to indicate success or failure.
	CHAOSMOVER_API virtual void AttemptTeleport(const FMoverTimeStep& TimeStep, const FTransform& TargetTransform, bool bUseActorRotation, FMoverSyncState& OutputState) override;
protected:
	// Whether the teleport can happen (does the movement mode allow it? Do we fit at the target transform?)
	CHAOSMOVER_API virtual bool CanTeleport(const FTransform& TargetTransform, bool bUseActorRotation, const FMoverSyncState& SyncState);
	// Actually perform the teleport
	CHAOSMOVER_API virtual void Teleport(const FTransform& TargetTransform, const FMoverTimeStep& TimeStep, FMoverSyncState& OutputState);

public:
	// Trace mover data to Chaos Visual Debugger
	CHAOSMOVER_API void TraceMoverData(const FMoverTimeStep& TimeStep, const UE::ChaosMover::FSimulationOutputData& OutputData);

	const FMoverSyncState& GetCurrentSyncState() const { return CurrentSyncState; }

	// Physics state accessors -- callable from IChaosPostSimulationTickInterface implementations.
	CHAOSMOVER_API Chaos::FPBDRigidParticleHandle* GetControlledParticle() const;
	CHAOSMOVER_API Chaos::FCharacterGroundConstraintHandle* GetCharacterGroundConstraintHandle() const;
	CHAOSMOVER_API Chaos::FPBDJointConstraintHandle* GetActuationConstraintHandle() const;
	CHAOSMOVER_API bool IsNonResimSimProxy() const;

	// Object state -- callable from IChaosPostSimulationTickInterface implementations
	// (e.g. executor classes that need to switch the particle between dynamic and kinematic).
	CHAOSMOVER_API void SetControlledParticleDynamic();
	CHAOSMOVER_API void SetControlledParticleKinematic();
	CHAOSMOVER_API bool IsControlledParticleDynamic() const;
	CHAOSMOVER_API bool IsControlledParticleKinematic() const;

	// Constraint enable helpers -- called by IChaosPostSimulationTickInterface implementations
	// to re-enable the constraint after the simulation disables it upfront each tick.
	CHAOSMOVER_API void EnableCharacterGroundConstraint();
	CHAOSMOVER_API void EnableActuationConstraint();

	// Target transform for the actuation joint -- callable from pathed movement mode.
	CHAOSMOVER_API void SetActuationTargetTransform(const FTransform& TargetTransform);

protected:
	// Character ground constraint
	CHAOSMOVER_API void DisableCharacterGroundConstraint();
	CHAOSMOVER_API bool IsCharacterGroundConstraintEnabled() const;

	// General purpose actuation joint constraint
	CHAOSMOVER_API void DisableActuationConstraint();
	CHAOSMOVER_API bool IsActuationConstraintEnabled() const;
	CHAOSMOVER_API void TeleportActuationTarget(const FTransform& TargetTransform, bool AlsoTeleportControlledParticle = false);
	
	// State structs
	FMoverSyncState CurrentSyncState;
	// Data internal to the simulation
	FMoverDataCollection InternalSimData;
	// Local input data, usually sent by the gameplay side locally, that is not expected to differ from that on the server so doesn't warrant networking
	FMoverDataCollection LocalSimInput;
	// Debug Data collection, sent to Chaos Visual Debugger when Trace Extra Sim Debug Data is selected
	FMoverDataCollection DebugSimData;

	// Movement mode state machine
	UE::ChaosMover::FMoverStateMachine StateMachine;

	// Optional movement mixer
	TWeakObjectPtr<UMovementMixer> MovementMixerWeakPtr = nullptr;

	// Controlled physics object
	Chaos::FConstPhysicsObjectHandle PhysicsObject = nullptr;

	// Character ground constraint, specifically for moving on ground like characters
	Chaos::FCharacterGroundConstraintHandle* CharacterGroundConstraintHandle = nullptr;

	// General purpose actuation joint constraint
	Chaos::FPBDJointConstraintHandle* ActuationConstraintHandle = nullptr;
	Chaos::FKinematicGeometryParticleHandle* ActuationConstraintEndPointParticleHandle= nullptr;
	
	// Some movement modes are relative to a movement basis transform, stored here
	FTransform MovementBasisTransform = FTransform::Identity;

	Chaos::FPhysicsSolver* Solver = nullptr;
	UWorld* World = nullptr;

private:
	void UpdateEvents(const FMoverTimeStep& TimeStep, UE::ChaosMover::FSimulationOutputData& OutputData);
	FChaosMoverPostSimContext BuildPostSimContext(const FMoverTimeStep& TimeStep, UE::ChaosMover::FSimulationOutputData& OutputData);

	FMoverInputCmdContext NetInputCmd;
	FMoverSyncState NetSyncState;

	// Events from this simulation update
	TArray<TSharedPtr<FMoverSimulationEventData>> Events;

	// Dedup record of events sent to the GT during the forward sim, used to suppress duplicate
	// re-emission during resimulation. Only contains events with bReEmitOnResim=false; events
	// that opt into re-emission are never added here and always pass through resim unchanged.
	TArray<TSharedPtr<FMoverSimulationEventData>> ProcessedEvents;

	bool bInputCmdOverridden = false;
	bool bSyncStateOverridden = false;
	bool bIsUsingAsyncPhysics = true;

	int32 InternalServerFrame = 0;

	bool bInitialized = false;
	ENetRole OwnerLocalRole = ENetRole::ROLE_None;
};
