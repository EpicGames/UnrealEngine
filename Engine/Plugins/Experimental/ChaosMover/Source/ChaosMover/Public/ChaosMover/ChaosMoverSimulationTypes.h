// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/Declares.h"
#include "Chaos/ImplicitObject.h"
#include "Chaos/PhysicsObject.h"
#include "CollisionQueryParams.h"
#include "DefaultMovementSet/MovementModifiers/StanceModifier.h"
#include "MoveLibrary/FloorQueryUtils.h"
#include "MoveLibrary/MovementUtils.h"
#include "MoveLibrary/WaterMovementUtils.h"
#include "MoverSimulationTypes.h"
#include "LayeredMove.h"
#include "StructUtils/InstancedStruct.h"

#include "ChaosMoverSimulationTypes.generated.h"

namespace Chaos
{
	class FPBDJointSettings;
	class FCharacterGroundConstraintSettings;
	class FCharacterGroundConstraintHandle;
	class FPBDJointConstraintHandle;
}

struct FConstraintProfileProperties;
class FAsyncNetworkPhysicsComponent;
class UChaosMoverSimulation;
class UBaseMovementMode;
class UMoverBlackboard;
class UMoverSimulation;

namespace UE::ChaosMover
{
	struct FSimulationInputData
	{
		void Reset()
		{
			InputCmd.Reset();
			AuxInputState.AuxStateCollection.Empty();
		}

		mutable FMoverInputCmdContext InputCmd; // Can be overridden by network physics
		FMoverAuxStateContext AuxInputState; // Optional aux input state. Not networked
	};

	using FSimulationOutputData = ::UE::Mover::FSimulationOutputData;

	/** Mode change event structure, used to postpone callbacks to gameplay code when a mode has changed */
	struct FMovementModeChangeEvent
	{
		FName PreviousMovementModeName;
		TWeakObjectPtr<UBaseMovementMode> PreviousMovementMode;
		FName NextMovementModeName;
		TWeakObjectPtr<UBaseMovementMode> NextMovementMode;
	};

	// Util function to be able to get the debug sim data collection from a UChaosMoverSimulation from another plugin,
	// without including ChaosMoverSimulation.h
	CHAOSMOVER_API FMoverDataCollection& GetDebugSimData(UChaosMoverSimulation* Simulation);

	namespace Blackboard
	{
		extern CHAOSMOVER_API const FName GroundDynamicsInfo;
	}

	struct FGroundDynamicsInfo
	{
		CHAOSMOVER_API FGroundDynamicsInfo();
		CHAOSMOVER_API FGroundDynamicsInfo(const FFloorCheckResult& InFloorResult);
		FVector LinearVelocity;
		FVector AngularVelocityDegrees;
		uint32 bIsMoving : 1;
		uint32 bIsDynamic : 1;
		uint32 bIsGravityEnabled : 1;
	};

} // namespace UE::ChaosMover

// Predicted trajectory deltas generated during simulation tick.
// Stored in FSimulationOutputData::AdditionalOutputData (not networked, local only).
USTRUCT(BlueprintType)
struct FChaosMoverPredictedTrajectoryData : public FMoverDataStructBase
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadWrite, Category = ChaosMover)
	FTransform BaseTransform = FTransform::Identity;

	// Predicted trajectory as delta transforms relative to each preceding step.
	// Element [0] is the starting transform (time offset 0, Identity), i.e., anchored at BaseTransform.
	UPROPERTY(BlueprintReadWrite, Category = ChaosMover)
	TArray<FTrajectorySampleInfo> Deltas;

	virtual UScriptStruct* GetScriptStruct() const override
	{
		return StaticStruct();
	}

	virtual FMoverDataStructBase* Clone() const override
	{
		return new FChaosMoverPredictedTrajectoryData(*this);
	}

	virtual bool ShouldReconcile(const FMoverDataStructBase& AuthorityState) const override
	{
		return false;
	}

	virtual void Interpolate(const FMoverDataStructBase& From, const FMoverDataStructBase& To, float Pct) override
	{
		// Always take the latest trajectory rather than blending so that downstream code can be responsive to predictions
		*this = static_cast<const FChaosMoverPredictedTrajectoryData&>(To);
	}

	virtual void Merge(const FMoverDataStructBase& From) override {}
};

template<>
struct TStructOpsTypeTraits<FChaosMoverPredictedTrajectoryData> : public TStructOpsTypeTraitsBase2<FChaosMoverPredictedTrajectoryData>
{
	enum
	{
		WithCopy = true
	};
};

// Local-only inputs that control how ChaosMover generates predicted trajectory samples.
USTRUCT(BlueprintType)
struct FChaosMoverTrajectoryPredictionInputs : public FMoverDataStructBase
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = ChaosMover, meta = (ClampMin = "1", UIMin = "1"))
	int32 NumPredictionSteps = 15;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = ChaosMover, meta = (ClampMin = "0.001", UIMin = "0.001"))
	float SecondsPerStep = 0.1f;

	virtual UScriptStruct* GetScriptStruct() const override
	{
		return StaticStruct();
	}

	virtual FMoverDataStructBase* Clone() const override
	{
		return new FChaosMoverTrajectoryPredictionInputs(*this);
	}

	virtual bool ShouldReconcile(const FMoverDataStructBase& AuthorityState) const override
	{
		return false;
	}

	virtual void Interpolate(const FMoverDataStructBase& From, const FMoverDataStructBase& To, float Pct) override
	{
		*this = static_cast<const FChaosMoverTrajectoryPredictionInputs&>((Pct < 0.5f) ? From : To);
	}

	virtual void Merge(const FMoverDataStructBase& From) override {}
};

template<>
struct TStructOpsTypeTraits<FChaosMoverTrajectoryPredictionInputs> : public TStructOpsTypeTraitsBase2<FChaosMoverTrajectoryPredictionInputs>
{
	enum
	{
		WithCopy = true
	};
};

// Mover ground state, holds movement properties relative to the ground
USTRUCT(BlueprintType)
struct FChaosMoverGroundSimState : public FMoverDataStructBase
{
	GENERATED_BODY()

	/** return newly allocated copy of this FMoverDataStructBase. MUST be overridden by derived types. */
	virtual FMoverDataStructBase* Clone() const override
	{
		return new FChaosMoverGroundSimState(*this);
	}

	/** Gets the type info of this FMoverDataStructBase. MUST be overridden by derived types. */
	virtual UScriptStruct* GetScriptStruct() const override
	{
		return StaticStruct();
	}

	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category = ChaosMover)
	FVector LocalVelocity = FVector::ZeroVector;
};

// Movement basis state, for any movement that is relative to a basis transform given in world coordinates
USTRUCT(BlueprintType)
struct FChaosMovementBasis : public FMoverDataStructBase
{
	GENERATED_BODY()

	// Implementation of FMoverDataStructBase
	CHAOSMOVER_API virtual bool NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess) override;
	CHAOSMOVER_API virtual FMoverDataStructBase* Clone() const override;
	CHAOSMOVER_API virtual UScriptStruct* GetScriptStruct() const override;
	CHAOSMOVER_API virtual void ToString(FAnsiStringBuilderBase& Out) const override;
	CHAOSMOVER_API virtual bool ShouldReconcile(const FMoverDataStructBase& AuthorityState) const override;
	CHAOSMOVER_API virtual void Interpolate(const FMoverDataStructBase& From, const FMoverDataStructBase& To, float Pct) override;
	CHAOSMOVER_API virtual void Merge(const FMoverDataStructBase& From) override;

	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category = ChaosMover)
	FVector BasisLocation = FVector::ZeroVector;

	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category = ChaosMover)
	FQuat BasisRotation = FQuat::Identity;
};

// Data block containing all default inputs required by the Chaos Mover simulation
USTRUCT(BlueprintType)
struct FChaosMoverSimulationDefaultInputs : public FMoverDataStructBase
{
	GENERATED_BODY()

	FChaosMoverSimulationDefaultInputs()
	{
		Reset();
	}

	CHAOSMOVER_API void Reset();

	/** return newly allocated copy of this FMoverDataStructBase. MUST be overridden by derived types. */
	virtual FMoverDataStructBase* Clone() const override
	{
		return new FChaosMoverSimulationDefaultInputs(*this);
	}

	/** Gets the type info of this FMoverDataStructBase. MUST be overridden by derived types. */
	virtual UScriptStruct* GetScriptStruct() const override
	{
		return StaticStruct();
	}

	FCollisionResponseParams CollisionResponseParams;
	FCollisionQueryParams CollisionQueryParams;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category=ChaosMover)
	FVector UpDir;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category=ChaosMover)
	FVector Gravity;

	// True if inputs are generated locally for this Actor
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category=ChaosMover)
	bool bIsGeneratingInputsLocally = false;

	// True if the Actor is a pawn, has a controller but that controller is not local
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category=ChaosMover)
	bool bIsRemotelyControlled = false;

	Chaos::FPhysicsObjectHandle PhysicsObject;
	AActor* OwningActor;
	UWorld* World;
	FAsyncNetworkPhysicsComponent* AsyncNetworkPhysicsComponent;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category=ChaosMover)
	float PhysicsObjectGravity;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category=ChaosMover)
	float PawnCollisionHalfHeight;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category=ChaosMover)
	float PawnCollisionRadius;

	// Relative transform of the primary visual (skeletal mesh) component with respect to the actor root.
	// Populated each physics frame by the backend from UMoverComponent::GetBaseVisualComponentTransform().
	// Used by async layered moves (e.g. AnimRootMotion) to convert mesh-local root motion into world space
	// without touching game-thread-only objects.
	UPROPERTY(VisibleAnywhere, Category = ChaosMover)
	FTransform PrimaryVisualComponentRelativeTransform = FTransform::Identity;

	UPROPERTY(VisibleAnywhere, Category = ChaosMover)
	TEnumAsByte<ECollisionChannel> CollisionChannel = ECC_WorldDynamic;
};

// This will be replaced eventually by the Chaos Visual Debugger supporting the display of this information
USTRUCT()
struct FChaosMoverTimeStepDebugData : public FMoverDataStructBase
{
	GENERATED_BODY()

	virtual FMoverDataStructBase* Clone() const override;
	virtual UScriptStruct* GetScriptStruct() const override;

	CHAOSMOVER_API void SetTimeStep(const FMoverTimeStep& InTimeStep);

	// This is so CVD can display TimeStep.bIsResimulating properly, which does not exist in FMoverTimeStep in a way that will show up by default (bitflag)
	UPROPERTY(VisibleAnywhere, Category = "Mover Info")
	bool bIsResimulating = false;

	// This is so CVD can display TimeStep.bIsFirstResimFrame properly, which does not exist in FMoverTimeStep in a way that will show up by default (bitflag)
	UPROPERTY(VisibleAnywhere, Category = "Mover Info")
	bool bIsFirstResimFrame = false;

	UPROPERTY(VisibleAnywhere, Category = "Mover Info")
	FMoverTimeStep TimeStep;
};

// Structure for tracing some network physics info to CVD
USTRUCT()
struct FNetworkPhysicsDebugData : public FMoverDataStructBase
{
	GENERATED_BODY()

	virtual FMoverDataStructBase* Clone() const override;
	virtual UScriptStruct* GetScriptStruct() const override;

	CHAOSMOVER_API void Set(const FAsyncNetworkPhysicsComponent* NetworkPhysicsComponent);

	UPROPERTY(VisibleAnywhere, Category = "Mover Info")
	bool bIsLocallyControlled = false;

	UPROPERTY(VisibleAnywhere, Category = "Mover Info")
	EPhysicsReplicationMode PhysicsReplicationMode = EPhysicsReplicationMode::Default;

	UPROPERTY(VisibleAnywhere, Category = "Mover Info")
	int32 NetworkPhysicsTickOffset = INDEX_NONE;

	UPROPERTY(VisibleAnywhere, Category = "Mover Info")
	int32 LatestReceivedStateFrame = INDEX_NONE;

	UPROPERTY(VisibleAnywhere, Category = "Mover Info")
	float ForwardPredictionTime = -1.0f;

	UPROPERTY(VisibleAnywhere, Category = "Mover Info")
	float CurrentSimProxyInputDecayAtRuntime = -1.0f;

	UPROPERTY(VisibleAnywhere, Category = "Mover Info")
	float CurrentInputDecay = -1.0f;
};

// CVD-only debug wrapper for simulation events emitted during a substep. Stored in DebugSimData.
// Each element holds a full copy of the concrete event type with all UPROPERTY fields visible.
USTRUCT()
struct FChaosMoverSimulationEventsDebugData : public FMoverDataStructBase
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category = "Mover Info")
	TArray<FInstancedStruct> Events;

	virtual FMoverDataStructBase* Clone() const override
	{
		return new FChaosMoverSimulationEventsDebugData(*this);
	}

	virtual UScriptStruct* GetScriptStruct() const override
	{
		return StaticStruct();
	}
};

template<>
struct TStructOpsTypeTraits<FChaosMoverSimulationEventsDebugData> : public TStructOpsTypeTraitsBase2<FChaosMoverSimulationEventsDebugData>
{
	enum { WithCopy = true };
};

USTRUCT(BlueprintType)
struct FChaosWaterResultData : public FMoverDataStructBase
{
	GENERATED_BODY()

public:
	UPROPERTY(BlueprintReadWrite, Category = Mover)
	FWaterCheckResult WaterResult;

	FChaosWaterResultData() = default;
	virtual ~FChaosWaterResultData() = default;

	virtual FMoverDataStructBase* Clone() const override
	{
		return new FChaosWaterResultData(*this);
	}

	virtual UScriptStruct* GetScriptStruct() const override
	{
		return StaticStruct();
	}

	virtual void ToString(FAnsiStringBuilderBase& Out) const override;
	virtual bool ShouldReconcile(const FMoverDataStructBase& AuthorityState) const override;
	virtual void Interpolate(const FMoverDataStructBase& From, const FMoverDataStructBase& To, float Pct) override;
	virtual void Merge(const FMoverDataStructBase& From) override;
	virtual void Decay(float DecayAmount) override;
};

UINTERFACE(BlueprintType, MinimalAPI, meta = (CannotImplementInterfaceInBlueprint))
class UChaosCharacterMovementModeInterface : public UInterface
{
	GENERATED_BODY()
};

class IChaosCharacterMovementModeInterface
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category=ChaosMover)
	virtual float GetTargetHeight() const = 0;

	UFUNCTION(BlueprintCallable, Category = ChaosMover)
	virtual float GetGroundQueryRadius() const = 0;

	UFUNCTION(BlueprintCallable, Category = ChaosMover)
	virtual float GetMaxWalkSlopeCosine() const = 0;

	UFUNCTION(BlueprintCallable, Category = ChaosMover)
	virtual bool ShouldCharacterRemainUpright() const = 0;

	UFUNCTION(BlueprintCallable, Category = ChaosMover)
	virtual float GetMaxSpeed() const = 0;

	UFUNCTION(BlueprintCallable, Category = ChaosMover)
	virtual void OverrideMaxSpeed(float Value) = 0;

	UFUNCTION(BlueprintCallable, Category = ChaosMover)
	virtual void ClearMaxSpeedOverride() = 0;

	UFUNCTION(BlueprintCallable, Category = ChaosMover)
	virtual float GetAcceleration() const = 0;

	UFUNCTION(BlueprintCallable, Category = ChaosMover)
	virtual void OverrideAcceleration(float Value) = 0;

	UFUNCTION(BlueprintCallable, Category = ChaosMover)
	virtual void ClearAccelerationOverride() = 0;

	UFUNCTION(BlueprintCallable, Category = ChaosMover)
	virtual void UpdateCurrentFloor(const FMoverTimeStep& TimeStep) const = 0;
};

// Interface for mover modes moving on ground like characters, using a character ground constraint
UINTERFACE(MinimalAPI)
class UChaosCharacterConstraintMovementModeInterface : public UInterface
{
	GENERATED_BODY()
};

class IChaosCharacterConstraintMovementModeInterface
{
	GENERATED_BODY()

public:
	virtual float GetTargetHeight() const = 0;
	virtual bool ShouldEnableConstraint() const = 0;
	virtual void UpdateConstraintSettings(Chaos::FCharacterGroundConstraintSettings& ConstraintSettings) const = 0;
};

/**
 * Context passed by mutable reference to all IChaosPreSimulationTickInterface
 * implementations during OnPreSimulationTick. Filled once per tick by the
 * simulation; implementations perform mode-specific pre-simulation setup
 * (floor sweeps, input processing, etc.) via the Simulation pointer.
 */
struct FChaosMoverPreSimContext
{
	FMoverTimeStep                              TimeStep;
	const UE::ChaosMover::FSimulationInputData& InputData;
	UMoverSimulation*                           Simulation = nullptr;
};

UINTERFACE(MinimalAPI)
class UChaosPreSimulationTickInterface : public UInterface
{
	GENERATED_BODY()
};

/**
 * Interface for pre-simulation tick implementations. Attach this to any
 * UChaosMovementMode or UChaosMoveExecutorBase to opt into the generic
 * OnPreSimulationTick dispatch -- no simulation changes needed.
 *
 * CollectSimulationInterfaces on UChaosMovementMode / UChaosCompositeMovementMode
 * discovers implementors via Cast<> and registers them in
 * FChaosMoverSimulationInterfaceCache::PreSimInterfaces.
 *
 * The simulation calls PreSimulationTick_Async for every collected interface
 * after updating the sync state from the physics particle each tick.
 */
class IChaosPreSimulationTickInterface
{
	GENERATED_BODY()
public:
	virtual void PreSimulationTick_Async(FChaosMoverPreSimContext& Context) = 0;
};

/**
 * Context passed by mutable reference to all IChaosPostSimulationTickInterface
 * implementations during OnPostSimulationTick. Filled once per tick by the
 * simulation; implementations read physics state and write results back to
 * OutputData via the Simulation pointer.
 *
 * The fallback velocity application is skipped whenever any interface was
 * dispatched. Set bApplyFallbackVelocity = true if the implementation does not
 * apply particle velocity itself and still wants the fallback to fire.
 */
struct FChaosMoverPostSimContext
{
	FMoverTimeStep                         TimeStep;
	UE::ChaosMover::FSimulationOutputData& OutputData;
	UMoverSimulation*                      Simulation = nullptr;
	// Set to true to request the simulation's fallback velocity application even when
	// an IChaosPostSimulationTickInterface implementation was dispatched. Leave false
	// (the default) if the implementation handles particle velocity itself.
	bool                                   bApplyFallbackVelocity = false;
};

UINTERFACE(MinimalAPI)
class UChaosPostSimulationTickInterface : public UInterface
{
	GENERATED_BODY()
};

/**
 * Interface for post-simulation tick implementations. Attach this to any
 * UChaosMovementMode or UChaosMoveExecutorBase to opt into the generic
 * OnPostSimulationTick dispatch -- no simulation changes needed.
 *
 * CollectSimulationInterfaces on UChaosMovementMode / UChaosCompositeMovementMode
 * discovers implementors via Cast<> and registers them in
 * FChaosMoverSimulationInterfaceCache::PostSimInterfaces.
 *
 * The simulation calls PostSimulationTick_Async for every collected interface,
 * then applies a fallback SetV unless at least one interface was dispatched
 * (or an implementation sets Context.bApplyFallbackVelocity = true).
 */
class IChaosPostSimulationTickInterface
{
	GENERATED_BODY()
public:
	virtual void PostSimulationTick_Async(FChaosMoverPostSimContext& Context) = 0;
};

/**
 * Holds pre- and post-simulation interface pointers for the active movement mode,
 * populated by UChaosMovementMode::CollectSimulationInterfaces each tick.
 */
struct FChaosMoverSimulationInterfaceCache
{
	TArray<IChaosPreSimulationTickInterface*,  TInlineAllocator<2>> PreSimInterfaces;
	TArray<IChaosPostSimulationTickInterface*, TInlineAllocator<2>> PostSimInterfaces;
};

UENUM(BlueprintType)
enum class EChaosMoverVelocityEffectMode : uint8
{
	/** Apply as an additive impulse*/
	Impulse,

	/** Apply as an additive velocity */
	AdditiveVelocity,

	/** Apply as an override velocity */
	OverrideVelocity,
};

USTRUCT()
struct FStanceModifiedEventData : public FMoverSimulationEventData
{
	GENERATED_BODY()

	FStanceModifiedEventData(const FMoverTimeStep& InEventTime, EStanceMode InOldStance, EStanceMode InNewStance, FEventProcessedCallbackPtr InEventProcessedCallback = nullptr)
		: FMoverSimulationEventData(InEventTime, InEventProcessedCallback)
		, OldStance(InOldStance)
		, NewStance(InNewStance)
	{
		bReEmitOnResim = true;
	}
	FStanceModifiedEventData()
	{
		bReEmitOnResim = true;
	}

	virtual UScriptStruct* GetScriptStruct() const override
	{
		return FStanceModifiedEventData::StaticStruct();
	}

	virtual bool IsEqual(const FMoverSimulationEventData& Other, int32 ServerFrameDiffTolerance = 0) const override
	{
		if (Super::IsEqual(Other, ServerFrameDiffTolerance))
		{
			const FStanceModifiedEventData& TypedOther = static_cast<const FStanceModifiedEventData&>(Other);
			return (OldStance == TypedOther.OldStance) && (NewStance == TypedOther.NewStance);
		}
		else
		{
			return false;
		}
	}

	UPROPERTY(VisibleAnywhere, Category = Mover)
	EStanceMode OldStance = EStanceMode::Invalid;

	UPROPERTY(VisibleAnywhere, Category = Mover)
	EStanceMode NewStance = EStanceMode::Invalid;
};

// WithCopy = true: inherited std::function in FMoverSimulationEventData requires operator= for safe copying via FInstancedStruct.
template<>
struct TStructOpsTypeTraits<FStanceModifiedEventData> : public TStructOpsTypeTraitsBase2<FStanceModifiedEventData>
{
	enum { WithCopy = true };
};

// Version of a FScheduledInstantMovementEffect bShouldRollBack flag
USTRUCT()
struct FChaosScheduledInstantMovementEffect
{
	GENERATED_BODY()

	// Whether this effect should rollback or not
	// We do not net serialize this value since any effect received from the network should always be rolled back
	// Only effects that were issued locally by the game thread should NOT roll back since resimulation does not run 
	// game thread logic again and will fail to generate them again
	UPROPERTY(VisibleAnywhere, Category = "Mover")
	bool bShouldRollBack = true;

	UPROPERTY(VisibleAnywhere, Category = ChaosMover)
	FScheduledInstantMovementEffect ScheduledEffect;
};

// Version of a FScheduledInstantMovementEffect with networkable instanced struct instead
USTRUCT()
struct FChaosNetInstantMovementEffect
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category = "Mover")
	int32 IssuanceServerFrame = INDEX_NONE;

	UPROPERTY(VisibleAnywhere, Category = ChaosMover)
	int32 ExecutionServerFrame = INDEX_NONE;

	UPROPERTY(VisibleAnywhere, Category = ChaosMover)
	uint8 UniqueID = 0xFF;

	// Whether this effect should rollback or not
	// We do not net serialize this value since any effect received from the network should always be rolled back
	// Only effects that were issued locally by the game thread should NOT roll back since resimulation does not run 
	// game thread logic again and will fail to generate them again
	UPROPERTY(VisibleAnywhere, Category = "Mover")
	bool bShouldRollBack = true;

	// If properties are added, FChaosNetInstantMovementEffectsQueue::NetSerialize should be updated to serialize them if necessary

	FScheduledInstantMovementEffect AsScheduledInstantMovementEffect() const
	{
		return FScheduledInstantMovementEffect(
			FMoverSchedulingInfo(FMoverTime(IssuanceServerFrame, /* TimeMs = */ 0.0), FMoverTime(ExecutionServerFrame, /* TimeMs = */ 0.0), /* bIsFixedDt = */ true),
			TSharedPtr<FInstantMovementEffect>(ensure(Effect.IsValid()) ? Effect.Get().Clone() : nullptr));
	}

	UPROPERTY(VisibleAnywhere, Category = ChaosMover)
	TInstancedStruct<FInstantMovementEffect> Effect;
};

USTRUCT()
struct FChaosNetInstantMovementEffectsQueue : public FMoverDataStructBase
{
	GENERATED_BODY()
	
	UPROPERTY(VisibleAnywhere, Category = ChaosMover)
	TArray<FChaosNetInstantMovementEffect> Effects;

	void Add(const FScheduledInstantMovementEffect& Effect, bool bShouldRollBack, uint8 UniqueID);

	// This function is used for debugging (sending to CVD)
	FChaosNetInstantMovementEffectsQueue& operator=(const TArray<FChaosScheduledInstantMovementEffect>& ScheduledInstantEffects);

	virtual FMoverDataStructBase* Clone() const override;
	virtual bool NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess) override;
	virtual UScriptStruct* GetScriptStruct() const override;
	virtual void ToString(FAnsiStringBuilderBase& Out) const override;
	virtual bool ShouldReconcile(const FMoverDataStructBase& AuthorityState) const override;
	virtual void Interpolate(const FMoverDataStructBase& From, const FMoverDataStructBase& To, float Pct) override;
	virtual void Merge(const FMoverDataStructBase& From) override;
	virtual void Decay(float DecayAmount) override;
};

// Version of a FScheduledLayeredMove with bShouldRollBack flag
USTRUCT()
struct FChaosScheduledLayeredMove
{
	GENERATED_BODY()

	// Whether this move should rollback or not
	// We do not net serialize this value since any move received from the network should always be rolled back
	// Only moves issued locally by the game thread should NOT roll back since resimulation does not run 
	// game thread logic again and will fail to generate them again
	UPROPERTY(VisibleAnywhere, Category = "Mover")
	bool bShouldRollBack = true;

	UPROPERTY(VisibleAnywhere, Category = ChaosMover)
	FScheduledLayeredMove ScheduledLayeredMove;
};

USTRUCT()
struct FChaosNetLayeredMove
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category = "Mover")
	int32 IssuanceServerFrame = INDEX_NONE;

	UPROPERTY(VisibleAnywhere, Category = ChaosMover)
	int32 ExecutionServerFrame = INDEX_NONE;

	UPROPERTY(VisibleAnywhere, Category = ChaosMover)
	uint8 UniqueID = 0xFF;

	// Whether this move should roll back or not
	// We do not net serialize this value since any move received from the network should always be rolled back
	// Only moves that were issued locally by the game thread should NOT roll back since resimulation does not run 
	// game thread logic again and will fail to generate them again
	UPROPERTY(VisibleAnywhere, Category = "Mover")
	bool bShouldRollBack = true;

	FScheduledLayeredMove AsScheduledLayeredMove() const
	{
		return FScheduledLayeredMove(
			FMoverSchedulingInfo(FMoverTime(IssuanceServerFrame, /* TimeMs = */ 0.0), FMoverTime(ExecutionServerFrame, /* TimeMs = */ 0.0), /* bIsFixedDt = */ true),
			TSharedPtr<FLayeredMoveBase>(ensure(Move.IsValid()) ? Move.Get().Clone() : nullptr));
	}

	UPROPERTY(VisibleAnywhere, Category = ChaosMover)
	TInstancedStruct<FLayeredMoveBase> Move;
};

USTRUCT()
struct FChaosNetLayeredMovesQueue : public FMoverDataStructBase
{
	GENERATED_BODY()
	
	UPROPERTY(VisibleAnywhere, Category = ChaosMover)
	TArray<FChaosNetLayeredMove> Moves;

	void Add(const FScheduledLayeredMove& InScheduledLayeredMove, bool bShouldRollBack, uint8 UniqueID);

	// This function is used for debugging (sending to CVD)
	FChaosNetLayeredMovesQueue& operator=(const TArray<FChaosScheduledLayeredMove>& LayeredMovesQueue);

	virtual FMoverDataStructBase* Clone() const override;
	virtual bool NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess) override;
	virtual UScriptStruct* GetScriptStruct() const override;
	virtual void ToString(FAnsiStringBuilderBase& Out) const override;
	virtual bool ShouldReconcile(const FMoverDataStructBase& AuthorityState) const override;
	virtual void Interpolate(const FMoverDataStructBase& From, const FMoverDataStructBase& To, float Pct) override;
	virtual void Merge(const FMoverDataStructBase& From) override;
	virtual void Decay(float DecayAmount) override;
};

// SM-internal queue entry for a GT-issued FLayeredMoveInstance move.
// Not a USTRUCT -- carries TSharedPtr which cannot be a UPROPERTY.
struct FChaosScheduledLayeredMoveInstance
{
	// Whether this move should roll back on resimulation.
	// Game-thread-issued moves set this to false since the GT is not resimulated.
	bool bShouldRollBack = true;

	FMoverSchedulingInfo SchedulingInfo;

	TSharedPtr<FLayeredMoveInstance> Move;
};

// Per-move entry in the networked FLayeredMoveInstance queue.
USTRUCT()
struct FChaosNetLayeredMoveInstance
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category = "Mover")
	int32 IssuanceServerFrame = INDEX_NONE;

	UPROPERTY(VisibleAnywhere, Category = ChaosMover)
	int32 ExecutionServerFrame = INDEX_NONE;

	UPROPERTY(VisibleAnywhere, Category = ChaosMover)
	uint8 UniqueID = 0xFF;

	// Not net-serialized: networked moves always roll back; only GT-issued moves skip rollback.
	UPROPERTY(VisibleAnywhere, Category = "Mover")
	bool bShouldRollBack = true;

	// Logic class serialized to allow re-linking on load (non-UPROPERTY Move holds the actual data).
	UPROPERTY(VisibleAnywhere, Category = ChaosMover)
	TSubclassOf<ULayeredMoveLogic> MoveLogicClass;

	// Populated during NetSerialize; not a UPROPERTY.
	TSharedPtr<FLayeredMoveInstance> Move;
};

// Networked queue of FLayeredMoveInstance moves, mirroring FChaosNetLayeredMovesQueue for the
// newer ULayeredMoveLogic / FLayeredMoveInstance layered move system.
USTRUCT()
struct FChaosNetLayeredMoveInstancesQueue : public FMoverDataStructBase
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category = ChaosMover)
	TArray<FChaosNetLayeredMoveInstance> Moves;

	CHAOSMOVER_API void Add(const FChaosScheduledLayeredMoveInstance& Scheduled, bool bShouldRollBack, uint8 UniqueID);

	// Used for debugging (sending to CVD): converts the SM-internal scheduled queue.
	CHAOSMOVER_API FChaosNetLayeredMoveInstancesQueue& operator=(const TArray<FChaosScheduledLayeredMoveInstance>& ScheduledMoves);

	virtual FMoverDataStructBase* Clone() const override;
	CHAOSMOVER_API virtual bool NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess) override;
	virtual UScriptStruct* GetScriptStruct() const override;
	CHAOSMOVER_API virtual void ToString(FAnsiStringBuilderBase& Out) const override;
	virtual bool ShouldReconcile(const FMoverDataStructBase& AuthorityState) const override;
	virtual void Interpolate(const FMoverDataStructBase& From, const FMoverDataStructBase& To, float Pct) override;
	virtual void Merge(const FMoverDataStructBase& From) override;
	virtual void Decay(float DecayAmount) override;
};


// This instant movement effect is ChaosMover only and executes AsyncFunction on the issuing end point only
// Being local only, this instant movement effect does not serialize anything, doesn't override NetSerialize
USTRUCT()
struct FAsyncLocalOnlyInstantMovementEffect : public FInstantMovementEffect
{
	GENERATED_BODY()

	FAsyncLocalOnlyInstantMovementEffect() {}
	virtual ~FAsyncLocalOnlyInstantMovementEffect() = default;

	FName OptionalName;
	// Async Function executed in ApplyMovementEffect_Async (only locally)
	TFunction<bool(FApplyMovementEffectParams_Async&, FMoverSyncState&)> AsyncFunction;

	CHAOSMOVER_API virtual bool ApplyMovementEffect_Async(FApplyMovementEffectParams_Async& ApplyEffectParams, FMoverSyncState& OutputState) override;
	CHAOSMOVER_API virtual FInstantMovementEffect* Clone() const override;
	// No NetSerialize override, this is a local only effect
	CHAOSMOVER_API virtual UScriptStruct* GetScriptStruct() const override;
	CHAOSMOVER_API virtual FString ToSimpleString() const override;
};

// This instant movement effect is ChaosMover only and teleport to a location without any checks whatsoever, this is meant for debugging only
USTRUCT()
struct FDebugTeleportToInstantMovementEffect : public FInstantMovementEffect
{
	GENERATED_BODY()

	FDebugTeleportToInstantMovementEffect() {}
	virtual ~FDebugTeleportToInstantMovementEffect() {}

	// A component can be set to MAX_FLT to signify "keep that current component unchanged"
	FVector TeleportLocation = FVector(MAX_FLT, MAX_FLT, MAX_FLT);
	FRotator TeleportRotation = FRotator(MAX_FLT, MAX_FLT, MAX_FLT);

	CHAOSMOVER_API virtual bool ApplyMovementEffect_Async(FApplyMovementEffectParams_Async& ApplyEffectParams, FMoverSyncState& OutputState) override;
	CHAOSMOVER_API virtual FInstantMovementEffect* Clone() const override;
	CHAOSMOVER_API virtual void NetSerialize(FArchive& Ar) override;
	CHAOSMOVER_API virtual UScriptStruct* GetScriptStruct() const override;
	CHAOSMOVER_API virtual FString ToSimpleString() const override;
};
