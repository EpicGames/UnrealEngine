// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DefaultMovementSet/MoverMontageSimulationTypes.h"
#include "ChaosMoverSourceBase.generated.h"

#define UE_API CHAOSMOVER_API

class UMoverBlackboard;
struct FMoverTickStartData;
struct FMoverTimeStep;
struct FMoverTime;
struct FProposedMove;

/**
 * Abstract base class for move sources.
 *
 * A move source is responsible for producing a FProposedMove from simulation state. It
 * decouples the "what velocity to generate" concern from the movement mode, allowing the
 * same generation logic to be reused across different modes or physics backends.
 *
 * All methods must be safe to call from the physics simulation thread — no access to
 * AActor or UActorComponent is permitted.
 */
UCLASS(Abstract, MinimalAPI, EditInlineNew, DefaultToInstanced)
class UChaosMoverSourceBase : public UObject
{
	GENERATED_BODY()

public:
	/** Called once when the owning mode is activated. Async-safe. */
	UE_API virtual void OnStart_Async(UMoverBlackboard* SimBlackboard, const FMoverTime& SimTime) {}

	/**
	 * Produces a proposed move from the current simulation state. Called during the mode's
	 * GenerateMove phase. Async-safe — no AActor or UActorComponent access.
	 */
	UE_API virtual void GenerateMove_Async(
		const FMoverTickStartData& StartState,
		const FMoverTimeStep& TimeStep,
		UMoverBlackboard* SimBlackboard,
		FProposedMove& OutProposedMove) {}

	/** Returns true when this source has naturally expired (e.g. a duration-limited move has elapsed). */
	UE_API virtual bool IsFinished(double CurrentSimTimeMs) const { return false; }

	/** Called when the owning mode is deactivated or when IsFinished() returns true. Async-safe. */
	UE_API virtual void OnEnd_Async(UMoverBlackboard* SimBlackboard, double CurrentSimTimeMs) {}

	/**
	 * Append simulation-authoritative montage state to OutEntries so the game thread can drive
	 * FAnimMontageInstance from this source's output. Called once per substep from
	 * UChaosMoverSimulation::SimulationTick alongside the equivalent call on FLayeredMoveGroup.
	 * Default implementation is a no-op: sources with no montage output need not override.
	 */
	UE_API virtual void AppendMontageOutputEntry(TArray<FMoverSimDrivenMontageEntry>& OutEntries, const FMoverTimeStep& TimeStep) const {}
};

#undef UE_API
