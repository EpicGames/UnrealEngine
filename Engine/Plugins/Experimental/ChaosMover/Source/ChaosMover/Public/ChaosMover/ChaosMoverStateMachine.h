// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "InstantMovementEffect.h"
#include "MovementMode.h"
#include "Templates/SubclassOf.h"
#include "ChaosMover/ChaosMoverSimulationTypes.h"

class UChaosMoverSimulation;
class ULayeredMoveLogic;

namespace UE::ChaosMover
{
	class FMoverStateMachine
	{
	public:
		CHAOSMOVER_API FMoverStateMachine();
		CHAOSMOVER_API virtual ~FMoverStateMachine();

		struct FInitParams
		{
			TWeakObjectPtr<UChaosMoverSimulation> Simulation;
			TWeakObjectPtr<UNullMovementMode> NullMovementMode;
			TWeakObjectPtr<UImmediateMovementModeTransition> ImmediateMovementModeTransition;
		};
		CHAOSMOVER_API void Init(const FInitParams& Params);

		CHAOSMOVER_API void RegisterMovementMode(FName ModeName, TWeakObjectPtr<UBaseMovementMode> Mode, bool bIsDefaultMode = false);

		CHAOSMOVER_API void UnregisterMovementMode(FName ModeName);
		CHAOSMOVER_API void ClearAllMovementModes();

		CHAOSMOVER_API void RegisterGlobalTransition(TWeakObjectPtr<UBaseMovementModeTransition> Transition);
		CHAOSMOVER_API void UnregisterGlobalTransition(TWeakObjectPtr<UBaseMovementModeTransition> Transition);
		CHAOSMOVER_API void ClearAllGlobalTransitions();

		CHAOSMOVER_API FName GetDefaultModeName() const;
		CHAOSMOVER_API void SetDefaultMode(FName NewDefaultModeName);

		CHAOSMOVER_API void QueueNextMode(FName DesiredNextModeName, bool bShouldReenter = false);
		CHAOSMOVER_API void SetModeImmediately(const FMoverTimeStep& TimeStep, FName DesiredModeName, bool bShouldReenter = false);
		CHAOSMOVER_API void ClearQueuedMode();

		CHAOSMOVER_API void OnSimulationTick(const FMoverSimContext& SimContext, const FMoverTimeStep& TimeStep, const FMoverTickStartData& StartState, UMoverBlackboard* SimBlackboard, UMovementMixer* MovementMixer, FMoverTickEndData& OutputState);
		CHAOSMOVER_API void OnSimulationRollback(const FMoverTimeStep& NewTimeStep, const FMoverSyncState& InvalidSyncState, const FMoverSyncState& NewSyncState);

		FName GetCurrentModeName() const
		{
			return CurrentModeName;
		}

		const FGameplayTagContainer& GetLastKnownGameplayTags() const
		{
			return LastKnownGameplayTags;
		}

		CHAOSMOVER_API const TWeakObjectPtr<UBaseMovementMode> GetCurrentMode() const;
		CHAOSMOVER_API const TWeakObjectPtr<UBaseMovementMode> FindMovementMode(FName ModeName) const;
		CHAOSMOVER_API TWeakObjectPtr<UBaseMovementMode> FindMovementMode_Mutable(FName ModeName);

		CHAOSMOVER_API void QueueLayeredMove(const FChaosScheduledLayeredMove& ScheduledMove);

		CHAOSMOVER_API void QueueLayeredMoveInstance(const FChaosScheduledLayeredMoveInstance& ScheduledMove);

		CHAOSMOVER_API void QueueInstantMovementEffect(const FChaosScheduledInstantMovementEffect& ScheduledEffect);

		// Updates the cached snapshot of registered ULayeredMoveLogic objects. Call from the game thread
		// before each simulation tick to keep the cache current with any dynamic re-registration.
		CHAOSMOVER_API void SetRegisteredMoves(const TArray<TObjectPtr<ULayeredMoveLogic>>& Moves);

		// Returns the cached registered move logic objects. Used by the simulation to re-link
		// FLayeredMoveInstance logic pointers when constructing instances from sim action payloads.
		const TArray<TObjectPtr<ULayeredMoveLogic>>& GetRegisteredMoves() const { return CachedRegisteredMoves; }

		CHAOSMOVER_API FMovementModifierHandle QueueMovementModifier(TSharedPtr<FMovementModifierBase> Modifier);
		CHAOSMOVER_API void CancelModifierFromHandle(FMovementModifierHandle ModifierHandle);
		CHAOSMOVER_API const FMovementModifierBase* FindQueuedModifier(FMovementModifierHandle ModifierHandle) const;
		CHAOSMOVER_API const FMovementModifierBase* FindQueuedModifierByType(const UScriptStruct* ModifierType) const;

		const TArray<FChaosScheduledInstantMovementEffect>& GetQueuedInstantEffects() const
		{
			return QueuedInstantEffects;
		}

		void SetOwnerActorName(const FString& InOwnerActorName);
		void SetOwnerActorLocalNetRole(ENetRole InOwnerActorLocalNetRole);

#if WITH_CHAOS_VISUAL_DEBUGGER
		// Capture any simulation commands (instant movement effects, layered moves, etc.) that have not yet been processed to be sent to CVD
		void CaptureUnprocessedSimCommandsForCVD();
		void OnEndTraceMoverData();
#endif

	protected:

		CHAOSMOVER_API void QueueNextModeInternal(FName DesiredNextModeName, bool bShouldReenter = false, bool bIsFromRollback=false);


		TMap<FName, TWeakObjectPtr<UBaseMovementMode>> Modes;
		TArray<TWeakObjectPtr<UBaseMovementModeTransition>> GlobalTransitions;

		TWeakObjectPtr<UImmediateMovementModeTransition> QueuedModeTransitionWeakPtr;

		TWeakObjectPtr<UImmediateMovementModeTransition> ImmediateMovementModeTransitionWeakPtr;
		TWeakObjectPtr<UNullMovementMode> NullMovementModeWeakPtr;

		FString OwnerActorName;
		ENetRole OwnerActorLocalNetRole;

		FName DefaultModeName = NAME_None;
		FName CurrentModeName = NAME_None;

		/** Moves that are queued to be executed when their scheduled time comes (earliest is at the start of the next sim subtick) */
		TArray<FChaosScheduledLayeredMove> QueuedLayeredMoves;

		/** FLayeredMoveInstance moves queued from the GT or network, flushed into LayeredMoveInstances at the appropriate sim frame */
		TArray<FChaosScheduledLayeredMoveInstance> QueuedLayeredMoveInstances;

		/** Effects that are queued to be applied when their schduled time comes (earliest at the start of the next sim subtick or at the end of the ongoing tick if queued mid tick) */
		TArray<FChaosScheduledInstantMovementEffect> QueuedInstantEffects;

		/** Modifiers that are queued to be added to the simulation at the start of the next sim subtick. */
		TArray<TSharedPtr<FMovementModifierBase>> QueuedMovementModifiers;

		/** Modifiers that are to be canceled at the start of the next sim subtick. */
		TArray<FMovementModifierHandle> ModifiersToCancel;

		// Internal-use-only tick data structs, for efficiency since they typically have the same contents from frame to frame
		FMoverTickStartData WorkingSubstepStartData;
		FSimulationTickParams WorkingSimTickParams;
		FMoverTickEndData* WorkingSimTickEndData = nullptr;	// Only valid during OnSimulationTick and null otherwise

		TWeakObjectPtr<UChaosMoverSimulation> Simulation;

		/** Last set of simulation tags that were diffed and emitted. Used to detect tag changes between substeps and on rollback. */
		FGameplayTagContainer LastKnownGameplayTags;

	private:
		// Snapshot of UMoverComponent::RegisteredMoves, used on the simulation thread to re-link
		// logic pointers on FLayeredMoveInstance moves that arrive via the replicated sync state.
		TArray<TObjectPtr<ULayeredMoveLogic>> CachedRegisteredMoves;

		void ConstructDefaultModes();

		/** Collect all gameplay tags contributed by the active mode, modifiers, and layered moves in SyncState. */
		void CollectGameplayTags(const FMoverSyncState& SyncState, FGameplayTagContainer& OutTags) const;

		/** Diff OldTags vs NewTags and emit FMoverGameplayTagChangeEventData for each change (bWasAdded=true for additions, false for removals). */
		void DiffAndEmitGameplayTagEvents(const FGameplayTagContainer& OldTags, const FGameplayTagContainer& NewTags, const FMoverTimeStep& TimeStep, bool bIsCausedByRollback);
		void AdvanceToNextMode(const FMoverTimeStep& TimeStep);
		void FlushQueuedMovesToGroup(const FMoverTimeStep& TimeStep, FLayeredMoveGroup& Group);
		void FlushQueuedModifiersToGroup(FMovementModifierGroup& ModifierGroup);
		void FlushModifierCancellationsToGroup(FMovementModifierGroup& ActiveModifierGroup);
		bool HasAnyInstantEffectsQueued(const FMoverTimeStep& TimeStep) const;
		bool ApplyInstantEffects(const FMoverTickStartData& SubstepStartData, const FMoverTimeStep& SubTimeStep, FMoverSyncState& OutputState);
		bool UpdateWorkingStateFromSyncState(const FMoverTimeStep& TimeStep, FMoverSyncState& OutputState);
		void ReceiveInstantMovementEffects(const FMoverTimeStep& TimeStep, const FChaosNetInstantMovementEffectsQueue* InstantMovementEffectsQueue);
		void ReceiveLayeredMoves(const FMoverTimeStep& TimeStep, const FChaosNetLayeredMovesQueue* LayeredMovesQueue);
		void ReceiveLayeredMoveInstances(const FMoverTimeStep& TimeStep, const FChaosNetLayeredMoveInstancesQueue* Queue);
		void RollBackQueuedInstantMovementEffects(int32 NewServerFrame);
		void RollBackQueuedLayeredMoves(int32 NewServerFrame);
		void RollBackQueuedLayeredMoveInstances(int32 NewServerFrame);
		void FlushQueuedMoveInstancesToGroup(const FMoverTimeStep& TimeStep, FLayeredMoveInstanceGroup& Group);
		FMoverSimContext GetSimContext() const;
	
		FMoverTimeStep InternalSimTime;

		struct FIDHistory
		{
			void AddID(int32 Frame, uint8 ID);
			void CullOldFrames(int32 CurrentFrame, int32 MaxAge = 30);
			void Rollback(int32 RollbackToFrameInclusive);
			bool WasIDAlreadySeen(uint8 ID) const;

			TMap<int32, TSet<uint8>> IDsByFrame;
		};

		FIDHistory InstantMovementEffectsIDHistory;
		FIDHistory LayeredMovesIDHistory;
		FIDHistory LayeredMoveInstancesIDHistory;

#if WITH_CHAOS_VISUAL_DEBUGGER
		// If true, queued simulation commands (instant movement effects. layered moves, etc.) will be added to DebugSimData
		// It is false from the last time we traced movement data to the time we call CaptureUnprocessedSimCommandsForCVD in FMoverStateMachine::OnSimulationTick
		// which allows us to trace any unprocessed sim commands, including scheduled ones, that exist on that frame
		// It is true after that point until we trace, so that we can capture the ones queued after that, for instance those queued by movement modes
		bool bCaptureSimCommandsForCVD = false;
		void TraceInstantMovementEffect(const FChaosScheduledInstantMovementEffect& ScheduledEffect);
		void TraceLayeredMove(const FChaosScheduledLayeredMove& ScheduledMove);
		void TraceLayeredMoveInstance(const FChaosScheduledLayeredMoveInstance& ScheduledMove);
#endif
	};
}