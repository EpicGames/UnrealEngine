// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosMover/Character/Effects/ChaosCharacterApplyVelocityEffect.h"
#include "MoverTypes.h"

#include "ChaosCharacterInputs.generated.h"

/** Add this struct to the character inputs to launch the character */
USTRUCT(MinimalAPI, BlueprintType)
struct FChaosMoverLaunchInputs : public FMoverDataStructBase
{
	GENERATED_BODY()

public:
	// Launch velocity in cm/s or impulse in kg cm/s
	UPROPERTY(BlueprintReadWrite, Category = "Chaos Mover")
	FVector LaunchVelocityOrImpulse = FVector::ZeroVector;

	UPROPERTY(BlueprintReadWrite, Category = "Chaos Mover")
	EChaosMoverVelocityEffectMode Mode = EChaosMoverVelocityEffectMode::AdditiveVelocity;

	virtual UScriptStruct* GetScriptStruct() const override
	{
		return StaticStruct();
	}
	virtual FMoverDataStructBase* Clone() const override
	{
		return new FChaosMoverLaunchInputs(*this);
	}
	CHAOSMOVER_API virtual bool NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess) override;
	CHAOSMOVER_API virtual void ToString(FAnsiStringBuilderBase& Out) const override;
	CHAOSMOVER_API virtual bool ShouldReconcile(const FMoverDataStructBase& AuthorityState) const override;
	CHAOSMOVER_API virtual void Interpolate(const FMoverDataStructBase& From, const FMoverDataStructBase& To, float Pct) override;
	CHAOSMOVER_API virtual void Merge(const FMoverDataStructBase& From) override;
};

template<>
struct TStructOpsTypeTraits< FChaosMoverLaunchInputs > : public TStructOpsTypeTraitsBase2< FChaosMoverLaunchInputs >
{
	enum
	{
		WithNetSerializer = true,
		WithCopy = true
	};
};

/** Add this struct to toggle crouch on the character */
USTRUCT(MinimalAPI, BlueprintType)
struct FChaosMoverCrouchInputs : public FMoverDataStructBase
{
	GENERATED_BODY()

public:
	// Launch velocity in cm/s or impulse in kg cm/s
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category = "Chaos Mover")
	bool bWantsToCrouch = false;

	virtual UScriptStruct* GetScriptStruct() const override
	{
		return StaticStruct();
	}
	virtual FMoverDataStructBase* Clone() const override
	{
		return new FChaosMoverCrouchInputs(*this);
	}
	CHAOSMOVER_API virtual bool NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess) override;
	CHAOSMOVER_API virtual void ToString(FAnsiStringBuilderBase& Out) const override;
	CHAOSMOVER_API virtual bool ShouldReconcile(const FMoverDataStructBase& AuthorityState) const override;
	CHAOSMOVER_API virtual void Interpolate(const FMoverDataStructBase& From, const FMoverDataStructBase& To, float Pct) override;
	CHAOSMOVER_API virtual void Merge(const FMoverDataStructBase& From) override;
};

template<>
struct TStructOpsTypeTraits< FChaosMoverCrouchInputs > : public TStructOpsTypeTraitsBase2< FChaosMoverCrouchInputs >
{
	enum
	{
		WithNetSerializer = true,
		WithCopy = true
	};
};

/** Add this struct to the character inputs to override the max speed and acceleration */
USTRUCT(MinimalAPI, BlueprintType)
struct FChaosMovementSettingsOverrides : public FMoverDataStructBase
{
	GENERATED_BODY()

public:
	/** The name of the mode to apply the overrides to. If this is set to none the override will apply to the currently active mode */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "General")
	FName ModeName = NAME_None;

	/** Override maximum speed in the movement plane */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "General", meta = (ClampMin = "0", UIMin = "0", ForceUnits = "cm/s"))
	float MaxSpeedOverride = 800.f;

	/** Override max linear rate of acceleration for controlled input. May be scaled based on magnitude of input. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "General", meta = (ClampMin = "0", UIMin = "0", ForceUnits = "cm/s^2"))
	float AccelerationOverride = 4000.f;

	virtual UScriptStruct* GetScriptStruct() const override
	{
		return StaticStruct();
	}
	virtual FMoverDataStructBase* Clone() const override
	{
		return new FChaosMovementSettingsOverrides(*this);
	}
	CHAOSMOVER_API virtual bool NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess) override;
	CHAOSMOVER_API virtual void ToString(FAnsiStringBuilderBase& Out) const override;
	CHAOSMOVER_API virtual bool ShouldReconcile(const FMoverDataStructBase& AuthorityState) const override;
	CHAOSMOVER_API virtual void Interpolate(const FMoverDataStructBase& From, const FMoverDataStructBase& To, float Pct) override;
	CHAOSMOVER_API virtual void Merge(const FMoverDataStructBase& From) override;
};

template<>
struct TStructOpsTypeTraits< FChaosMovementSettingsOverrides > : public TStructOpsTypeTraitsBase2< FChaosMovementSettingsOverrides >
{
	enum
	{
		WithNetSerializer = true,
		WithCopy = true
	};
};

/** Add this struct to the character inputs to cancel any max speed and acceleration for the input mode */
USTRUCT(MinimalAPI, BlueprintType)
struct FChaosMovementSettingsOverridesRemover : public FMoverDataStructBase
{
	GENERATED_BODY()

public:
	/** The name of the mode to apply the overrides to. If this is set to none the override will apply to the currently active mode */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "General")
	FName ModeName = NAME_None;

	virtual UScriptStruct* GetScriptStruct() const override
	{
		return StaticStruct();
	}
	virtual FMoverDataStructBase* Clone() const override
	{
		return new FChaosMovementSettingsOverridesRemover(*this);
	}
	CHAOSMOVER_API virtual bool NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess) override;
	CHAOSMOVER_API virtual void ToString(FAnsiStringBuilderBase& Out) const override;
	CHAOSMOVER_API virtual bool ShouldReconcile(const FMoverDataStructBase& AuthorityState) const override;
	CHAOSMOVER_API virtual void Interpolate(const FMoverDataStructBase& From, const FMoverDataStructBase& To, float Pct) override;
	CHAOSMOVER_API virtual void Merge(const FMoverDataStructBase& From) override;
};

template<>
struct TStructOpsTypeTraits< FChaosMovementSettingsOverridesRemover > : public TStructOpsTypeTraitsBase2< FChaosMovementSettingsOverridesRemover >
{
	enum
	{
		WithNetSerializer = true,
		WithCopy = true
	};
};

/** Add this struct to the character inputs to apply a requested move the character */
USTRUCT(MinimalAPI, BlueprintType)
struct FChaosMoverRequestedMoveInputs : public FMoverDataStructBase
{
	GENERATED_BODY()

public:
	// Requested velocity in cm/s
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category = "Chaos Mover")
	FVector RequestedVelocity = FVector::ZeroVector;

	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category = "Chaos Mover")
	bool bForceMaxSpeed = false;

	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category = "Chaos Mover")
	bool bUseAcceleration = true;

	virtual UScriptStruct* GetScriptStruct() const override
	{
		return StaticStruct();
	}
	virtual FMoverDataStructBase* Clone() const override
	{
		return new FChaosMoverRequestedMoveInputs(*this);
	}
	CHAOSMOVER_API virtual bool NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess) override;
	CHAOSMOVER_API virtual void ToString(FAnsiStringBuilderBase& Out) const override;
	CHAOSMOVER_API virtual bool ShouldReconcile(const FMoverDataStructBase& AuthorityState) const override;
	CHAOSMOVER_API virtual void Interpolate(const FMoverDataStructBase& From, const FMoverDataStructBase& To, float Pct) override;
	CHAOSMOVER_API virtual void Merge(const FMoverDataStructBase& From) override;
};

template<>
struct TStructOpsTypeTraits< FChaosMoverRequestedMoveInputs > : public TStructOpsTypeTraitsBase2< FChaosMoverRequestedMoveInputs >
{
	enum
	{
		WithNetSerializer = true,
		WithCopy = true
	};
};
