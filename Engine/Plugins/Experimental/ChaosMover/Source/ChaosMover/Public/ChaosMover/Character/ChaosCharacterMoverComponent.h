// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "ChaosMover/ChaosMoverSimulationTypes.h"
#include "DefaultMovementSet/CharacterMoverComponent.h"
#include "DefaultMovementSet/CharacterMoverSimulationTypes.h"
#include "MoveLibrary/FloorQueryUtils.h"

#include "ChaosCharacterMoverComponent.generated.h"

// Fired after the actor lands on a valid surface.
DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FChaosMover_OnLanded, class UChaosCharacterMoverComponent*, ChaosMoverComponent, FLandedEventData, LandedEventData);

// Fired after the actor jumps. First param is the starting jump height.
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FChaosMover_OnJumped, float, StartingJumpHeight);


UCLASS(MinimalAPI, BlueprintType, Blueprintable, meta = (BlueprintSpawnableComponent))
class UChaosCharacterMoverComponent : public UCharacterMoverComponent
{
	GENERATED_BODY()

public:
	CHAOSMOVER_API UChaosCharacterMoverComponent();

	CHAOSMOVER_API virtual bool TryGetFloorCheckHitResult(FHitResult& OutHitResult) const override;

	UFUNCTION(BlueprintPure, Category = "Chaos Mover")
	CHAOSMOVER_API virtual bool TryGetLastWaterResult(FWaterCheckResult& OutWaterResult) const;

	CHAOSMOVER_API virtual void ProduceInput(const int32 DeltaTimeMS, FMoverInputCmdContext* Cmd) override;
	CHAOSMOVER_API virtual void ProduceServerInput(const int32 DeltaTimeMS, FMoverInputCmdContext* Cmd);
	CHAOSMOVER_API virtual void ProduceLocalInput(FMoverDataCollection& OutLocalSimInput) const;
	CHAOSMOVER_API virtual void DoQueueNextMode(FName DesiredModeName, bool bShouldReenter=false) override;
	CHAOSMOVER_API virtual FName GetNextMovementModeName() const override;

	// Broadcast when this actor lands on a valid surface.
	UPROPERTY(BlueprintAssignable, Category = "Chaos Mover", DisplayName = "On Landed")
	FChaosMover_OnLanded OnLandedDelegate;

	// Broadcast when this actor jumps.
	UPROPERTY(BlueprintAssignable, Category = "Chaos Mover")
	FChaosMover_OnJumped OnJumped;

	// Launch the character using either impulse or velocity
	// Note: This will only trigger a launch if a launch transition is implemented on the current movement mode
	UFUNCTION(BlueprintCallable, Category = "Chaos Mover")
	void Launch(const FVector& VelocityOrImpulse, EChaosMoverVelocityEffectMode Mode = EChaosMoverVelocityEffectMode::AdditiveVelocity);

	// Override the movement mode settings on a mode
	// If the name is not set it will apply to the current mode
	UFUNCTION(BlueprintCallable, Category = "Chaos Mover")
	CHAOSMOVER_API void OverrideMovementSettings(const FChaosMovementSettingsOverrides Overrides);

	// Cancel overrides of movement mode settings
	// If the name is not set it will apply to the current mode
	UFUNCTION(BlueprintCallable, Category = "Chaos Mover")
	void CancelMovementSettingsOverrides(FName ModeName = NAME_None);

	// Get cached predicted trajectory deltas.
	// The first element is Identity corresponding to the starting transform at time offset 0.
	TConstArrayView<FTrajectorySampleInfo> GetCachedPredictedTrajectoryDeltas() const { return LatestPredictedTrajectoryData.Deltas; }

	// Returns predicted trajectory constructed from the cached deltas.
	// The first element is the starting transform at time offset 0.
	// Supported parameters: bUseVisualComponentRoot, NumPredictionSamples and SecondsPerSample.
	CHAOSMOVER_API virtual TArray<FTrajectorySampleInfo> GetPredictedTrajectory(FMoverPredictTrajectoryParams PredictionParams) override;

	CHAOSMOVER_API void SetPredictedTrajectoryParams(int32 InSteps, float InStepSeconds, bool bInEnableTrajectoryPrediction);
	CHAOSMOVER_API void GetPredictedTrajectoryParams(int32& OutSteps, float& OutStepSeconds, bool& bIsTrajectoryPredictionEnabled) const;

protected:
	CHAOSMOVER_API virtual void OnMoverPreSimulationTick(const FMoverTimeStep& TimeStep, const FMoverInputCmdContext& InputCmd) override;
	CHAOSMOVER_API virtual void DispatchSimulationEvents(const UE::Mover::FSimulationOutputData& OutputData) override;
	CHAOSMOVER_API virtual void ProcessSimulationEvent(const FMoverSimulationEventData& EventData) override;
	CHAOSMOVER_API virtual void SetAdditionalSimulationOutput(const FMoverDataCollection& Data) override;

	CHAOSMOVER_API void ClearQueuedMode();

	CHAOSMOVER_API void NotifyOnLanded(const FLandedEventData& LandedEvent);
	CHAOSMOVER_API virtual void OnLanded(const FLandedEventData& LandedEvent);

	CHAOSMOVER_API virtual void ProduceInputImpl(const int32 DeltaTimeMS, FMoverInputCmdContext* Cmd);

	FName ModeToOverrideSettings = NAME_None;
	float MaxSpeedOverride = 0.0f;
	float AccelerationOverride = 0.0f;
	bool bOverrideMovementSettings = false;
	bool bCancelMovementOverrides = false;

	bool bFloorResultSet = false;
	FFloorCheckResult LatestFloorResult;

	bool bWaterResultSet = false;
	FWaterCheckResult LatestWaterResult;

	bool bPredictedTrajectorySet = false;
	FChaosMoverPredictedTrajectoryData LatestPredictedTrajectoryData;

	int32 PredictedTrajectorySteps = 15;
	float PredictedTrajectoryStepSeconds = 0.1f;
	bool bEnableTrajectoryPrediction = false;

	FVector LaunchVelocityOrImpulse = FVector::ZeroVector;
	EChaosMoverVelocityEffectMode LaunchMode = EChaosMoverVelocityEffectMode::AdditiveVelocity;

	// Queued immediate mode transition, it will be transmitted to the simulation next time ProduceInput is called
	FName QueuedModeTransitionName = NAME_None;

	// Batch accumulation for stance dispatch: avoids spurious visual transitions caused by
	// rollback/resim transient transitions (e.g. forced uncrouch then recrouch).
	// ProcessSimulationEvent accumulates the final stance seen in a batch,
	// and SetAdditionalSimulationOutput fires OnStanceChanged only if the net result changed.
	EStanceMode LastDispatchedStance = EStanceMode::Invalid;
	TOptional<EStanceMode> PendingBatchFinalStance;
};
