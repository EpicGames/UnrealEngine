// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosMover/ChaosMoverSimulationTypes.h"
#include "InstantMovementEffect.h"

#include "ChaosCharacterApplyVelocityEffect.generated.h"

#define UE_API CHAOSMOVER_API


/** Applies a velocity or impulse for a single tick */
USTRUCT(BlueprintInternalUseOnly)
struct FChaosCharacterApplyVelocityEffect : public FInstantMovementEffect
{
	GENERATED_BODY()

	// Velocity or impulse to apply to the actor.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	FVector VelocityOrImpulseToApply = FVector::ZeroVector;

	// Controls whether to apply velocity or impulse and if the velocity will be additive
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	EChaosMoverVelocityEffectMode Mode = EChaosMoverVelocityEffectMode::AdditiveVelocity;

	UE_API virtual bool ApplyMovementEffect_Async(FApplyMovementEffectParams_Async& ApplyEffectParams, FMoverSyncState& OutputState);
	UE_API virtual FInstantMovementEffect* Clone() const override;
	UE_API virtual void NetSerialize(FArchive& Ar) override;
	UE_API virtual UScriptStruct* GetScriptStruct() const override;
	UE_API virtual FString ToSimpleString() const override;
	UE_API virtual void AddReferencedObjects(class FReferenceCollector& Collector) override;
};

#undef UE_API
