// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ChaosMover/ChaosMoveExecutorBase.h"
#include "ChaosMover/ChaosMoverSimulationTypes.h"
#include "ChaosCharacterAirExecutor.generated.h"

#define UE_API CHAOSMOVER_API

class USharedChaosCharacterMovementSettings;

/**
 * UChaosCharacterAirExecutor: a concrete UChaosMoveExecutorBase that applies
 * air physics to the character's sync state each simulation tick.
 *
 * Implements IChaosCharacterMovementModeInterface and
 * IChaosCharacterConstraintMovementModeInterface so that it works correctly
 * when used inside a UChaosCompositeMovementMode (which delegates interface
 * resolution to the executor via CollectSimulationInterfaces). The constraint
 * is enabled with RadialForceLimit = 0 so it provides only upright swing
 * torque, not positional driving force -- velocity-based movement is applied
 * via SetV each post-simulation tick.
 */
UCLASS(MinimalAPI, Blueprintable, BlueprintType, EditInlineNew, DefaultToInstanced)
class UChaosCharacterAirExecutor
	: public UChaosMoveExecutorBase
	, public IChaosCharacterMovementModeInterface
	, public IChaosCharacterConstraintMovementModeInterface
	, public IChaosPreSimulationTickInterface
	, public IChaosPostSimulationTickInterface
{
	GENERATED_BODY()

public:

	// Maximum torque the character can apply to remain upright
	UPROPERTY(EditAnywhere, Category = "Constraint Settings", meta = (ClampMin = "0", UIMin = "0", ForceUnits = "NewtonMeters"))
	float SwingTorqueLimit = 3000.0f;

	// Controls whether the character capsule is forced to remain upright
	UPROPERTY(EditAnywhere, Category = "Constraint Settings")
	bool bShouldCharacterRemainUpright = true;

	/**
	 * Optional override for target height (desired distance from capsule center to floor).
	 * If unset, derived automatically from the owning character's skeletal mesh Z offset.
	 */
	UPROPERTY(EditAnywhere, Category = "Query Settings", meta = (ClampMin = "0", UIMin = "0", ForceUnits = "cm"))
	TOptional<float> TargetHeightOverride;

	// Optional override for the ground query radius. If unset, derived from capsule radius.
	UPROPERTY(EditAnywhere, Category = "Query Settings", meta = (ClampMin = "0", UIMin = "0", ForceUnits = "cm"))
	TOptional<float> QueryRadiusOverride;

	// UChaosMoveExecutorBase overrides
	UE_API virtual void OnModeRegistered(const FName ModeName) override;
	UE_API virtual void OnModeUnregistered() override;
	UE_API virtual void ExecuteMove_Async(const FSimulationTickParams& Params, FMoverTickEndData& OutputState) override;

	// IChaosPreSimulationTickInterface
	UE_API virtual void PreSimulationTick_Async(FChaosMoverPreSimContext& Context) override;

	// IChaosPostSimulationTickInterface
	UE_API virtual void PostSimulationTick_Async(FChaosMoverPostSimContext& Context) override;

	// IChaosCharacterMovementModeInterface
	virtual float GetTargetHeight() const override { return TargetHeight; }
	virtual float GetGroundQueryRadius() const override { return QueryRadius; }
	UE_API virtual float GetMaxWalkSlopeCosine() const override;
	virtual bool ShouldCharacterRemainUpright() const override { return bShouldCharacterRemainUpright; }
	UE_API virtual float GetMaxSpeed() const override;
	UE_API virtual void OverrideMaxSpeed(float Value) override;
	UE_API virtual void ClearMaxSpeedOverride() override;
	UE_API virtual float GetAcceleration() const override;
	UE_API virtual void OverrideAcceleration(float Value) override;
	UE_API virtual void ClearAccelerationOverride() override;
	UE_API virtual void UpdateCurrentFloor(const FMoverTimeStep& TimeStep) const override;

	// IChaosCharacterConstraintMovementModeInterface
	virtual bool ShouldEnableConstraint() const override { return true; }
	UE_API virtual void UpdateConstraintSettings(Chaos::FCharacterGroundConstraintSettings& ConstraintSettings) const override;

private:
	// Resolved on the game thread in OnModeRegistered
	float TargetHeight = 95.0f;
	float QueryRadius  = 30.0f;
	TWeakObjectPtr<const USharedChaosCharacterMovementSettings> SharedSettingsWeakPtr;

	TOptional<float> MaxSpeedOverride;
	TOptional<float> AccelerationOverride;
};

#undef UE_API
