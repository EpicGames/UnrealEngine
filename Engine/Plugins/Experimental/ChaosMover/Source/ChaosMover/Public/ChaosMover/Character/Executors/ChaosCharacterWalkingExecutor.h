// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ChaosMover/ChaosMoveExecutorBase.h"
#include "ChaosMover/ChaosMoverSimulationTypes.h"
#include "ChaosCharacterWalkingExecutor.generated.h"

#define UE_API CHAOSMOVER_API

class USharedChaosCharacterMovementSettings;

/**
 * UChaosCharacterWalkingExecutor: a concrete UChaosMoveExecutorBase that applies
 * walking physics to the Chaos character ground constraint simulation.
 *
 * This executor also implements IChaosCharacterMovementModeInterface and
 * IChaosCharacterConstraintMovementModeInterface. When used inside a
 * UChaosCompositeMovementMode, CollectSimulationInterfaces automatically exposes
 * those interfaces to the simulation.
 *
 * The physics application logic (ExecuteMove_Async) is equivalent to
 * UChaosWalkingMode::SimulationTick_Implementation. Move generation is intentionally
 * left to the paired UChaosMoverSourceBase — use UChaosLayeredMoveSource or a custom
 * subclass to drive the proposed velocity each tick.
 *
 * OnModeRegistered resolves TargetHeight and QueryRadius from the owning character
 * and locates the shared movement settings — all on the game thread before simulation
 * begins.
 */
UCLASS(MinimalAPI, Blueprintable, BlueprintType, EditInlineNew, DefaultToInstanced)
class UChaosCharacterWalkingExecutor
	: public UChaosMoveExecutorBase
	, public IChaosCharacterMovementModeInterface
	, public IChaosCharacterConstraintMovementModeInterface
	, public IChaosPreSimulationTickInterface
	, public IChaosPostSimulationTickInterface
{
	GENERATED_BODY()

public:
	// Maximum force the character can apply to reach the motion target
	UPROPERTY(EditAnywhere, Category = "Constraint Settings", meta = (ClampMin = "0", UIMin = "0", ForceUnits = "Newtons"))
	float RadialForceLimit = 2000.0f;

	// Maximum torque the character can apply to rotate in air about the vertical axis
	UPROPERTY(EditAnywhere, Category = "Constraint Settings", meta = (ClampMin = "0", UIMin = "0", ForceUnits = "NewtonMeters"))
	float TwistTorqueLimit = 1500.0f;

	// Maximum torque the character can apply to remain upright
	UPROPERTY(EditAnywhere, Category = "Constraint Settings", meta = (ClampMin = "0", UIMin = "0", ForceUnits = "NewtonMeters"))
	float SwingTorqueLimit = 3000.0f;

	// Damping factor to control the softness of the interaction between the character and the ground.
	// Set to 0 for no damping and 1 for maximum damping.
	UPROPERTY(EditAnywhere, Category = "Constraint Settings", meta = (ClampMin = "0", UIMin = "0", ClampMax = "1", UIMax = "1"))
	float GroundDamping = 0.0f;

	// Maximum force the character can apply to hold in place while standing on an unwalkable incline
	UPROPERTY(EditAnywhere, Category = "Constraint Settings", meta = (ClampMin = "0", UIMin = "0", ForceUnits = "Newtons"))
	float FrictionForceLimit = 100.0f;

	// Scaling applied to the radial force limit to raise the limit to always allow the character to
	// reach the motion target. 1 = scale as needed, 0 = no scaling.
	UPROPERTY(EditAnywhere, Category = "Constraint Settings", meta = (ClampMin = "0", UIMin = "0", ClampMax = "1", UIMax = "1"))
	float FractionalRadialForceLimitScaling = 1.0f;

	// Controls the reaction force applied to the ground in the ground plane when the character is moving.
	// 1 = full reaction force, 0 = normal force only.
	UPROPERTY(EditAnywhere, Category = "Constraint Settings", meta = (ClampMin = "0", UIMin = "0", ClampMax = "1", UIMax = "1"))
	float FractionalGroundReaction = 1.0f;

	// Controls whether the character capsule is forced to remain upright
	UPROPERTY(EditAnywhere, Category = "Constraint Settings")
	bool bShouldCharacterRemainUpright = true;

	/**
	 * Optional override for target height (desired distance from capsule center to floor).
	 * If unset, derived automatically from the owning character's skeletal mesh Z offset.
	 */
	UPROPERTY(EditAnywhere, Category = "Constraint Settings", meta = (ClampMin = "0", UIMin = "0", ForceUnits = "cm"))
	TOptional<float> TargetHeightOverride;

	// Optional override for the ground query radius. If unset, derived from capsule radius.
	UPROPERTY(EditAnywhere, Category = "Query Settings", meta = (ClampMin = "0", UIMin = "0", ForceUnits = "cm"))
	TOptional<float> QueryRadiusOverride;

	// Controls how much downward velocity is applied to keep the character rooted to the ground
	// when within MaxStepHeight of the surface.
	UPROPERTY(EditAnywhere, Category = "Movement Settings", meta = (ClampMin = "0", UIMin = "0", ClampMax = "1", UIMax = "1"))
	float FractionalDownwardVelocityToTarget = 1.0f;

	// Whether the character can apply angular velocity in response to stick input
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Movement Settings")
	bool bShouldApplyAngularVelocityToTarget = true;

	// UChaosMoveExecutorBase overrides
	UE_API virtual void OnModeRegistered(const FName ModeName) override;
	UE_API virtual void OnModeUnregistered() override;
	UE_API virtual void ExecuteMove_Async(const FSimulationTickParams& Params, FMoverTickEndData& OutputState) override;

	// IChaosPreSimulationTickInterface
	UE_API virtual void PreSimulationTick_Async(FChaosMoverPreSimContext& Context) override;

	// IChaosPostSimulationTickInterface
	UE_API virtual void PostSimulationTick_Async(FChaosMoverPostSimContext& Context) override;

	// IChaosCharacterMovementModeInterface — implemented directly on this executor
	virtual float GetTargetHeight() const override { return TargetHeight; }
	virtual float GetGroundQueryRadius() const override { return QueryRadius; }
	UE_API virtual float GetMaxWalkSlopeCosine() const override;
	virtual bool ShouldCharacterRemainUpright() const override { return bShouldCharacterRemainUpright; }
	UE_API virtual float GetMaxSpeed() const override;
	UE_API virtual void OverrideMaxSpeed(float Value) override;
	UE_API virtual void ClearMaxSpeedOverride() override;
	UE_API virtual float GetAcceleration() const override;
	UE_API virtual void OverrideAcceleration(float Value) override;
	UE_API virtual void ClearAccelerationOverride() override;
	UE_API virtual void UpdateCurrentFloor(const FMoverTimeStep& TimeStep) const override;

	// IChaosCharacterConstraintMovementModeInterface — implemented directly on this executor
	virtual bool ShouldEnableConstraint() const override { return true; }
	UE_API virtual void UpdateConstraintSettings(Chaos::FCharacterGroundConstraintSettings& ConstraintSettings) const override;

private:
	// Resolved on the game thread in OnModeRegistered
	float TargetHeight = 95.0f;
	float QueryRadius = 30.0f;
	TWeakObjectPtr<const USharedChaosCharacterMovementSettings> SharedSettingsWeakPtr;

	TOptional<float> MaxSpeedOverride;
	TOptional<float> AccelerationOverride;
};

#undef UE_API
