// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosMover/ChaosMovementMode.h"
#include "ChaosMover/ChaosMoverSimulationTypes.h"

#include "ChaosCharacterMovementMode.generated.h"

namespace Chaos
{
	class FCharacterGroundConstraintSettings;
}

UENUM(BlueprintType)
enum class ECharacterMoverFrictionOverrideMode : uint8
{
	DoNotOverride,
	AlwaysOverrideToZero,
	OverrideToZeroWhenMoving,
};

/**
 * Base class for all Chaos character movement modes
 */
UCLASS(MinimalAPI, Abstract, Blueprintable, BlueprintType, EditInlineNew, DefaultToInstanced)
class UChaosCharacterMovementMode :	public UChaosMovementMode,
									public IChaosCharacterMovementModeInterface,
									public IChaosCharacterConstraintMovementModeInterface,
									public IChaosPreSimulationTickInterface,
									public IChaosPostSimulationTickInterface
{
	GENERATED_BODY()

public:
	CHAOSMOVER_API UChaosCharacterMovementMode(const FObjectInitializer& ObjectInitializer);

	CHAOSMOVER_API virtual void OnRegistered(const FName ModeName, const FMoverSimContext& SimContext) override;

	// IChaosPreSimulationTickInterface
	CHAOSMOVER_API virtual void PreSimulationTick_Async(FChaosMoverPreSimContext& Context) override;

	// IChaosPostSimulationTickInterface
	CHAOSMOVER_API virtual void PostSimulationTick_Async(FChaosMoverPostSimContext& Context) override;

	// IChaosCharacterConstraintMovementModeInterface
	virtual bool ShouldEnableConstraint() const override
	{
		return true;
	}

	CHAOSMOVER_API virtual void ModifyContacts(const FMoverTimeStep& TimeStep, const UE::ChaosMover::FSimulationInputData& InputData, const UE::ChaosMover::FSimulationOutputData& OutputData, Chaos::FCollisionContactModifier& Modifier) const override;

	CHAOSMOVER_API virtual void UpdateConstraintSettings(Chaos::FCharacterGroundConstraintSettings& ConstraintSettings) const override;

	// IChaosCharacterMovementModeInterface
	virtual float GetTargetHeight() const override
	{
		return TargetHeight;
	}

	virtual float GetGroundQueryRadius() const override
	{
		return QueryRadius;
	}

	CHAOSMOVER_API virtual float GetMaxWalkSlopeCosine() const override;

	virtual bool ShouldCharacterRemainUpright() const override
	{
		return bShouldCharacterRemainUpright;
	}

	CHAOSMOVER_API virtual float GetMaxSpeed() const override;
	CHAOSMOVER_API virtual void OverrideMaxSpeed(float Value) override;
	CHAOSMOVER_API virtual void ClearMaxSpeedOverride() override;

	CHAOSMOVER_API virtual float GetAcceleration() const override;
	CHAOSMOVER_API virtual void OverrideAcceleration(float Value) override;
	CHAOSMOVER_API virtual void ClearAccelerationOverride() override;

	// Updates the floor result in the blackboard
	CHAOSMOVER_API virtual void UpdateCurrentFloor(const FMoverTimeStep& TimeStep) const override;

protected:
	CHAOSMOVER_API void SetTargetHeightOverride(float InTargetHeight);
	CHAOSMOVER_API void ClearTargetHeightOverride();

	CHAOSMOVER_API void SetQueryRadiusOverride(float InQueryRadius);
	CHAOSMOVER_API void ClearQueryRadiusOverride();

	TWeakObjectPtr<const class USharedChaosCharacterMovementSettings> SharedSettingsWeakPtr;

	// Maximum force the character can apply to reach the motion target
	UPROPERTY(EditAnywhere, Category = "Constraint Settings", meta = (ClampMin = "0", UIMin = "0", ForceUnits = "Newtons"))
	float RadialForceLimit = 1500.0f;

	// Maximum torque the character can apply to rotate in air about the vertical axis
	UPROPERTY(EditAnywhere, Category = "Constraint Settings", meta = (ClampMin = "0", UIMin = "0", ForceUnits = "NewtonMeters"))
	float TwistTorqueLimit = 1000.0f;

	// Maximum torque the character can apply to remain upright
	UPROPERTY(EditAnywhere, Category = "Constraint Settings", meta = (ClampMin = "0", UIMin = "0", ForceUnits = "NewtonMeters"))
	float SwingTorqueLimit = 3000.0f;

	// Controls whether the character capsule is forced to remain upright
	UPROPERTY(EditAnywhere, Category = "Constraint Settings")
	bool bShouldCharacterRemainUpright = true;

	/**
	 * Optional override target height for the character (the desired distance from the center of the capsule to the floor).
	 * If left blank, the -Z offset of the owning character's skeletal mesh comp will be used automatically.
	 */
	UPROPERTY(EditAnywhere, Category = "Constraint Settings", meta = (ClampMin = "0", UIMin = "0", ForceUnits = "cm"))
	TOptional<float> TargetHeightOverride;

	// Radius used for ground queries
	UPROPERTY(EditAnywhere, Category = "Query Settings", meta = (ClampMin = "0", UIMin = "0", ForceUnits = "cm"))
	TOptional<float> QueryRadiusOverride;

	// Allows the mode to override friction on collision with other physics bodies.
	UPROPERTY(EditAnywhere, Category = "Collision Settings")
	ECharacterMoverFrictionOverrideMode FrictionOverrideMode = ECharacterMoverFrictionOverrideMode::OverrideToZeroWhenMoving;

private:
	float TargetHeight = 95.0f;
	float QueryRadius = 30.0f;

	TOptional<float> MaxSpeedOverride;
	TOptional<float> AccelerationOverride;
};