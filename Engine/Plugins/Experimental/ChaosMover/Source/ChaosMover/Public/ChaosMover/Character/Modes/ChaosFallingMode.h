// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosMover/Character/Modes/ChaosCharacterMovementMode.h"

#include "ChaosFallingMode.generated.h"

#define UE_API CHAOSMOVER_API

/**
 * Chaos character falling mode
 */
UCLASS(MinimalAPI, Blueprintable, BlueprintType, EditInlineNew, DefaultToInstanced)
class UChaosFallingMode : public UChaosCharacterMovementMode
{
	GENERATED_BODY()

public:
	UE_API UChaosFallingMode(const FObjectInitializer& ObjectInitializer);

	UE_API virtual void GenerateMove_Implementation(const FMoverSimContext& SimContext, const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, FProposedMove& OutProposedMove) const override;
	UE_API virtual void SimulationTick_Implementation(const FSimulationTickParams& Params, FMoverTickEndData& OutputState) override;

	/**
	 * If true, actor will land and lose all speed in the vertical direction upon landing. If false, actor's vertical speed will be redirected based on the surface normal it hit.
	 * Note: Actor's horizontal speed will not be affected if true. If false, horizontal speed may be increased on landing.
	 */
	UPROPERTY(Category = Mover, EditAnywhere, BlueprintReadWrite)
	bool bCancelVerticalSpeedOnLanding;

	/**
	 * When falling, amount of movement control available to the actor.
	 * 0 = no control, 1 = full control
	 */
	UPROPERTY(Category = Mover, EditAnywhere, BlueprintReadWrite, meta = (ClampMin = "0", ClampMax = "1.0"))
	float AirControlPercentage;

	/**
	 * Deceleration to apply to air movement when falling slower than terminal velocity.
	 * Note: This is NOT applied to vertical velocity, only movement plane velocity
	 */
	UPROPERTY(Category = Mover, EditAnywhere, BlueprintReadWrite, meta = (ClampMin = "0", ForceUnits = "cm/s^2"))
	float FallingDeceleration;

	/**
	 * Friction to apply to air movement.
	 * Lateral velocity is scaled each tick by a factor (1-Friction * TimeStep) so friction
	 * values greater than 1/TimeStep will result in all lateral velocity being removed.
	 * Note: This is NOT applied to vertical velocity, only movement plane velocity
	 */
	UPROPERTY(Category = Mover, EditAnywhere, BlueprintReadWrite, meta = (ClampMin = "0"))
	float FallingLateralFriction;

	/**
	 * Friction to apply to rotational air movement.
	 * Angular velocity is scaled each tick by a factor (1-Friction * TimeStep) so friction
	 * values greater than 1/TimeStep will result in all angular velocity being removed.
	 */
	UPROPERTY(Category = Mover, EditAnywhere, BlueprintReadWrite, meta = (ClampMin = "0"))
	float FallingAngularFriction;

	/**
	 * Deceleration to apply to air movement when falling faster than terminal velocity
	 * Note: This is NOT applied to vertical velocity, only movement plane velocity
	 */
	UPROPERTY(Category = Mover, EditAnywhere, BlueprintReadWrite, meta = (ClampMin = "0", ForceUnits = "cm/s^2"))
	float OverTerminalSpeedFallingDeceleration;

	/**
	 * If the actor's movement plane velocity is greater than this speed falling will start applying OverTerminalSpeedFallingDeceleration instead of FallingDeceleration
	 * The expected behavior is to set OverTerminalSpeedFallingDeceleration higher than FallingDeceleration so the actor will slow down faster
	 * when going over TerminalMovementPlaneSpeed.
	 */
	UPROPERTY(Category = Mover, EditAnywhere, BlueprintReadWrite, meta = (ClampMin = "0", ForceUnits = "cm/s"))
	float TerminalMovementPlaneSpeed;

	/** When exceeding maximum vertical speed, should it be enforced via a hard clamp? If false, VerticalFallingDeceleration will be used for a smoother transition to the terminal speed limit. */
	UPROPERTY(Category = Mover, EditAnywhere, BlueprintReadWrite)
	bool bShouldClampTerminalVerticalSpeed;

	/** Deceleration to apply to vertical velocity when it's greater than TerminalVerticalSpeed. Only used if bShouldClampTerminalVerticalSpeed is false. */
	UPROPERTY(Category = Mover, EditAnywhere, BlueprintReadWrite, meta = (EditCondition = "!bShouldClampTerminalVerticalSpeed", ClampMin = "0", ForceUnits = "cm/s^2"))
	float VerticalFallingDeceleration;

	/**
	 * If the actors vertical velocity is greater than this speed VerticalFallingDeceleration will be applied to vertical velocity
	 */
	UPROPERTY(Category = Mover, EditAnywhere, BlueprintReadWrite, meta = (ClampMin = "0", ForceUnits = "cm/s"))
	float TerminalVerticalSpeed;

	/**
	 * If true and input requests a velocity, whether to always clamp that velocity to max speed, regardless of input's bForceMaxSpeed flag
	 */
	UPROPERTY(Category = Mover, EditAnywhere, BlueprintReadWrite)
	bool bAlwaysClampRequestedVelocityToMaxSpeed = false;

	/**
	 * Whether the character can apply angular velocity to respond to stick input
	 */
	UPROPERTY(Category = Mover, EditAnywhere, BlueprintReadWrite)
	bool bShouldApplyAngularVelocityToTarget = true;

protected:

	/*
	* To allow other modes disable gravity
	*/
	uint8 bIgnoreGravity = 0;
};

#undef UE_API
