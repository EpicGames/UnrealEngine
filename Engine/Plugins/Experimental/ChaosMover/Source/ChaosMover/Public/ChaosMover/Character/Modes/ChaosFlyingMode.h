// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosMover/Character/Modes/ChaosCharacterMovementMode.h"

#include "ChaosFlyingMode.generated.h"

/**
 * Chaos character flying mode
 */
UCLASS(MinimalAPI, Blueprintable, BlueprintType, EditInlineNew, DefaultToInstanced)
class UChaosFlyingMode : public UChaosCharacterMovementMode
{
	GENERATED_BODY()

public:
	CHAOSMOVER_API UChaosFlyingMode(const FObjectInitializer& ObjectInitializer);

	CHAOSMOVER_API virtual void GenerateMove_Implementation(const FMoverSimContext& SimContext, const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, FProposedMove& OutProposedMove) const override;
	CHAOSMOVER_API virtual void SimulationTick_Implementation(const FSimulationTickParams& Params, FMoverTickEndData& OutputState) override;

	CHAOSMOVER_API virtual void UpdateCurrentFloor(const FMoverTimeStep& TimeStep) const override;
};
