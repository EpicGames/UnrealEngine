// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ChaosMover/Character/Modes/ChaosWalkingMode.h"

#include "ChaosSimpleWalkingMode.generated.h"

#define UE_API CHAOSMOVER_API

/**
 * Chaos walking mode variant that mirrors Mover's USimpleWalkingMode pattern:
 * - Compute DesiredVelocity + DesiredFacing from inputs
 * - Call GenerateWalkMove hook to allow derived modes to shape/smooth motion
 *
 * Still reuses UChaosWalkingMode's floor checking, step filtering, and contact modification.
 */
UCLASS(MinimalAPI, Blueprintable, BlueprintType, Abstract, EditInlineNew, DefaultToInstanced)
class UChaosSimpleWalkingMode : public UChaosWalkingMode
{
	GENERATED_BODY()

public:
	UE_API virtual void GenerateMove_Implementation(const FMoverSimContext& SimContext, const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, FProposedMove& OutProposedMove) const override;

	/**
	 * Override this to implement a Chaos "simple walking" variant.
	 * - DesiredVelocity is in world space, typically constrained to the movement plane.
	 * - DesiredFacing is the target world-space orientation.
	 */
	UFUNCTION(BlueprintNativeEvent, meta = (DisplayName = "Generate Simple Walk Move (Chaos)"))
	UE_API void GenerateWalkMove(UPARAM(ref) FMoverTickStartData& StartState, float DeltaSeconds, const FVector& DesiredVelocity,
		const FQuat& DesiredFacing, const FQuat& CurrentFacing, UPARAM(ref) FVector& InOutAngularVelocityDegrees, UPARAM(ref) FVector& InOutVelocity);

protected:
	UE_API virtual void GenerateWalkMove_Implementation(FMoverTickStartData& StartState, float DeltaSeconds, const FVector& DesiredVelocity,
		const FQuat& DesiredFacing, const FQuat& CurrentFacing, FVector& InOutAngularVelocityDegrees, FVector& InOutVelocity);
};

#undef UE_API

