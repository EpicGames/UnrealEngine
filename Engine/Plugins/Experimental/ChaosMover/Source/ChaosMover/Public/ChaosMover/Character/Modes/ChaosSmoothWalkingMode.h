// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ChaosMover/Character/Modes/ChaosSimpleWalkingMode.h"

#include "ChaosSmoothWalkingMode.generated.h"

#define UE_API CHAOSMOVER_API

/**
 * A Chaos walking mode that ports Mover's USmoothWalkingMode spring/smoothing model.
 * Uses FChaosSmoothWalkingState in SyncStateCollection to persist spring intermediates.
 */
UCLASS(MinimalAPI, BlueprintType, Experimental, EditInlineNew, DefaultToInstanced)
class UChaosSmoothWalkingMode : public UChaosSimpleWalkingMode
{
	GENERATED_BODY()

public:
	UE_API virtual void SimulationTick_Implementation(const FSimulationTickParams& Params, FMoverTickEndData& OutputState) override;

	UE_API virtual void GenerateWalkMove_Implementation(FMoverTickStartData& StartState, float DeltaSeconds, const FVector& DesiredVelocity,
		const FQuat& DesiredFacing, const FQuat& CurrentFacing, FVector& InOutAngularVelocityDegrees, FVector& InOutVelocity) override;

protected: // Velocity Controls

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ChaosMover|Smooth Walking Settings", meta = (ClampMin = "0", UIMin = "0", ForceUnits = "cm/s^2"))
	float Acceleration = 1500.0f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ChaosMover|Smooth Walking Settings", meta = (ClampMin = "0", UIMin = "0", ForceUnits = "cm/s^2"))
	float Deceleration = 1500.0f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ChaosMover|Advanced Smooth Walking Settings", meta = (ClampMin = "0", UIMin = "0", ClampMax = "1", UIMax = "1"))
	float DirectionalAccelerationFactor = 1.0f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ChaosMover|Smooth Walking Settings", meta = (ClampMin = "0", UIMin = "0"))
	float TurningStrength = 10.0f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ChaosMover|Smooth Walking Settings", meta = (ClampMin = "0", UIMin = "0", ForceUnits = "s"))
	float AccelerationSmoothingTime = 0.1f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ChaosMover|Smooth Walking Settings", meta = (ClampMin = "0", UIMin = "0", ForceUnits = "s"))
	float DecelerationSmoothingTime = 0.1f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ChaosMover|Advanced Smooth Walking Settings", meta = (ClampMin = "0", UIMin = "0", ClampMax = "1", UIMax = "1"))
	float AccelerationSmoothingCompensation = 0.0f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ChaosMover|Advanced Smooth Walking Settings", meta = (ClampMin = "0", UIMin = "0", ClampMax = "1", UIMax = "1"))
	float DecelerationSmoothingCompensation = 0.0f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ChaosMover|Advanced Smooth Walking Settings", meta = (ClampMin = "0", UIMin = "0", ForceUnits = "cm/s"))
	float VelocityDeadzoneThreshold = 0.01f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ChaosMover|Advanced Smooth Walking Settings", meta = (ClampMin = "0", UIMin = "0", ForceUnits = "cm/s^2"))
	float AccelerationDeadzoneThreshold = 0.001f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ChaosMover|Advanced Smooth Walking Settings", meta = (ClampMin = "0", UIMin = "0", ForceUnits = "s"))
	float OutsideInfluenceSmoothingTime = 0.05f;

protected: // Facing Controls

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ChaosMover|Smooth Walking Settings", meta = (ClampMin = "0", UIMin = "0", ForceUnits = "s"))
	float FacingSmoothingTime = 0.25f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ChaosMover|Advanced Smooth Walking Settings")
	bool bSmoothFacingWithDoubleSpring = true;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ChaosMover|Advanced Smooth Walking Settings", meta = (ClampMin = "0", UIMin = "0", ForceUnits = "deg"))
	float FacingDeadzoneThreshold = 0.1f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ChaosMover|Advanced Smooth Walking Settings", meta = (ClampMin = "0", UIMin = "0", ForceUnits = "deg/s"))
	float AngularVelocityDeadzoneThreshold = 0.01f;
};

#undef UE_API

