// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosMover/Character/Modes/ChaosCharacterMovementMode.h"

#include "ChaosSwimmingMode.generated.h"

#define UE_API CHAOSMOVER_API

// Controls for the Swimming Movement
USTRUCT(BlueprintType)
struct FSwimmingSettings
{
	GENERATED_BODY()

	// Max speed when moving up in water
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SwimmingControl)
	float MaxSpeedUp = 500.00f;

	// Max speed when moving down in water
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SwimmingControl)
	float MaxSpeedDown = 1000.00f;

	// At or above this depth, use max velocity. Interps down to WaterVelocityMinMultiplier at wading depth (where player can start swimming)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SwimmingControl)
	float WaterVelocityDepthForMax = 175.00f;

	// Min velocity multiplier applied when depth equals min swimming depth (where they transition from wading to swimming). Interps between this and 1.0 at WaterVelocityDepthForMax.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SwimmingControl)
	float WaterVelocityMinMultiplier = 0.50f;

	// Max water force, after WaterVelocity * (WaterForceMultiplier * WaterForceSecondMultiplier).
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SwimmingControl)
	float MaxWaterForce = 400.00f;

	// Multiplier to water force acceleration in direction of current.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SwimmingControl)
	float WaterForceMultiplier = 2.00f;

	// Multiplier applied on the top of WaterForceMultiplier, to water force acceleration in direction of current. Used only for inherited objects.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SwimmingControl)
	float WaterForceSecondMultiplier = 1.00f;

	// Bobbing: Max force
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SwimmingControl)
	float BobbingMaxForce = 3800.00f;

	// Bobbing: Slow down strongly when within this tolerance of the ideal immersion depth. Normally we apply drag only when going away from the ideal depth, this allows some slowdown when approaching it.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SwimmingControl)
	float BobbingIdealDepthTolerance = 7.50f;

	// Bobbing: friction/drag opposed to downward velocity, linear multiplier per second
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SwimmingControl)
	float BobbingFrictionDown = 5.00f;

	// Bobbing: friction/drag opposed to downward velocity, squared with velocity per second. Ramps up faster with higher speeds, less effect at low speeds.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SwimmingControl)
	float BobbingExpDragDown = 0.05f;

	// Bobbing: friction/drag opposed to downward velocity, linear multiplier per second. Only used when fully submerged (replaces other value).
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SwimmingControl)
	float BobbingFrictionDownSubmerged = 7.50f;

	// Bobbing: friction/drag opposed to upward velocity, squared with velocity per second. Ramps up faster with higher speeds, less effect at low speeds.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SwimmingControl)
	float BobbingExpDragDownSubmerged = 0.10f;

	// Bobbing: friction/drag opposed to upward velocity, linear multiplier per second
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SwimmingControl)
	float BobbingFrictionUp = 7.00f;

	// Bobbing: friction/drag opposed to upward velocity, squared with velocity per second. Ramps up faster with higher speeds, less effect at low speeds.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SwimmingControl)
	float BobbingExpDragUp = 0.20f;

	// Bobbing: friction multiplier, multiplies the fluid friction value.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SwimmingControl)
	float BobbingFrictionMultiplier = 1.f;

	// Bobbing: multiplier for the exponential drag.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SwimmingControl)
	float BobbingExpDragMultiplier = 1.f;
};

/**
 * Chaos character swimming mode
 */
UCLASS(MinimalAPI, Blueprintable, BlueprintType, EditInlineNew, DefaultToInstanced)
class UChaosSwimmingMode : public UChaosCharacterMovementMode
{
	GENERATED_BODY()

public:
	UE_API UChaosSwimmingMode(const FObjectInitializer& ObjectInitializer);

	// Ideal depth when in water. Measured from the center of the collision shape.
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Water Control")
	float SwimmingIdealImmersionDepth = 43.f;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Water Control")
	FSwimmingSettings SurfaceSwimmingWaterControlSettings;
	
	UE_API virtual void GenerateMove_Implementation(const FMoverSimContext& SimContext, const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, FProposedMove& OutProposedMove) const override;
	UE_API virtual void SimulationTick_Implementation(const FSimulationTickParams& Params, FMoverTickEndData& OutputState) override;

protected:
	float OriginalHalfHeight = 75.f;
};

#undef UE_API
