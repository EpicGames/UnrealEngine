// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosMover/Character/Modes/ChaosCharacterMovementMode.h"

#include "ChaosWalkingMode.generated.h"

#define UE_API CHAOSMOVER_API

struct FFloorCheckResult;
struct FWaterCheckResult;

/**
 * Chaos character walking mode
 */
UCLASS(MinimalAPI, Blueprintable, BlueprintType, EditInlineNew, DefaultToInstanced)
class UChaosWalkingMode : public UChaosCharacterMovementMode
{
	GENERATED_BODY()

public:
	UE_API UChaosWalkingMode(const FObjectInitializer& ObjectInitializer);

	// Damping factor to control the softness of the interaction between the character and the ground
	// Set to 0 for no damping and 1 for maximum damping
	UPROPERTY(EditAnywhere, Category = "Constraint Settings", meta = (ClampMin = "0", UIMin = "0", ClampMax = "1", UIMax = "1"))
	float GroundDamping = 0.0f;

	// Maximum force the character can apply to hold in place while standing on an unwalkable incline
	UPROPERTY(EditAnywhere, Category = "Constraint Settings", meta = (ClampMin = "0", UIMin = "0", ForceUnits = "Newtons"))
	float FrictionForceLimit = 100.0f;

	// Scaling applied to the radial force limit to raise the limit to always allow the character to
	// reach the motion target/
	// A value of 1 means that the radial force limit will be increased as needed to match the force
	// required to achieve the motion target.
	// A value of 0 means no scaling will be applied.
	UPROPERTY(EditAnywhere, Category = "Constraint Settings", meta = (ClampMin = "0", UIMin = "0", ClampMax = "1", UIMax = "1"))
	float FractionalRadialForceLimitScaling = 1.0f;

	// Controls the reaction force applied to the ground in the ground plane when the character is moving
	// A value of 1 means that the full reaction force is applied
	// A value of 0 means the character only applies a normal force to the ground and no tangential force
	UPROPERTY(EditAnywhere, Category = "Constraint Settings", meta = (ClampMin = "0", UIMin = "0", ClampMax = "1", UIMax = "1"))
	float FractionalGroundReaction = 1.0f;

	// Controls how much downward velocity is applied to keep the character rooted to the ground when the character
	// is within MaxStepHeight of the ground surface.
	UPROPERTY(EditAnywhere, Category = "Movement Settings", meta = (ClampMin = "0", UIMin = "0", ClampMax = "1", UIMax = "1"))
	float FractionalDownwardVelocityToTarget = 1.0f;

	/**
	 * If true, walking movement always maintains horizontal velocity when moving up ramps, which causes movement up ramps to be faster parallel to the ramp surface.
	 * If false, then walking movement maintains velocity magnitude parallel to the ramp surface.
	 */
	UPROPERTY(EditAnywhere, Category = "Movement Settings")
	uint8 bMaintainHorizontalGroundVelocity : 1 = 0;

	/**
	 * Whether the character can apply angular velocity to respond to stick input
	 */
	UPROPERTY(Category = Mover, EditAnywhere, BlueprintReadWrite)
	bool bShouldApplyAngularVelocityToTarget = true;

	UE_API virtual void UpdateConstraintSettings(Chaos::FCharacterGroundConstraintSettings& ConstraintSettings) const override;

	UE_API virtual void GenerateMove_Implementation(const FMoverSimContext& SimContext, const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, FProposedMove& OutProposedMove) const override;
	UE_API virtual void SimulationTick_Implementation(const FSimulationTickParams& Params, FMoverTickEndData& OutputState) override;

	UE_API virtual void ModifyContacts(const FMoverTimeStep& TimeStep, const UE::ChaosMover::FSimulationInputData& InputData, const UE::ChaosMover::FSimulationOutputData& OutputData, Chaos::FCollisionContactModifier& Modifier) const override;

protected:
	UE_API virtual bool CanStepUpOnHitSurface(const FFloorCheckResult& FloorResult, const FVector& CharacterVelocity) const;
	UE_API virtual void GetFloorAndCheckMovement(const FMoverDefaultSyncState& SyncState, const FProposedMove& ProposedMove, const FChaosMoverSimulationDefaultInputs& DefaultSimInputs, float DeltaSeconds, FFloorCheckResult& FloorResult, FWaterCheckResult& WaterResult, FVector& OutDeltaPos) const;
};

#undef UE_API
