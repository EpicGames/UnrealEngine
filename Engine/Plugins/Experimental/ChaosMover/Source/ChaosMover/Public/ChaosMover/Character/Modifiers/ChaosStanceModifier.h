// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovementModifier.h"
#include "Chaos/ParticleHandleFwd.h"
#include "ChaosMover/ChaosMoverSimulationTypes.h"
#include "DefaultMovementSet/MovementModifiers/StanceModifier.h"

#include "ChaosStanceModifier.generated.h"


USTRUCT(MinimalAPI, BlueprintType)
struct FChaosStanceModifier : public FMovementModifierBase
{
	GENERATED_BODY()

public:
	CHAOSMOVER_API FChaosStanceModifier();
	virtual ~FChaosStanceModifier() override = default;

	EStanceMode ActiveStance;

	/** Whether to cancel the modifier when the movement mode changes */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Parameters)
	bool bCancelOnModeChange = false;

	/** Height of the modified capsule */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Parameters, meta = (ClampMin = "0", UIMin = "0", ForceUnits = "cm"))
	float ModifiedCapsuleHalfHeight = 60.0f;

	/** Radius of the modified capsule */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Parameters, meta = (ClampMin = "0", UIMin = "0", ForceUnits = "cm"))
	float ModifiedCapsuleRadius = 30.0f;

	/** Ground clearance of the modified capsule */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Parameters, meta = (ClampMin = "0", UIMin = "0", ForceUnits = "cm"))
	float ModifiedCapsuleGroundClearance = 10.0f;

	/** Height of the default unmodified capsule */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Parameters, meta = (ClampMin = "0", UIMin = "0", ForceUnits = "cm"))
	float DefaultCapsuleHalfHeight = 80.0f;

	/** Radius of the default unmodified capsule */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Parameters, meta = (ClampMin = "0", UIMin = "0", ForceUnits = "cm"))
	float DefaultCapsuleRadius = 30.0f;

	/** Ground clearance of the default unmodified capsule */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Parameters, meta = (ClampMin = "0", UIMin = "0", ForceUnits = "cm"))
	float DefaultCapsuleGroundClearance = 10.0f;

	/** Override Max Speed */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Parameters, meta = (ClampMin = "0", UIMin = "0", ForceUnits = "cm/s"))
	TOptional<float> MaxSpeedOverride;

	/** Override Acceleration */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Parameters, meta = (ClampMin = "0", UIMin = "0", ForceUnits = "cm/s^2"))
	TOptional<float> AccelerationOverride;

	CHAOSMOVER_API virtual bool HasGameplayTag(FGameplayTag TagToFind, bool bExactMatch) const override;
	CHAOSMOVER_API virtual void GetGameplayTags(FGameplayTagContainer& InOutTags) const override;

	CHAOSMOVER_API virtual void OnStart_Async(const FMovementModifierParams_Async& Params) override;
	CHAOSMOVER_API virtual void OnEnd_Async(const FMovementModifierParams_Async& Params) override;
	CHAOSMOVER_API virtual void OnPostMovement_Async(const FMovementModifierParams_Async& Params) override;

	CHAOSMOVER_API virtual FMovementModifierBase* Clone() const override;
	CHAOSMOVER_API virtual void NetSerialize(FArchive& Ar) override;
	CHAOSMOVER_API virtual UScriptStruct* GetScriptStruct() const override;
	CHAOSMOVER_API virtual FString ToSimpleString() const override;
	CHAOSMOVER_API virtual void AddReferencedObjects(class FReferenceCollector& Collector) override;

protected:
	CHAOSMOVER_API virtual void UpdateStance(const FMovementModifierParams_Async& AsyncParams, EStanceMode NewStance);

	CHAOSMOVER_API void UpdateCapsule(Chaos::FPBDRigidParticleHandle* ParticleHandle, float NewRadius, float NewHalfHeight, float NewGroundClearance, float TargetHeight);
	CHAOSMOVER_API const Chaos::FCapsule* GetCapsule(Chaos::FPBDRigidParticleHandle* ParticleHandle) const;

	FName CurrentModeName = NAME_None;
};

template<>
struct TStructOpsTypeTraits< FChaosStanceModifier > : public TStructOpsTypeTraitsBase2< FChaosStanceModifier >
{
	enum
	{
		WithCopy = true
	};
};
