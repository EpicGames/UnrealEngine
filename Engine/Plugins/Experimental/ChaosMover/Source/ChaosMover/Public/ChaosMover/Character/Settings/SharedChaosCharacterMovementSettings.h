// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovementMode.h"

#include "SharedChaosCharacterMovementSettings.generated.h"

/**
 * SharedChaosCharacterMovementSettings: collection of settings that are shared between the chaos character movement modes
 */
UCLASS(MinimalAPI, BlueprintType)
class USharedChaosCharacterMovementSettings : public UObject, public IMovementSettingsInterface
{
	GENERATED_BODY()

	virtual FString GetDisplayName() const override
	{
		return GetName();
	}

public:
	float GetMaxWalkableSlopeCosine() const
	{
		return FMath::Cos(FMath::DegreesToRadians(MaxWalkableSlopeAngle));
	}

	/** Default max linear rate of deceleration when there is no controlled input */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "General", meta = (ClampMin = "0", UIMin = "0", ForceUnits = "cm/s^2"))
	float Deceleration = 4000.f;

	/** Default max linear rate of acceleration for controlled input. May be scaled based on magnitude of input. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "General", meta = (ClampMin = "0", UIMin = "0", ForceUnits = "cm/s^2"))
	float Acceleration = 4000.f;

	/** Maximum rate of turning rotation (degrees per second). Negative numbers indicate instant rotation and should cause rotation to snap instantly to desired direction. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "General", meta = (ClampMin = "-1", UIMin = "0", ForceUnits = "degrees/s"))
	float TurningRate = 500.f;

	/** Speeds velocity direction changes while turning, to reduce sliding */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "General", meta = (ClampMin = "0", UIMin = "0", ForceUnits = "Multiplier"))
	float TurningBoost = 8.f;

	/** Maximum speed in the movement plane */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "General", meta = (ClampMin = "0", UIMin = "0", ForceUnits = "cm/s"))
	float MaxSpeed = 800.f;

	/**
	 * Should use acceleration for velocity based movement intent?
	 * If true, acceleration is applied when using velocity input to reach the target velocity.
	 * If false, velocity is set directly, disregarding acceleration.
	 */
	UPROPERTY(Category = "General", EditAnywhere, BlueprintReadWrite, AdvancedDisplay)
	bool bUseAccelerationForVelocityMove = true;

	/**
	 * Setting that affects movement control. Higher values allow faster changes in direction. This can be used to simulate slippery
	 * surfaces such as ice or oil by lowering the value (possibly based on the material the actor is standing on).
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "General|Friction", meta = (ClampMin = "0", UIMin = "0"))
	float GroundFriction = 8.0f;

	/**
	  * If true, BrakingFriction will be used to slow the character to a stop (when there is no Acceleration).
	  * If false, braking uses the same friction passed to CalcVelocity() (ie GroundFriction when walking), multiplied by BrakingFrictionFactor.
	  * This setting applies to all movement modes; if only desired in certain modes, consider toggling it when movement modes change.
	  * @see BrakingFriction
	  */
	UPROPERTY(Category = "General|Friction", EditDefaultsOnly, BlueprintReadWrite)
	uint8 bUseSeparateBrakingFriction : 1;

	/**
	 * Friction (drag) coefficient applied when braking (whenever Acceleration = 0, or if character is exceeding max speed); actual value used is this multiplied by BrakingFrictionFactor.
	 * When braking, this property allows you to control how much friction is applied when moving across the ground, applying an opposing force that scales with current velocity.
	 * Braking is composed of friction (velocity-dependent drag) and constant deceleration.
	 * This is the current value, used in all movement modes; if this is not desired, override it or bUseSeparateBrakingFriction when movement mode changes.
	 * @note Only used if bUseSeparateBrakingFriction setting is true, otherwise current friction such as GroundFriction is used.
	 * @see bUseSeparateBrakingFriction, BrakingFrictionFactor, GroundFriction, BrakingDecelerationWalking
	 */
	UPROPERTY(Category = "General|Friction", EditAnywhere, BlueprintReadWrite, meta = (ClampMin = "0", UIMin = "0", EditCondition = "bUseSeparateBrakingFriction"))
	float BrakingFriction = 8.0f;

	/**
	 * Factor used to multiply actual value of friction used when braking.
	 * This applies to any friction value that is currently used, which may depend on bUseSeparateBrakingFriction.
	 * @note This is 2 by default for historical reasons, a value of 1 gives the true drag equation.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "General|Friction", meta = (ClampMin = "0", UIMin = "0"))
	float BrakingFrictionFactor = 2.0f;

	/** Mover actors will be able to step up onto or over obstacles shorter than this */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Ground Movement", meta = (ClampMin = "0", UIMin = "0", ForceUnits = "cm"))
	float MaxStepHeight = 40.0f;

	// Default movement mode to use when falling.
	UPROPERTY(Category = "General", EditAnywhere, BlueprintReadWrite)
	FName DefaultFallingMode = DefaultModeNames::Falling;

	// Maximum angle of slope that the character can walk on without sliding
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Ground Movement", meta = (ClampMin = "0", UIMin = "0", ClampMax = "90", UIMax = "90", ForceUnits = "degrees"))
	float MaxWalkableSlopeAngle = 45.0f;
};
