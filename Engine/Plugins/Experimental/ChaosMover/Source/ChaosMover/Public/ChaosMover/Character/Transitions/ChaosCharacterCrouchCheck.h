// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosMover/ChaosMovementModeTransition.h"

#include "ChaosCharacterCrouchCheck.generated.h"


UCLASS(MinimalAPI, Blueprintable, BlueprintType, EditInlineNew, DefaultToInstanced)
class UChaosCharacterCrouchCheck : public UChaosMovementModeTransition
{
	GENERATED_BODY()

public:
	CHAOSMOVER_API UChaosCharacterCrouchCheck(const FObjectInitializer& ObjectInitializer);

	CHAOSMOVER_API virtual void OnRegistered();
	CHAOSMOVER_API virtual void OnUnregistered();

	CHAOSMOVER_API virtual FTransitionEvalResult Evaluate_Implementation(const FSimulationTickParams& Params) const override;
	CHAOSMOVER_API virtual void Trigger_Implementation(const FSimulationTickParams& Params) override;

	/** Whether to cancel the modifier when the movement mode changes */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Parameters)
	bool bCancelOnModeChange = false;

	/** Height of the modified capsule */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Parameters, meta = (ClampMin = "0", UIMin = "0", ForceUnits = "cm"))
	TOptional<float> CapsuleHalfHeight;

	/** Radius of the modified capsule */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Parameters, meta = (ClampMin = "0", UIMin = "0", ForceUnits = "cm"))
	TOptional<float> CapsuleRadius;

	/** Ground clearance of the modified capsule */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Parameters, meta = (ClampMin = "0", UIMin = "0", ForceUnits = "cm"))
	TOptional<float> CapsuleGroundClearance;

	/** Override Max Speed */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Parameters, meta = (ClampMin = "0", UIMin = "0", ForceUnits = "cm/s"))
	TOptional<float> MaxSpeed;

	/** Override Acceleration */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Parameters, meta = (ClampMin = "0", UIMin = "0", ForceUnits = "cm/s^2"))
	TOptional<float> Acceleration;

	/** Name of movement mode to transition to when crouched */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Parameters)
	TOptional<FName> TransitionToCrouchedMode;

	/** Name of movement mode to transition to when uncrouched */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Parameters)
	TOptional<FName> TransitionToUncrouchedMode;

protected:
	CHAOSMOVER_API virtual bool CanCrouch(const FSimulationTickParams& Params) const;
	CHAOSMOVER_API virtual bool CanUncrouch(const FSimulationTickParams& Params) const;

	CHAOSMOVER_API virtual void Crouch(const FSimulationTickParams& Params);
	CHAOSMOVER_API virtual void Uncrouch(const FSimulationTickParams& Params);

	float OriginalCapsuleHalfHeight = 0.0f;
	float OriginalCapsuleRadius = 0.0f;

	mutable bool bTriggerCrouch = false;
	mutable bool bTriggerUncrouch = false;
};