// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosMover/ChaosMovementModeTransition.h"

#include "ChaosCharacterFallingCheck.generated.h"

#define UE_API CHAOSMOVER_API


UCLASS(MinimalAPI, Blueprintable, BlueprintType, EditInlineNew, DefaultToInstanced)
class UChaosCharacterFallingCheck : public UChaosMovementModeTransition
{
	GENERATED_BODY()

public:
	UE_API UChaosCharacterFallingCheck(const FObjectInitializer& ObjectInitializer);

	UE_API virtual void OnRegistered() override;

	UE_API virtual FTransitionEvalResult Evaluate_Implementation(const FSimulationTickParams& Params) const override;
	UE_API virtual void Trigger_Implementation(const FSimulationTickParams& Params) override;

	/** Name of movement mode to transition to when landing on ground */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Parameters)
	FName TransitionToFallingMode;

	// Time limit for being unsupported before moving from a walking to a falling state.
	// This provides some grace period when walking off of an edge during which locomotion
	// and jumping are still possible even though the character has started falling under gravity
	UPROPERTY(EditAnywhere, Category = Parameters, meta = (ClampMin = "0", UIMin = "0", ForceUnits = "s"))
	float MaxUnsupportedTimeBeforeFalling = 0.06f;

protected:
	TWeakObjectPtr<const class USharedChaosCharacterMovementSettings> SharedSettingsWeakPtr;
};

#undef UE_API
