// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosMover/ChaosMovementModeTransition.h"

#include "ChaosCharacterJumpCheck.generated.h"

#define UE_API CHAOSMOVER_API


UCLASS(MinimalAPI, Blueprintable, BlueprintType, EditInlineNew, DefaultToInstanced)
class UChaosCharacterJumpCheck : public UChaosMovementModeTransition
{
	GENERATED_BODY()

public:
	UE_API UChaosCharacterJumpCheck(const FObjectInitializer& ObjectInitializer);

	UE_API virtual FTransitionEvalResult Evaluate_Implementation(const FSimulationTickParams& Params) const override;
	UE_API virtual void Trigger_Implementation(const FSimulationTickParams& Params) override;

	/** Instantaneous speed induced in an actor upon jumping */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Parameters, meta = (ClampMin = "0", UIMin = "0", ForceUnits = "cm/s"))
	float JumpUpwardsSpeed = 700.0f;

	/**
	* Controls how much of the jump impulse the character will apply to the ground.
	* A value of 0 means no impulse will be applied to the ground.
	* A value of 1 means that the full equal and opposite jump impulse will be applied.
	*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Parameters, meta = (ClampMin = "0", UIMin = "0", ClampMax = "1", UIMax = "1"))
	float FractionalGroundReactionImpulse = 1.0f;

	/** Name of movement mode to transition to when jumping */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Parameters)
	FName TransitionToMode;
};

#undef UE_API
