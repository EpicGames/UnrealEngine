// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosMover/ChaosMovementModeTransition.h"
#include "DefaultMovementSet/CharacterMoverSimulationTypes.h"

#include "ChaosCharacterLandingCheck.generated.h"

#define UE_API CHAOSMOVER_API


UCLASS(MinimalAPI, Blueprintable, BlueprintType, EditInlineNew, DefaultToInstanced)
class UChaosCharacterLandingCheck : public UChaosMovementModeTransition
{
	GENERATED_BODY()

public:
	UE_API UChaosCharacterLandingCheck(const FObjectInitializer& ObjectInitializer);

	UE_API virtual FTransitionEvalResult Evaluate_Implementation(const FSimulationTickParams& Params) const override;
	UE_API virtual void Trigger_Implementation(const FSimulationTickParams& Params) override;

	/** Height at which we consider the character to be on the ground */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Parameters, meta = (Units = "cm"))
	float FloorDistanceTolerance = 0.5f;

	/** Name of movement mode to transition to when landing on ground */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Parameters)
	FName TransitionToGroundMode;

protected:
	/** Create the event data we will use for the landed event*/
	UE_API virtual TSharedPtr<FLandedEventData> AllocLandedEventData();
	
	/** Used to fill the landed event data before adding the Landed Event into the simulation */
	UE_API virtual void PopulateLandedEventData(const FSimulationTickParams& Params, const FFloorCheckResult& FloorResult, FLandedEventData& OutLandedEvent);
};

#undef UE_API
