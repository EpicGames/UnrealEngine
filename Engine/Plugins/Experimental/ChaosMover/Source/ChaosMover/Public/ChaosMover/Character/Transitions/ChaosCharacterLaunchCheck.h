// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosMover/ChaosMovementModeTransition.h"

#include "ChaosCharacterLaunchCheck.generated.h"

/**
 * Transition that handles launching based on input for a physics-based character
 */
UCLASS(MinimalAPI, Blueprintable, BlueprintType)
class UChaosCharacterLaunchCheck : public UChaosMovementModeTransition
{
	GENERATED_BODY()

public:
	CHAOSMOVER_API UChaosCharacterLaunchCheck(const class FObjectInitializer& ObjectInitializer);

	CHAOSMOVER_API virtual FTransitionEvalResult Evaluate_Implementation(const FSimulationTickParams& Params) const override;
	CHAOSMOVER_API virtual void Trigger_Implementation(const FSimulationTickParams& Params) override;

	/**
	Optional: Name of movement mode to transition to when launch is activated.
	If set to None launching will not change the current mode
	*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Parameters)
	FName TransitionToMode;
};
