// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosMover/ChaosMovementModeTransition.h"

#include "ChaosCharacterWaterCheck.generated.h"

#define UE_API CHAOSMOVER_API


UCLASS(MinimalAPI, Blueprintable, BlueprintType, EditInlineNew, DefaultToInstanced)
class UChaosCharacterWaterCheck : public UChaosMovementModeTransition
{
	GENERATED_BODY()

public:
	UE_API UChaosCharacterWaterCheck(const FObjectInitializer& ObjectInitializer);

	UE_API virtual FTransitionEvalResult Evaluate_Implementation(const FSimulationTickParams& Params) const override;
	UE_API virtual void Trigger_Implementation(const FSimulationTickParams& Params) override;

	// Depth at which the pawn starts water mode. Measured from the center of the collision shape.
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Parameters, meta = (Units = "cm"))
	float WaterModeStartImmersionDepth = 45.f;

	// Depth at which the pawn stops water mode. Measured from the center of the collision shape.
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Parameters, meta = (Units = "cm"))
	float WaterModeStopImmersionDepth = 40.f;

	// Name of movement mode to transition to when immersed in water.
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Parameters)
	FName WaterModeName;

	// Name of movement mode to transition to when ground is within reach.
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Parameters)
	FName GroundModeName;

	// Name of movement mode to transition to when exiting water but ground is not in reach.
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Parameters)
	FName AirModeName;
};

#undef UE_API
