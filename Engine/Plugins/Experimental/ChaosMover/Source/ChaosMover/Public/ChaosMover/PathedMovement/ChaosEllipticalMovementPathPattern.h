// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ChaosMover/PathedMovement/ChaosPathedMovementPatternBase.h"

#include "ChaosEllipticalMovementPathPattern.generated.h"

UCLASS()
class UChaosEllipticalMovementPathPattern : public UChaosPathedMovementPatternBase
{
	GENERATED_BODY()
	
public:
	virtual void InitializePattern(UChaosMoverSimulation* InSimulation) override;

protected:
	virtual FTransform CalcUnmaskedTargetTransform(float OverallPathFraction, const FTransform& BasisTransform) const override;

	/** Radius of the ellipse along the x axis */
	UPROPERTY(EditAnywhere, Category = EllipticalPathPattern)
	float RadiusX = 100.f;
	
	UPROPERTY(EditAnywhere, Category = EllipticalPathPattern)
	float RadiusY = 100.f;

	/** The angle between the path origin (i.e. the initial location of the component/particle) and the center of the ellipse */
	UPROPERTY(EditAnywhere, Category = EllipticalPathPattern, meta = (Units = Degrees, UIMin = 0, ClampMin = 0, UIMax = 360, ClampMax = 360))
	float OriginAngle = 0.f;

	/** The amount of the ellipsoid arc to actually use for the path (where 360 is the entire ellipse) */
	UPROPERTY(EditAnywhere, Category = EllipticalPathPattern, meta = (Units = Degrees, UIMin = 0, ClampMin = 0, UIMax = 360, ClampMax = 360))
	float UsableArcAngle = 360.f;
	
	/** The world rotation of the plane the ellipse is on */
	UPROPERTY(EditAnywhere, Category = EllipticalPathPattern)
	FRotator EllipsePlaneRotation = FRotator::ZeroRotator;
};