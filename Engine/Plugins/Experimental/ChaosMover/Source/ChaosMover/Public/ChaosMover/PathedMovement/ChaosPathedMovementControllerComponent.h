// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "ChaosMover/ChaosMoverSimulationTypes.h"
#include "ChaosMover/PathedMovement/ChaosPathedMovementTypes.h"
#include "ChaosMover/PathedMovement/ChaosPathedMovementDebugDrawComponent.h"

#include "ChaosPathedMovementControllerComponent.generated.h"

namespace Chaos
{
class FPBDRigidsSolver;
}
class UMoverComponent;
class UChaosPathedMovementDebugDrawComponent;

// Fired on the game thread when it has received notification that the pathed movement has started
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FChaosMover_OnPathedMovementStarted);
// Fired on the game thread when it has received notification that the pathed movement has stopped
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FChaosMover_OnPathedMovementStopped, bool, bReachedEndOfPlayback);
// Fired on the game thread when it has received notification that the pathed movement has bounced (reached the end of one leg in a round trip)
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FChaosMover_OnPathedMovementBounced);
// Fired on the game thread when reverse playback has changed
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FChaosMover_OnPathedMovementReversePlaybackChanged, bool, bIsReversePlayback);
// Fired on the game thread when looping playback has changed
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FChaosMover_OnPathedMovementLoopingPlaybackChanged, bool, bIsLoopingPlayback);
// Fired on the game thread when one way playback has changed
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FChaosMover_OnPathedMovementOneWayPlaybackChanged, bool, bIsOneWayPlayback);

/** Execution type for pathed movement modifications, i.e. whether the modification is executed on the authority only or predicted on the client */
UENUM(BlueprintType)
enum class EChaosPathedMovementExecutionType : uint8
{
	// Only executed if attempted on the authority.
	// If attempted on non authoritative client, this action will be ignored.
	// This will execute on an authoritative client (client only actors)
	AuthorityOnly UMETA(DisplayName = "Authority Only"),
	// If the actor is autonomous proxy, this action will execute predictively.
	// This will also result in an input that will cause the action to execute on the server and replicate to sim proxies.
	// If the actor is simulated proxy, this will be ignored.
	// On the server, this will be ignored, relying on input replication instead.
	// This will execute on an authoritative client (client only actors)
	ClientPredicted_AutonomousOnly UMETA(DisplayName = "Client Predicted - Autonomous Only"),
};

/** 
*   This controller component gives control over pathed movement of an actor with a mover component and an active pathed movement mode.
*   You can start/stop, play forward or in reverse, play once or loop and play one way (0 to 1) or round trip (0 to 1 to 0).
*   In async mode, control requests are sent as scheduled to take effect at a future frame, to give enough time for all end points
*   to receive the request and react at the same exact scheduled frame.
*/
UCLASS(BlueprintType, Blueprintable, meta = (BlueprintSpawnableComponent))
class UChaosPathedMovementControllerComponent : public UActorComponent
	, public IMoverInputProducerInterface
	, public IChaosPathedMovementDebugDrawInterface
{
	GENERATED_BODY()

public:
	UChaosPathedMovementControllerComponent();

	// UActorComponent
	virtual void BeginPlay() override;
	virtual void OnRegister() override;
	virtual void OnUnregister() override;

	// IMoverInputProducerInterface
	virtual void ProduceInput_Implementation(int32 SimTimeMs, FMoverInputCmdContext& InputCmdResult) override;

	// Pathed Movement Control Layer
	// Start/Stop Playing Path
	/** Request to start or resume moving along the path. If a previous non looping playback reached the end naturally, this will restart playback.
	*   @param ExecutionType The execution type of this action @see EChaosPathedMovementExecutionType
	*   @param bIsScheduled If true, the execution will be delayed to leave enough time for all end points to receive request and start at the same frame to avoid a correction. @see UChaosMoverSettings::PathedMovementSchedulingDelaySeconds
	*/
	UFUNCTION(BlueprintCallable, Category = "Pathed Movement")
	CHAOSMOVER_API void RequestStartPlayingPath(EChaosPathedMovementExecutionType ExecutionType = EChaosPathedMovementExecutionType::AuthorityOnly, bool bIsScheduled = true);
	/** Request to stop playback. */
	UFUNCTION(BlueprintCallable, Category = "Pathed Movement")
	CHAOSMOVER_API void RequestStopPlayingPath(EChaosPathedMovementExecutionType ExecutionType = EChaosPathedMovementExecutionType::AuthorityOnly, bool bIsScheduled = true);
	/** Do we want to be starting or resuming playback? */
	UFUNCTION(BlueprintCallable, Category = "Pathed Movement")
	CHAOSMOVER_API bool WantsPlayingPath() const;
	/** Is a playback under way (not just requested) or stopped? */
	UFUNCTION(BlueprintCallable, Category = "Pathed Movement")
	CHAOSMOVER_API bool IsPlayingPath() const;
	/** Broadcast when the pathed movement has started */
	UPROPERTY(BlueprintAssignable, Category = "Pathed Movement")
	FChaosMover_OnPathedMovementStarted OnPathedMovementStarted;
	/** Broadcast when the pathed movement has stopped */
	UPROPERTY(BlueprintAssignable, Category = "Pathed Movement")
	FChaosMover_OnPathedMovementStopped OnPathedMovementStopped;
	/** Broadcast when the pathed movement has bounced (reached the end of one leg in a round trip) */
	UPROPERTY(BlueprintAssignable, Category = "Pathed Movement")
	FChaosMover_OnPathedMovementBounced OnPathedMovementBounced;
	// Reverse Playback
	/** Request to set playback in reverse (bWantsReversePlayback = True) or forward (False) */
	UFUNCTION(BlueprintCallable, Category = "Pathed Movement")
	CHAOSMOVER_API void RequestReversePlayback(bool bWantsReversePlayback, EChaosPathedMovementExecutionType ExecutionType = EChaosPathedMovementExecutionType::AuthorityOnly, bool bIsScheduled = true);
	/** Do we want the playback to be in reverse? */
	UFUNCTION(BlueprintCallable, Category = "Pathed Movement")
	CHAOSMOVER_API bool WantsReversePlayback() const;
	/** Is the playback in reverse */
	UFUNCTION(BlueprintCallable, Category = "Pathed Movement")
	CHAOSMOVER_API bool IsReversePlayback() const;
	/** Broadcast when reverse has changed */
	UPROPERTY(BlueprintAssignable, Category = "Pathed Movement")
	FChaosMover_OnPathedMovementReversePlaybackChanged OnPathedMovementReversePlaybackChanged;
	/** Broadcast when the pathed movement has stopped */
	// Looping Playback
	/** Request the playback to be looping (True) or not (False)*/
	UFUNCTION(BlueprintCallable, Category = "Pathed Movement")
	CHAOSMOVER_API void RequestLoopingPlayback(bool bWantsLoopingPlayback, EChaosPathedMovementExecutionType ExecutionType = EChaosPathedMovementExecutionType::AuthorityOnly, bool bIsScheduled = true);
	/** Do we want the playback to be looping or forward? */
	UFUNCTION(BlueprintCallable, Category = "Pathed Movement")
	CHAOSMOVER_API bool WantsLoopingPlayback() const;
	/** Is the playback looping */
	UFUNCTION(BlueprintCallable, Category = "Pathed Movement")
	CHAOSMOVER_API bool IsLoopingPlayback() const;
	/** Broadcast when looping has changed */
	UPROPERTY(BlueprintAssignable, Category = "Pathed Movement")
	FChaosMover_OnPathedMovementLoopingPlaybackChanged OnPathedMovementLoopingPlaybackChanged;
	// One-way vs. round trip playback
	/** Request the playback be one way (0->1), or round trip (there and back, 0->1->0) */
	UFUNCTION(BlueprintCallable, Category = "Pathed Movement")
	CHAOSMOVER_API void RequestOneWayPlayback(bool bWantsOneWayPlayback, EChaosPathedMovementExecutionType ExecutionType = EChaosPathedMovementExecutionType::AuthorityOnly, bool bIsScheduled = true);
	UFUNCTION(BlueprintCallable, Category = "Pathed Movement")
	/** Do we want the playback to be one way or round trip? */
	CHAOSMOVER_API bool WantsOneWayPlayback() const;
	/** Is the playback one way or round trip */
	UFUNCTION(BlueprintCallable, Category = "Pathed Movement")
	CHAOSMOVER_API bool IsOneWayPlayback() const;
	/** Broadcast when one way has changed */
	UPROPERTY(BlueprintAssignable, Category = "Pathed Movement")
	FChaosMover_OnPathedMovementOneWayPlaybackChanged OnPathedMovementOneWayPlaybackChanged;

	UPROPERTY(EditAnywhere, Category="Pathed Movement")
	FChaosMutablePathedMovementProperties InitialProperties;

protected:
	// Processes output events coming from the simulation and broadcasts delegates accordingly
	UFUNCTION()
	virtual void OnPostSimEventReceived(const FMoverSimulationEventData& EventData);

	UFUNCTION()
	virtual void OnPostFinalize(const FMoverSyncState& SyncState, const FMoverAuxStateContext& AuxState);

protected:
	// IChaosPathedMovementDebugDrawInterface implementation
	virtual bool ShouldDisplayProgressPreviewMesh_Implementation() const override;
	virtual float GetPreviewMeshOverallPathProgress_Implementation() const override;
	virtual UMaterialInterface* GetProgressPreviewMeshMaterial_Implementation() const override;

	/** When true and the root component of the actor is a mesh, a duplicate mesh will be shown at PreviewMeshProgress along the starting/default path */
	UPROPERTY(EditAnywhere, Category = "Pathed Movement|Debug")
	bool bDisplayProgressPreviewMesh = true;

	/** How far along the starting/default path to preview the controlled mesh */
	UPROPERTY(EditAnywhere, Category = "Pathed Movement|Debug", meta = (EditCondition = bDisplayProgressPreviewMesh, EditConditionHides, UIMin = 0, ClampMin = 0, UIMax = 1, ClampMax = 1))
	float PreviewMeshOverallPathProgress = 1.f;

	/** The material to apply to the preview mesh displayed along the path at PreviewMeshProgress (leave empty for an exact duplicate) */
	UPROPERTY(EditAnywhere, Category = "Pathed Movement|Debug", meta = (EditCondition = bDisplayProgressPreviewMesh, EditConditionHides))
	TObjectPtr<UMaterialInterface> PreviewMeshMaterial;

	UPROPERTY(Transient)
	TObjectPtr<UChaosPathedMovementDebugDrawComponent> DebugDrawComp;

private:
	/* Schedule NewProperties to take effect at the earliest possible time
	*  @param NewProperties The properties that should take effect
	*  @param bIsScheduled When true, a delay is added to ensure all end points execute the change on the same frame
	*  @param bIsStart If true, this indicates that a start is requested as part of the changes. This is to restart if bWantsToPlay was true but the pathed movement stopped by reaching the end of the path.
	*/
	void ExecuteOrScheduleChange(const FChaosMutablePathedMovementProperties& NewProperties, bool bIsScheduled, bool bIsStart = false);

	// Util function to get the physics solver
	Chaos::FPhysicsSolver* GetPhysicsSolver() const;

	// Mutable pathed movement properties to take effect when server frame reaches LastChangeFrame
	FChaosMutablePathedMovementProperties PathedMovementMutableProperties;

	// The (server) frame at which PathedMovementMutableProperties took effect
	// Note that it can be greater than the current server frame to schedule that change
	// (the changes are sent immediately but scheduled to take effect in the future)
	int32 LastChangeFrame = INDEX_NONE;

	// The frame at which movement last started (but this may be in the future for a scheduled start)
	int32 MovementStartFrame = INDEX_NONE;
	FTimerHandle PathedMovementDelayedStartTimerHandle;

	// The (possibly interpolated) post simulation state 
	FChaosPathedMovementState PostSimState;

	// Cached mover component if found on owner actor
	// We use the OnPostMovement delegate to reflect the current state to gameplay (interpolated state in async)
	UPROPERTY(Transient)
	TObjectPtr<UMoverComponent> MoverComponent = nullptr;

	// Delay added to changes, when they are scheduled @see ExecuteOrScheduleChange
	float EventSchedulingMinDelaySeconds = 0.3f;
};
