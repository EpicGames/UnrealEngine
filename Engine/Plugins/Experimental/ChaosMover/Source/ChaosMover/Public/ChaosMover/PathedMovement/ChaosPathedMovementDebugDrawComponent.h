// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Debug/DebugDrawComponent.h"

#include "ChaosPathedMovementDebugDrawComponent.generated.h"

class UMeshComponent;
class UMoverComponent;

UINTERFACE(MinimalAPI, BlueprintType)
class UChaosPathedMovementDebugDrawInterface : public UInterface
{
	GENERATED_BODY()
};

/**
 * ChaosPathedMovementDebugDrawInterface: Controls the debug draws of pathed movement modes, showing a preview mesh at a given overall progression on the path
 */
class IChaosPathedMovementDebugDrawInterface : public IInterface
{
	GENERATED_BODY()

public:
	/** Whether pathed movement debug draws should display a preview mesh at a given overall progress of the aggregate path */
	UFUNCTION(BlueprintNativeEvent)
	bool ShouldDisplayProgressPreviewMesh() const;

	/** How far along the path progression to preview the controlled mesh */
	UFUNCTION(BlueprintNativeEvent)
	float GetPreviewMeshOverallPathProgress() const;

	/** The material to apply to the preview mesh displayed along the path at PreviewMeshProgress (leave empty for an exact duplicate) */
	UFUNCTION(BlueprintNativeEvent)
	UMaterialInterface* GetProgressPreviewMeshMaterial() const;
};

//@todo DanH: Ideally clicking anything drawn by this would select the mover comp in the editor. I think I need a whole FComponentVisualizer set up for that though?
UCLASS()
class UChaosPathedMovementDebugDrawComponent : public UDebugDrawComponent
{
	GENERATED_BODY()

public:
	UChaosPathedMovementDebugDrawComponent();

	virtual void OnRegister() override;
	virtual void OnUnregister() override;

	virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;

	TArray<FDebugRenderSceneProxy::FDebugLine> DebugLines;
	TArray<FDebugRenderSceneProxy::FDashedLine> DebugDashedLines;
	TArray<FDebugRenderSceneProxy::FArrowLine> DebugArrowLines;
	TArray<FDebugRenderSceneProxy::FWireStar> DebugStars;
	TArray<FDebugRenderSceneProxy::FSphere> DebugSpheres;
	
protected:
	virtual void CreateRenderState_Concurrent(FRegisterComponentContext* Context) override;
	virtual FDebugRenderSceneProxy* CreateDebugSceneProxy() override;

	void UpdatePreviewMeshComp(bool bForce = false);
	void DestroyProgressPreviewMeshComp();
	UMeshComponent* GetOwnerMeshRoot() const;
	UMoverComponent* GetMoverComponent() const;

#if WITH_EDITOR
	void HandleObjectPropertyChanged(UObject* Object, FPropertyChangedEvent& PropertyChangedEvent);
#endif

	UPROPERTY(Transient)
	TObjectPtr<UMeshComponent> ProgressPreviewMeshComp = nullptr;

	TObjectPtr<UObject> DebugDrawInterfaceObject;
};