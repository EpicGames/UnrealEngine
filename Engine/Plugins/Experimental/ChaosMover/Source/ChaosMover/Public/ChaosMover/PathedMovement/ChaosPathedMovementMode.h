// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MovementMode.h"
#include "MoverTypes.h"
#include "AlphaBlend.h"
#include "PhysicsEngine/ConstraintInstance.h"
#include "ChaosMover/ChaosMoverSimulationTypes.h"
#include "ChaosMover/ChaosMovementMode.h"
#include "ChaosMover/PathedMovement/ChaosPathedMovementTypes.h"
#include "Chaos/PBDJointConstraintTypes.h"

#include "ChaosPathedMovementMode.generated.h"

class UChaosPathedMovementPatternBase;

/**
 * Base for any movement mode that proceeds along a defined path within a specified amount of time
 */
UCLASS(Within = MoverComponent)
class UChaosPathedMovementMode
	: public UChaosMovementMode
	, public IChaosMovementActuationInterface
	, public IChaosPathedMovementModeInterface
	, public IChaosPostSimulationTickInterface
{
	GENERATED_BODY()

public:
	UChaosPathedMovementMode(const FObjectInitializer& ObjectInitializer);

	// UBaseMovementMode
	virtual void OnRegistered(const FName ModeName, const FMoverSimContext& SimContext) override;
	virtual void GenerateMove_Implementation(const FMoverSimContext& SimContext, const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, FProposedMove& OutProposedMove) const override;
	virtual void SimulationTick_Implementation(const FSimulationTickParams& Params, FMoverTickEndData& OutputState) override;
	virtual bool UsesMovementBasisTransform() const override;

	// IChaosConstraintMovementModeInterface
	virtual bool ShouldUseConstraint() const override;
	virtual const Chaos::FPBDJointSettings& GetConstraintSettings() const override;

	// IChaosPathedMovementModeInterface
	virtual FTransform CalcTargetTransform(float OverallPathProgress, const FTransform& BasisTransform) const override;

	// IChaosPostSimulationTickInterface
	virtual void PostSimulationTick_Async(FChaosMoverPostSimContext& Context) override;


#if WITH_EDITOR
	// UBaseMovementMode Interface
	EDataValidationResult IsDataValid(FDataValidationContext& Context) const override;
#endif // WITH_EDITOR
	
	/** Called when the owning UChaosPathedPhysicsMoverComponent is registered (i.e. at both edit and game time) */
	virtual void InitializePath();
	
	/** Finds the first path pattern on this mode of the given type, if there is one */
	UFUNCTION(BlueprintPure, Category = PathedMovementMode, DisplayName = "Find Pattern", meta=(DeterminesOutputType = PatternType))
	UChaosPathedMovementPatternBase* BP_FindPattern(TSubclassOf<UChaosPathedMovementPatternBase> PatternType) const;
	
	/**
	 * Set the time it takes to travel one way along this path from start to end.
	 * Only valid to call in BeginPlay or the Construction Script, ignored otherwise.
	 * Once play has begun, the path duration can be modified indirectly via SetMovementTimeDilation on the owning PathedMoverComponent
	 */
	UFUNCTION(BlueprintCallable, Category = PathedMovementMode, DisplayName = "Set Path Duration (BeginPlay only)")
	void SetOneWayTripDuration_BeginPlayOnly(float NewDuration);
	float GetOneWayTripDuration() const { return OneWayPlaybackDuration; }
	
protected:
	void OnPlaybackStarting(const FMoverTimeStep& SimTime);
	void OnPlaybackStopping(const FMoverTimeStep& SimTime, bool bReachedEndOfPlayback);
	void OnPlaybackBounced(const FMoverTimeStep& SimTime);
	void OnInReverseChanging(const FMoverTimeStep& SimTime, bool bNewInReverse);
	void OnLoopingChanging(const FMoverTimeStep& SimTime, bool bNewLooping);
	void OnOneWayChanging(const FMoverTimeStep& SimTime, bool bNewOneWay);

	void SetDefaultJointConstraintProperties();
	
private:
	/**
	 * True to have the updated component be able to respond to physics inputs by being "pulled" along the path using a joint constraint
	 * configured using JointConstraintProperties below.
	 * 
	 * If false, no joint will be used, and the affected component will be moved kinematically (i.e. independent of other physics effects and treated as though it has
	 * infinite mass in the physics sim) and never deviate from the path.
	 *
	 * Note that the movement is calculated on the worker thread either way, so it will correctly interact with other bodies in the sim.
	 *
	 * Ex: Platform moving along a path in a straight line. If this is enabled, the platform could wobble, dip, shake, stall, etc. in response to physics forces/impulses
	 *		(depending on the joint and physics properties of the platform). If this is disabled, the platform will rigidly move along the path and ignore all physics.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Pathed Movement Mode", meta = (AllowPrivateAccess))
	bool bUseJointConstraint = true;

	/**
	 * Configuration of the joint constraint that (if enabled) pulls the updated component along the path
	 * The default settings are for a very stiff joint that allows just a little bit of "play"
	 */
	UPROPERTY(EditAnywhere, Category = "Pathed Movement Mode", meta = (EditCondition = bUseJointConstraint))
	FConstraintProfileProperties JointConstraintProperties;

	/** Total duration in seconds of a one-way trip from start to end. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Pathed Movement Mode", meta = (ClampMin=0, UIMin=0, Units=Seconds, AllowPrivateAccess))
	float OneWayPlaybackDuration = 5.f;

	//@todo DanH: Try to think of a way to make this less order-dependent/error-prone
	/**
	 * The set of patterns to follow that combine to form the path the component will follow
	 *
	 * NOTE: Order can matter here!! The patterns are evaluated each physics frame in order starting from the front.
	 *		 Each successive pattern is aware of the aggregate results from the previous patterns.
	 *		 In most cases, this doesn't matter, as the pattern calculates the target transform against the static path origin point.
	 *		 However, some patterns necessarily determine their target based on the calculated target to that point
	 *		 For example, the LookAtRotationPattern calculates the look-at rotation target based on the target location
	 */
	UPROPERTY(EditAnywhere, Instanced, BlueprintReadOnly, Category = "Pathed Movement Mode", meta = (AllowPrivateAccess))
	TArray<TObjectPtr<UChaosPathedMovementPatternBase>> PathPatterns;
	
	/** The kind of easing to apply when traveling along the path */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Pathed Movement Mode", meta = (AllowPrivateAccess))
	EAlphaBlendOption Easing = EAlphaBlendOption::Linear;

	/** If using a custom ease, this is the curve that will be used. If blank, will fall back to standard linear interpolation. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Pathed Movement Mode", meta = (EditCondition = "Easing==EAlphaBlendOption::Custom", AllowPrivateAccess))
	TObjectPtr<UCurveFloat> CustomEasingCurve = nullptr;

	Chaos::FPBDJointSettings JointSettings;

	// Whether the physics simulation runs asynchronously with a fixed timestep.
	// Cached from the simulation at registration time; used only when recording LastChangeTime reference points.
	bool bIsUsingAsyncPhysics = true;

	// Builds the FMoverTime reference point to store when a change occurs (start of the current frame).
	// Sets FrameCount in fixed-dt (async) mode, leaves it INDEX_NONE in variable-dt (non-async) mode.
	FMoverTime MakeChangeTime(const FMoverTimeStep& TimeStep) const;

	/** True to draw debug lines for the aggregate path resulting from all the patterns in the path (i.e. the path that will actually be followed in game) */
	UPROPERTY(EditAnywhere, Category = "Pathed Movement Mode|Debug")
	bool bDebugDrawAggregatePath = true;

	/** True to allow individual patterns to draw debug representations. Can be toggled individually on each pattern as well, this is a master switch for convenience. */
	UPROPERTY(EditAnywhere, Category = "Pathed Movement Mode|Debug")
	bool bAllowPatternDebugDrawing = true;

	/** The color used when drawing the aggregate path */
	UPROPERTY(EditAnywhere, Category = "Pathed Movement Mode|Debug", meta = (EditCondition = bDebugDrawAggregatePath))
	FColor PathDebugDrawColor = FColor::Blue;
	
	friend class UChaosPathedMovementDebugDrawComponent;

#if WITH_CHAOS_VISUAL_DEBUGGER
	mutable FChaosPathedMovementModeDebugData DebugData;
#endif // !UE_BUILD_SHIPPING
};

