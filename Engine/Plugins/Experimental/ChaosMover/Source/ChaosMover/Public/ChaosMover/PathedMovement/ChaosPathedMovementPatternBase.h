// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ChaosMover/PathedMovement/ChaosPathedMovementTypes.h"

#include "ChaosPathedMovementPatternBase.generated.h"

class UCurveFloat;
enum class EAlphaBlendOption : uint8;
class UChaosPathedMovementMode;
class UChaosMoverSimulation;
class UChaosPathedMovementDebugDrawComponent;

//@todo DanH: What's the Flags specifier for? Does that do everything now or do I still need the Bitflags meta?
UENUM(Flags, meta = (Bitflags))
enum class EChaosPatternAxisMaskFlags : uint8
{
	None = 0 UMETA(Hidden),

	X = 1 UMETA(DisplayName = "X / Roll"),
	Y = 1 << 1 UMETA(DisplayName = "Y / Pitch"),
	Z = 1 << 2 UMETA(DisplayName = "Z / Yaw)"),

	All = X | Y | Z UMETA(Hidden)
};
ENUM_CLASS_FLAGS(EChaosPatternAxisMaskFlags);

UCLASS(Abstract, Within = ChaosPathedMovementMode, DefaultToInstanced, Blueprintable, BlueprintType, EditInlineNew)
class UChaosPathedMovementPatternBase : public UObject
{
	GENERATED_BODY()

public:
	virtual void InitializePattern(UChaosMoverSimulation* InSimulation);

	const UChaosMoverSimulation* GetSimulation() const
	{
		return Simulation;
	}

	UChaosPathedMovementMode& GetMovementMode() const;
	
	// Calculates the path target transform in world coordinates, at OverallPathProgress, given the path basis transform
	FTransform CalcTargetTransform(float OverallPathProgress, const FTransform& BasisTransform) const;

	virtual bool DebugDrawUsingStepSamples() const { return bDebugDrawPattern; }
	virtual void AppendDebugDrawElements(UChaosPathedMovementDebugDrawComponent& DebugDrawComp, FBoxSphereBounds::Builder& InOutDebugBoundsBuilder) {}

#if WITH_EDITOR
	//@todo DanH: Pipe this through (I wonder if there should be a UPackage::IsDataValid() override that auto-pipes it to all UObjects in the package?)
	// virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;

	friend class FChaosPatternPostChangeMovementModeHelper;
#endif
	
protected:
	/**
	* Where patterns calculate their target transforms at the given path progress
	* The transform returned is in world coordinates, given the path basis transform BasisTransform
	* Axis masking is applied to this result before it's actually used.
	*/
	virtual FTransform CalcUnmaskedTargetTransform(float PatternProgress, const FTransform& BasisTransform) const { return BasisTransform; }

	/** Utility function that calls CalcUnmaskedTargetTransform and then applies masking. This is mostly for debug drawing, which wants to
	*   deal with PatternProgress and not OverallPathProgress but also get the masked transform
	*/
	FTransform CalcMaskedTargetTransform(float PatternProgress, const FTransform& BasisTransform) const;

	/** Converts an overall path progress to a progress on the pattern
	*   Returns -1 if it can't do it, say, e.g. if somehow StartAtPathProgress > EndAtPathProgress
	*/
	float ConvertPathToPatternProgress(float OverallPathProgress) const;

	/** Converts a pattern progress to one overall path possible progress
	*	If the pattern is looping, it will return the path progress corresponding to the first loop
	*   Note that ConvertPathToPatternProgress is not invertible, it is a surjective mapping (i.e. many-to-one mapping)
	*/
	float ConvertPatternToPathProgress(float PatternProgress) const;

	//
	// NOTE: Making any of these properties mutable at game time will require either replicating them (easier, but not as good) or pushing them as inputs to the PT (harder, more correct)
	//

	/** Along which axes is this pattern disallowed from modifying the translation/location of the updated component? */
	UPROPERTY(EditAnywhere, Category = "Path Pattern", meta = (Bitmask, BitmaskEnum = "/Script/ChaosMover.EChaosPatternAxisMaskFlags"))
	uint8 TranslationMasks = (uint8)EChaosPatternAxisMaskFlags::None;

	/** Along which axes is this pattern disallowed from modifying the rotation of the updated component? */
	UPROPERTY(EditAnywhere, Category = "Path Pattern", meta = (Bitmask, BitmaskEnum = "/Script/ChaosMover.EChaosPatternAxisMaskFlags"))
	uint8 RotationMasks = (uint8)EChaosPatternAxisMaskFlags::None;
	
	/** Along which axes is this pattern disallowed from modifying the scale of the updated component? */
	UPROPERTY(EditAnywhere, Category = "Path Pattern", meta = (Bitmask, BitmaskEnum = "/Script/ChaosMover.EChaosPatternAxisMaskFlags"))
	uint8 ScaleMasks = (uint8)EChaosPatternAxisMaskFlags::None;

	/**
	 * If true, this pattern will not begin to take effect until the previous pattern has completed.
	 * Note: If true and the previous pattern's EndAtPathProgress is 1, this pattern will never start.
	 */
	UPROPERTY(EditAnywhere, Category = "Path Pattern")
	bool bStartAfterPreviousPattern = false;

	/** The overall path progress when this pattern should begin, where 0 is the start of the path and 1 is the end. Must be less than EndAtPathProgress. */
	UPROPERTY(EditAnywhere, Category = "Path Pattern", meta = (EditCondition = "!bStartAfterPreviousPattern", UIMin = 0.f, ClampMin = 0.f, UIMax = 1.f, ClampMax = 1.f))
	float StartAtPathProgress = 0.f;

	/** The overall path progress when this pattern should complete, where 0 is the start of the path and 1 is the end. Must be greater than StartAtPathProgress. */
	UPROPERTY(EditAnywhere, Category = "Path Pattern", meta = (UIMin = 0.f, ClampMin = 0.f, UIMax = 1.f, ClampMax = 1.f))
	float EndAtPathProgress = 1.f;

	/** This is an offset within a loop of the pattern, allowing to offset the start of the loop within the path. For example if this is set to 0.2,
	*   a loop will start at 20% of the path (instead of at the start), progress to the end, then wrap around to the start and continue progressing to 20%.
	*   If the path itself is not a loop (if start and end are not the same), then the position will jump abruptly when it wraps around.
	*/
	UPROPERTY(EditAnywhere, Category = "Path Pattern", meta = (UIMin = 0.f, ClampMin = 0.f, UIMax = 1.f, ClampMax = 1.f))
	float LoopOffset = 0.f;

	//@todo DanH: Maybe this should be a float and allow partial loops?
	/**
	 * The number of loops to complete within the active span of this pattern (i.e. between StartAtPathProgress and EndAtPathProgress)
	 * on a single run along the full aggregate path. Setting to 0 effectively disables this pattern.
	 */
	UPROPERTY(EditAnywhere, Category = "Path Pattern", meta = (UIMin = 0, ClampMin = 0))
	int32 NumLoopsPerPath = 1;

	/** If true, each loop plays forward only (0 → 1). If false, each loop plays there-and-back (0 → 1 → 0). */
	UPROPERTY(EditAnywhere, Category = "Path Pattern")
	bool IsOneWay = true;

	/**
	 * When true, progress bounces at 0 and 1 instead of crossing the seam (wrapping).
	 * Use this for non-looping paths to avoid teleports at the endpoints.
	 * When false (default), progress wraps at the seam — correct for geometrically looping paths.
	 */
	UPROPERTY(EditAnywhere, Category = "Path Pattern")
	bool bBounceAtEnds = false;
	
	/** 
	 * If true, the component will be rotated to face in the direction of this pattern's motion.
	 * To have the component face in the direction of the aggregate path, enable this on all movement patterns.
	 */
	UPROPERTY(EditAnywhere, Category = "Path Pattern")
	bool bOrientComponentToPath = false;

	/** The kind of easing to apply when traveling along the path */
	UPROPERTY(EditAnywhere, Category = "Path Pattern")
	EAlphaBlendOption Easing;

	/** If using a custom ease, this is the curve that will be used. If blank, will fall back to standard linear interpolation. */
	UPROPERTY(EditAnywhere, Category = "Path Pattern", meta = (EditCondition = "Easing==EAlphaBlendOption::Custom"))
	TObjectPtr<UCurveFloat> CustomEasingCurve = nullptr;

	/** True to draw debug lines for this specific pattern in editor views */
	UPROPERTY(EditAnywhere, Category = "Path Pattern|Debug")
	bool bDebugDrawPattern = true;

	//@todo DanH: More central setting for color per pattern index maybe?
	/** The color used for debug draws of this pattern */
	UPROPERTY(EditAnywhere, Category = "Path Pattern|Debug", meta = (EditCondition = bDrawDebug))
	FColor PatternDebugDrawColor;

	UChaosMoverSimulation* Simulation = nullptr;

	friend class UChaosPathedMovementDebugDrawComponent;
};
