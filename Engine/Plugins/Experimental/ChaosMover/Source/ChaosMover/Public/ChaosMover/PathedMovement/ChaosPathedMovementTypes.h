// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MoverTypes.h"
#include "Chaos/Core.h"
#include "MoverSimulationTypes.h"

#include "ChaosPathedMovementTypes.generated.h"

namespace UE::ChaosMover::PathBlackboard
{
	const FName TargetRelativeTransform = TEXT("TargetRelativeTransform");
}

namespace Chaos
{
	class FPBDJointSettings;
}

/**
 * Properties that can change during game-time and affect path movement calculations.
 * Split into its own struct because both the input and sync state structs want to hold onto these. 
 */
USTRUCT()
struct FChaosMutablePathedMovementProperties
{
	GENERATED_BODY()

	// True = Wants to being playing the path, False = Wants to be stopped
	UPROPERTY(EditAnywhere, Category = ChaosMover)
	bool bWantsToPlay = false;

	// True = Reverse, False = Forward
	UPROPERTY(EditAnywhere, Category = ChaosMover)
	bool bWantsReversePlayback = false;

	// True = Looping, False = Will stop next time it reaches the end
	// (that end could be 0 or 1 depending on bWantsOneWayPlayback and bWantsReversePlayback)
	UPROPERTY(EditAnywhere, Category = ChaosMover)
	bool bWantsLoopingPlayback = false;

	// True = One way playback (0->1), False = Round trip, there and back (0->1->0)
	UPROPERTY(EditAnywhere, Category = ChaosMover)
	bool bWantsOneWayPlayback = false;

	// These properties cannot be mutated yet, but were supported by Dan's implementation. Coming soon.
	//UPROPERTY(VisibleAnywhere, Category = ChaosMover)
	//FTransform PathOrigin;
	//UPROPERTY(VisibleAnywhere, Category = ChaosMover)
	//bool bIsUsingConstraint;

	void NetSerialize(FArchive& Ar);
	void ToString(FAnsiStringBuilderBase& Out) const;
	
	bool operator==(const FChaosMutablePathedMovementProperties& Other) const;
	bool operator!=(const FChaosMutablePathedMovementProperties& Other) const { return !operator==(Other); }
};

/**
 * Frame inputs for all pathed movement
 * Any property that can change during game time must be delivered in this way, only immutable properties can be safely
 * referenced directly on the object they come from (usually the movement mode)
 *
 * NOTE: MovementStartFrame and LastChangeFrame use integer frame numbers for scheduling and ordering.
 * This is correct in async (fixed-dt) mode, where both server and client share the same synchronized frame counter.
 * In non-async (variable-dt) mode, frame numbers are a local monotonic counter and are NOT synchronized across
 * endpoints. The current implicit assumption is: networked => fixed-dt, variable-dt => standalone only.
 * To support variable-dt networking properly, these fields would need to become FMoverTime.
 */
USTRUCT()
struct FChaosPathedMovementInputs : public FMoverDataStructBase
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category = ChaosMover)
	int32 MovementStartFrame = INDEX_NONE;

	UPROPERTY(VisibleAnywhere, Category = ChaosMover)
	int32 LastChangeFrame = INDEX_NONE;

	UPROPERTY(VisibleAnywhere, Category = ChaosMover)
	FChaosMutablePathedMovementProperties Props;

	// Implementation of FMoverDataStructBase
	virtual bool NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess) override;
	virtual FMoverDataStructBase* Clone() const override { return new FChaosPathedMovementInputs(*this); }
	virtual UScriptStruct* GetScriptStruct() const override { return StaticStruct(); }
	virtual void ToString(FAnsiStringBuilderBase& Out) const override;
	virtual bool ShouldReconcile(const FMoverDataStructBase& AuthorityState) const override;
	virtual void Interpolate(const FMoverDataStructBase& From, const FMoverDataStructBase& To, float Pct) override;
	virtual void Merge(const FMoverDataStructBase& From) override;
};

/** State of a pathed movement that persists from frame to frame */
USTRUCT()
struct FChaosPathedMovementState : public FMoverDataStructBase
{
	GENERATED_BODY()

	/** A path progress is a value between 0 and 1 that can be directly used along the path to calculate
	  * the position on it, 0.0 being a the start of the progression along the path, 1.0 being at the end of that progression.
	  * Note that this is not necessarily the start and end of that path as the start could be offset relative to the path primitive (say, a spline)
	  * LastChangePathProgress is the path progress we were at the last time an important change happened. We calculate the current
	  * progress based on that LastChangePathProgress, the number of frames elapsed since that change, and the progression rate,
	  * assuming all properties involved in updating that progress have not changed
	  */
	UPROPERTY(VisibleAnywhere, Category = ChaosMover)
	float LastChangePathProgress = 0.0f;

	// The time at which the last change in movement properties occurred (set to the start of that frame).
	// FrameCount is valid in fixed-dt (async) mode; TimeMs is valid in variable-dt (non-async) mode.
	// @see LastChangePathProgress, FMoverTime::ElapsedSecondsTo
	UPROPERTY(VisibleAnywhere, Category = ChaosMover)
	FMoverTime LastChangeTime;

	/** The frame at which we last started playing, or INDEX_NONE if we were never playing.
	 *  Used to order start requests against MovementStartFrame from inputs. See note on FChaosPathedMovementInputs. */
	UPROPERTY(VisibleAnywhere, Category = ChaosMover)
	int32 LatestMovementStartFrame = INDEX_NONE;

	/** Whether traveling from 0 to 1 (true) or 1 to 0 (false) */
	UPROPERTY(VisibleAnywhere, Category = ChaosMover)
	bool bIsPathProgressionIncreasing = true;
	
	/** Whether the previous playback finished by reaching the end of its overall progression (could be 0 or 1 depending on movement properties)
	*/
	UPROPERTY(VisibleAnywhere, Category = ChaosMover)
	bool bHasFinished = false;
	
	/** The mutable properties in effect */
	UPROPERTY(VisibleAnywhere, Category = ChaosMover)
	FChaosMutablePathedMovementProperties PropertiesInEffect;

	virtual bool NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess) override;
	virtual FMoverDataStructBase* Clone() const override { return new FChaosPathedMovementState(*this); }
	virtual UScriptStruct* GetScriptStruct() const override { return StaticStruct(); }
	virtual void ToString(FAnsiStringBuilderBase& Out) const override;
	virtual bool ShouldReconcile(const FMoverDataStructBase& AuthorityState) const override;
	virtual void Interpolate(const FMoverDataStructBase& From, const FMoverDataStructBase& To, float Pct) override;
};

USTRUCT()
struct FChaosPathedMovementModeDebugData : public FMoverDataStructBase
{
	GENERATED_BODY()

	virtual FMoverDataStructBase* Clone() const override;
	virtual UScriptStruct* GetScriptStruct() const override;

	void Advance()
	{
		PreviousServerFrame = ServerFrame;
		bPlaybackStartedThisFrame = false;
		bPlaybackStoppedThisFrame = false;
		bPlaybackBouncedThisFrame = false;
		bPlaybackReachedEndThisFrame = false;
		bPlaybackReverseChangedThisFrame = false;
		bPlaybackLoopingChangedThisFrame = false;
		bPlaybackOneWayChangedThisFrame = false;
	}

	// This is the server frame last time we ticked, it might not be ServerFrame-1 if we are resimming
	UPROPERTY(VisibleAnywhere, Category = "Mover Info")
	int32 PreviousServerFrame = INDEX_NONE;

	UPROPERTY(VisibleAnywhere, Category = "Mover Info")
	int32 ServerFrame = INDEX_NONE;

	UPROPERTY(VisibleAnywhere, Category = "Mover Info")
	FChaosPathedMovementState PreSimMoveState;

	UPROPERTY(VisibleAnywhere, Category = "Mover Info")
	FChaosPathedMovementState PostSimMoveState;

	UPROPERTY(VisibleAnywhere, Category = "Mover Info")
	FVector PreviousWorldTranslation = FVector::ZeroVector;

	UPROPERTY(VisibleAnywhere, Category = "Mover Info")
	FRotator PreviousWorldRotation = FRotator::ZeroRotator;

	UPROPERTY(VisibleAnywhere, Category = "Mover Info")
	FVector NewWorldTranslation = FVector::ZeroVector;

	UPROPERTY(VisibleAnywhere, Category = "Mover Info")
	FRotator NewWorldRotation = FRotator::ZeroRotator;

	UPROPERTY(VisibleAnywhere, Category = "Mover Info")
	bool bPlaybackStartedThisFrame = false;

	UPROPERTY(VisibleAnywhere, Category = "Mover Info")
	bool bPlaybackStoppedThisFrame = false;

	UPROPERTY(VisibleAnywhere, Category = "Mover Info")
	bool bPlaybackBouncedThisFrame = false;

	UPROPERTY(VisibleAnywhere, Category = "Mover Info")
	bool bPlaybackReachedEndThisFrame = false;

	UPROPERTY(VisibleAnywhere, Category = "Mover Info")
	bool bPlaybackReverseChangedThisFrame = false;

	UPROPERTY(VisibleAnywhere, Category = "Mover Info")
	bool bPlaybackLoopingChangedThisFrame = false;

	UPROPERTY(VisibleAnywhere, Category = "Mover Info")
	bool bPlaybackOneWayChangedThisFrame = false;
};

USTRUCT()
struct FPathedMovementStartedEventData : public FMoverSimulationEventData
{
	GENERATED_BODY()

	FPathedMovementStartedEventData(const FMoverTimeStep& InEventTime)
		: FMoverSimulationEventData(InEventTime)
	{
	}
	FPathedMovementStartedEventData() {}

	virtual UScriptStruct* GetScriptStruct() const override
	{
		return FPathedMovementStartedEventData::StaticStruct();
	}
};

USTRUCT()
struct FPathedMovementStoppedEventData : public FMoverSimulationEventData
{
	GENERATED_BODY()

	FPathedMovementStoppedEventData(const FMoverTimeStep& InEventTime, bool InbReachedEndOfPlayback)
		: FMoverSimulationEventData(InEventTime)
		, bReachedEndOfPlayback(InbReachedEndOfPlayback)
	{
	}
	FPathedMovementStoppedEventData() {}

	virtual UScriptStruct* GetScriptStruct() const override
	{
		return FPathedMovementStoppedEventData::StaticStruct();
	}

	bool bReachedEndOfPlayback = false;
};

USTRUCT()
struct FPathedMovementBouncedEventData : public FMoverSimulationEventData
{
	GENERATED_BODY()

	FPathedMovementBouncedEventData(const FMoverTimeStep& InEventTime)
		: FMoverSimulationEventData(InEventTime)
	{
	}
	FPathedMovementBouncedEventData() {}

	virtual UScriptStruct* GetScriptStruct() const override
	{
		return FPathedMovementBouncedEventData::StaticStruct();
	}
};

USTRUCT()
struct FPathedMovementInReverseChangedEventData : public FMoverSimulationEventData
{
	GENERATED_BODY()

	FPathedMovementInReverseChangedEventData(const FMoverTimeStep& InEventTime, bool InbNewInReverse)
		: FMoverSimulationEventData(InEventTime)
		, bNewInReverse(InbNewInReverse)
	{
	}
	FPathedMovementInReverseChangedEventData() {}

	virtual UScriptStruct* GetScriptStruct() const override
	{
		return FPathedMovementInReverseChangedEventData::StaticStruct();
	}

	bool bNewInReverse = false;
};

USTRUCT()
struct FPathedMovementLoopingChangedEventData : public FMoverSimulationEventData
{
	GENERATED_BODY()

	FPathedMovementLoopingChangedEventData(const FMoverTimeStep& InEventTime, bool InbNewLooping)
		: FMoverSimulationEventData(InEventTime)
		, bNewLooping(InbNewLooping)
	{
	}
	FPathedMovementLoopingChangedEventData() {}

	virtual UScriptStruct* GetScriptStruct() const override
	{
		return FPathedMovementLoopingChangedEventData::StaticStruct();
	}

	bool bNewLooping = false;
};

USTRUCT()
struct FPathedMovementOneWayChangedEventData : public FMoverSimulationEventData
{
	GENERATED_BODY()

	FPathedMovementOneWayChangedEventData(const FMoverTimeStep& InEventTime, bool InbNewOneWay)
		: FMoverSimulationEventData(InEventTime)
		, bNewOneWay(InbNewOneWay)
	{
	}
	FPathedMovementOneWayChangedEventData() {}

	virtual UScriptStruct* GetScriptStruct() const override
	{
		return FPathedMovementOneWayChangedEventData::StaticStruct();
	}

	bool bNewOneWay = false;
};

// Interface for mover modes which can move the controlled component
// with a physical constraint or kinematically
UINTERFACE(MinimalAPI)
class UChaosMovementActuationInterface : public UInterface
{
	GENERATED_BODY()
};
class IChaosMovementActuationInterface
{
	GENERATED_BODY()
public:

	// Whether the simulation should use a constraint to move the controlled particle, or moving it kinematically
	virtual bool ShouldUseConstraint() const = 0;

	// Constraint settings of the joint constraint, only valid when ShouldUseConstraint() is true
	virtual const Chaos::FPBDJointSettings& GetConstraintSettings() const = 0;
};

// Interface for mover modes which target transform is parameterized by an overall path progression, a value in [0, 1]
// 0 being at the beginning of the progression, 1 being at the end of that progression. A given progression should
// always result in the same resulting transform, but the spatial trajectory resulting from the progression
// varying from 0 to 1 may loop or revisit the same curves multiple times
// Implementing this interface implies the usage of a path basis (centralized in the simulation, not in modes)
// that the path is relative to
UINTERFACE(MinimalAPI)
class UChaosPathedMovementModeInterface : public UInterface
{
	GENERATED_BODY()
};
class IChaosPathedMovementModeInterface
{
	GENERATED_BODY()
public:

	// Returns the transform on the path at Progression
	virtual FTransform CalcTargetTransform(float Progression, const FTransform& BasisTransform) const = 0;
};
