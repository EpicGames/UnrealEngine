// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ChaosMover/PathedMovement/ChaosPathedMovementPatternBase.h"

#include "ChaosPointMovementPathPattern.generated.h"

class UChaosPathedMovementDebugDrawComponent;

UENUM()
enum class EChaosPointMovementLocationBasis : uint8
{
	/** The location is relative to the previous point in the path */
	PreviousPoint,
	/** The location is relative to the origin of the path */
	PathOrigin,
	/** The location is relative to the world origin */
	World
};

USTRUCT(BlueprintType)
struct FChaosPointMovementPathPoint
{
	GENERATED_BODY()

public:
	FChaosPointMovementPathPoint() {}

	//@todo DanH: Same options for control as FSplinePathPatternPointData - might want a shared base
	
	/** Basis that the location of this point is relative to */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Location)
	EChaosPointMovementLocationBasis Basis = EChaosPointMovementLocationBasis::PreviousPoint;
	
	/** The location to move toward (i.e. root component location by default) */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Location, meta = (MakeEditWidget = true))
	FVector Location = FVector::ZeroVector;

	/** Basis that the rotation at this point is relative to */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Location)
	EChaosPointMovementLocationBasis RotationBasis = EChaosPointMovementLocationBasis::PreviousPoint;

	/** The rotation to rotate toward */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Location, meta = (MakeEditWidget = true))
	FRotator Rotation = FRotator::ZeroRotator;

	FTransform GetWorldTransform(const FTransform& BasisTransform) const;

	/**
	 * The progress along the path that this point corresponds to.
	 * Read-only, calculated automatically based on number of points, distances between them, and movement rules for the point 
	 */
	// UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Transient, Category = Location)
	mutable float Progress = 0.f;
	
	/** Distance from the start of the path to this point. Read-only. */
	// UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Transient, Category = Location)
	mutable float PathDistanceFromStart = 0.f;

	// This location is precomputed as a function of Location and Basis.
	// If Basis is World, then this is a world location. Otherwise this location is relative to the BasisTransform that will be provided (@see CalcUnmaskedTargetTransform)
	mutable FVector BasedLocation = FVector::ZeroVector;
	// This rotation is precomputed as a function of Rotation and Basis.
	// If RotationBasis is World, then this is a world rotation. Otherwise this rotation is relative to the BasisTransform that will be provided (@see CalcUnmaskedTargetTransform)
	mutable FQuat BasedRotation = FQuat::Identity;
	// This basis indicates whether the BasedLocation is relative to the path basis or in world coordinates
	// It is either PathOrigin or World, but cannot be PreviousPoint, since the purpose of BasedLocation is to precompute as much as possible beforehand
	mutable EChaosPointMovementLocationBasis EffectiveBasis = EChaosPointMovementLocationBasis::PathOrigin;
	// This basis indicates whether the BasedRotation is relative to the path basis or in world coordinates
	// It is either PathOrigin or World, but cannot be PreviousPoint, since the purpose of BasedRotation is to precompute as much as possible beforehand
	mutable EChaosPointMovementLocationBasis EffectiveRotationBasis = EChaosPointMovementLocationBasis::PathOrigin;
};

/** Movement pattern that moves between explicitly defined points */
UCLASS()
class UChaosPointMovementPathPattern : public UChaosPathedMovementPatternBase
{
	GENERATED_BODY()

public:
	virtual void InitializePattern(UChaosMoverSimulation* InSimulation) override;

#if WITH_EDITOR
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	
	virtual void AppendDebugDrawElements(UChaosPathedMovementDebugDrawComponent& DebugDrawComp, FBoxSphereBounds::Builder& InOutDebugBoundsBuilder) override;

protected:
	virtual FTransform CalcUnmaskedTargetTransform(float PatternProgress, const FTransform& BasisTransform) const override;

	void RefreshAssignedPointProgress(const FTransform& PathBasisTransform) const;
	FTransform GetPathBasisTransform() const;

	/** Relative point locations to move toward, in sequence from first to last */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Point Movement Pattern")
	TArray<FChaosPointMovementPathPoint> PathPoints;

	/** The sum total distance of this path. Read-only for reference only. */
	// UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Transient, Category = "Point Movement Pattern")
	mutable float TotalPathDistance = 0.f;

	mutable bool bHasAssignedPointProgress = false;

	mutable FTransform CachedPathBasisTransform = FTransform::Identity;
};