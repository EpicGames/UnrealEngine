// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "ChaosMover/PathedMovement/ChaosPathedMovementPatternBase.h"

#include "ChaosSplineMovementPathPattern.generated.h"

class USplineComponent;

/** This isn't formal "spline metadata" because that requires a custom spline and component - we want to add info to vanilla splines */
USTRUCT()
struct FChaosSplinePathPatternPointData
{
	GENERATED_BODY()

	/** Key of the spline point (or somewhere between two points) where this metadata applies */
	UPROPERTY()
	float SplinePointKey = 0.f;
};

UCLASS()
class UChaosSplineMovementPathPattern : public UChaosPathedMovementPatternBase
{
	GENERATED_BODY()
	
public:
	virtual void InitializePattern(UChaosMoverSimulation* InSimulation) override;

	// The spline is already drawn, no need to draw it again at lower rez
	virtual bool DebugDrawUsingStepSamples() const override { return false; }
	
	bool HasValidBounds() const { return UpperBound > LowerBound; }
	
protected:
	virtual FTransform CalcUnmaskedTargetTransform(float PatternProgress, const FTransform& BasisTransform) const override;

	/** How far into the spline the movement path actually begins.  */
	UPROPERTY(EditAnywhere, Category = SplinePathPattern, meta = (UIMin = 0.f, ClampMin = 0.f, UIMax = 1.f, ClampMax = 1.f))
	float LowerBound = 0.f;

	/** How far into the spline the movement path ends. Must be greater than the lower bound. */
	UPROPERTY(EditAnywhere, Category = SplinePathPattern, meta = (UIMin = 0.f, ClampMin = 0.f, UIMax = 1.f, ClampMax = 1.f))
	float UpperBound = 1.f;

	/**
	 * If true, the path duration will be shortened according to how much of the spline is not being followed.
	 * If false (default), the path duration is unchanged, so the object will move slower when the usable spline range is reduced.
	 */
	UPROPERTY(EditAnywhere, Category = SplinePathPattern)
	bool bOffsetsModifyDuration = false;

	UPROPERTY(EditAnywhere, Category = SplinePathPattern)
	bool bApplySplineScaling = false;
	
	/**
	 * Optional property to specify the spline component that defines the path to follow. If blank, we'll use the first spline component we find in this actor.
	 * This is only necessary to set if you have multiple spline components on the actor, or want to follow a spline on an external actor.
	 * Thread Safety Warning, this spline cannot be modified whilst in use
	 */
	UPROPERTY(EditAnywhere, DisplayName = "Spline Component", Category = SplinePathPattern, meta = (AllowAnyActor, UseComponentPicker, AllowedClasses = "/Script/Engine.SplineComponent"))
	FComponentReference SplineComponentRef;

	UPROPERTY(Transient)
	TObjectPtr<const USplineComponent> SplineComp;
};
