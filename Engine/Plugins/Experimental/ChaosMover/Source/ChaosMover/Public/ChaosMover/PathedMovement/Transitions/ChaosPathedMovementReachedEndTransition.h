// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosMover/ChaosMovementModeTransition.h"

#include "ChaosPathedMovementReachedEndTransition.generated.h"

#define UE_API CHAOSMOVER_API

// Transition checking if an ongiong pathed movement mode has reached the end of the path
UCLASS(MinimalAPI, Blueprintable, BlueprintType, EditInlineNew, DefaultToInstanced)
class UChaosPathedMovementReachedEndTransition : public UChaosMovementModeTransition
{
	GENERATED_BODY()

public:
	UE_API UChaosPathedMovementReachedEndTransition(const FObjectInitializer& ObjectInitializer);

	UE_API virtual FTransitionEvalResult Evaluate_Implementation(const FSimulationTickParams& Params) const override;
	UE_API virtual void Trigger_Implementation(const FSimulationTickParams& Params) override;

	/** Name of movement mode to transition to when the pathed movement has reached the end of the path */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Parameters)
	FName TransitionToMode;
};

#undef UE_API
