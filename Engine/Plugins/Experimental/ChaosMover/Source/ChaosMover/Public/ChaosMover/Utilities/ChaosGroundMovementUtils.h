// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/ParticleHandleFwd.h"
#include "Kismet/BlueprintFunctionLibrary.h"

#include "ChaosGroundMovementUtils.generated.h"

struct FFloorCheckResult;

// Input parameters for controlled ground movement function
USTRUCT(BlueprintType)
struct FRequestedMoveParams
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = ChaosMover)
	FQuat WorldToGravityQuat = FQuat::Identity;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = ChaosMover)
	FRotator PriorOrientation = FRotator::ZeroRotator;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = ChaosMover)
	FVector PriorVelocity = FVector::ZeroVector;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = ChaosMover)
	FVector RequestedVelocity = FVector::ZeroVector;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = ChaosMover)
	FVector GroundNormal = FVector::UpVector;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = ChaosMover)
	float TurningRate = 500.f;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = ChaosMover)
	float MaxSpeed = 800.f;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = ChaosMover)
	float Acceleration = 4000.f;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = ChaosMover)
	float Deceleration = 8000.f;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = ChaosMover)
	float DeltaSeconds = 0.f;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = ChaosMover)
	bool bRequestedMoveWithMaxSpeed = false;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = ChaosMover)
	bool bShouldComputeAcceleration = false;
};

UCLASS(MinimalAPI)
class UChaosGroundMovementUtils : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/** Computes the local velocity at the supplied position of the hit object in floor result */
	UFUNCTION(BlueprintCallable, Category = ChaosMover)
	static CHAOSMOVER_API FVector ComputeLocalGroundVelocity_Internal(const FVector& Position, const FFloorCheckResult& FloorResult);

	/** Computes the local velocity at the supplied position of the hit object in floor result */
	UFUNCTION(BlueprintCallable, Category = ChaosMover)
	static CHAOSMOVER_API FProposedMove ComputeRequestedMove(const FRequestedMoveParams& Params);

	static CHAOSMOVER_API Chaos::FPBDRigidParticleHandle* GetRigidParticleHandleFromFloorResult_Internal(const FFloorCheckResult& FloorResult);
};
