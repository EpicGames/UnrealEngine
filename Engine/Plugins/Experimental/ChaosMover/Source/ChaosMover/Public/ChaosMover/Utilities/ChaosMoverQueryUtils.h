// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CollisionQueryParams.h"

struct FFloorCheckResult;
struct FWaterCheckResult;
class UWorld;

namespace UE::ChaosMover::Utils
{
	struct FFloorSweepParams
	{
		FCollisionResponseParams ResponseParams;
		FCollisionQueryParams QueryParams;
		FVector Location;
		FVector DeltaPos;
		FVector UpDir;
		const UWorld* World;
		float QueryDistance;
		float QueryRadius;
		float MaxWalkSlopeCosine;
		float TargetHeight;
		ECollisionChannel CollisionChannel;
	};

	extern CHAOSMOVER_API void FloorSweep_Internal(const FFloorSweepParams& Params, FFloorCheckResult& OutFloorResult, FWaterCheckResult& OutWaterResult);

	struct FCapsuleOverlapTestParams
	{
		FCollisionResponseParams ResponseParams;
		FCollisionQueryParams QueryParams;
		FVector Location;
		FVector UpDir;
		const UWorld* World;
		float CapsuleHalfHeight;
		float CapsuleRadius;
		ECollisionChannel CollisionChannel;
		bool bBlockOnDynamic = false;
	};

	extern CHAOSMOVER_API bool CapsuleOverlapTest_Internal(const FCapsuleOverlapTestParams& Params);

}